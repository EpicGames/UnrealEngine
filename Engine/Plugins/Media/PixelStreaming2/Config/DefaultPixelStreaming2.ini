[/Script/PixelStreaming2Settings.PixelStreaming2PluginSettings]
DefaultCursorClassName=/PixelStreaming2/DefaultCursor.DefaultCursor_C
TextEditBeamCursorClassName=/PixelStreaming2/TextEditBeamCursor.TextEditBeamCursor_C
HiddenCursorClassName=/PixelStreaming2/HiddenCursor.HiddenCursor_C

[CoreRedirects]
+ClassRedirects=(OldName="/Script/PixelStreaming2.PixelStreaming2VideoProducer",NewName="/Script/PixelStreaming2.PixelStreaming2VideoProducerBase")
# Remap all the old PixelStreaming2 blueprint classes to PixelStreaming2RTC
+ClassRedirects=(OldName="/Script/PixelStreaming2.PixelStreaming2Peer",NewName="/Script/PixelStreaming2RTC.PixelStreaming2Peer")

# Remap old PixelStreaming2Core to PixelStreaming2Settings
+ClassRedirects=(OldName="/Script/PixelStreaming2Core.PixelStreaming2PluginSettings",NewName="/Script/PixelStreaming2Settings.PixelStreaming2PluginSettings")
+EnumRedirects=(OldName="/Script/PixelStreaming2Core.EPortAllocatorFlags",NewName="/Script/PixelStreaming2Settings.EPortAllocatorFlags")
+EnumRedirects=(OldName="/Script/PixelStreaming2Core.EInputControllerMode",NewName="/Script/PixelStreaming2Settings.EInputControllerMode")
+EnumRedirects=(OldName="/Script/PixelStreaming2Core.EPixelStreaming2EditorStreamTypes",NewName="/Script/PixelStreaming2Settings.EPixelStreaming2EditorStreamTypes")
