{
	"FileVersion": 3,
	"Version": 1.0,
	"VersionName": "1.0",
	"FriendlyName": "Pixel Streaming 2",
	"Description": "Streaming of Unreal Engine audio and rendering to WebRTC-compatible media players such as a web browsers.",
	"Category": "Graphics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Platforms/PixelStreaming/index.html",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PixelStreaming2Core",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": []
		},
		{
			"Name": "PixelStreaming2",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": []
		},
		{
			"Name": "PixelStreaming2Settings",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": []
		},
		{
			"Name": "PixelStreaming2RTC",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": []
		},
		{
			"Name": "PixelStreaming2Servers",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": []
		},
		{
			"Name": "PixelStreaming2Editor",
			"Type": "Editor",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreaming2HMD",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PixelStreaming2Input",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "PixelCapture",
			"Enabled": true
		},
		{
			"Name": "WmfMedia",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		},
    {
      "Name": "AVCodecsCore",
      "Enabled": true,
      "PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ]
    },
		{
			"Name": "NVCodecs",
			"Enabled": true,
            "PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "AMFCodecs",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		},
        {
			"Name": "VTCodecs",
			"Enabled": true,
			"PlatformAllowList": [ "Mac", "IOS" ]
		},
		{
			"Name": "LibVpxCodecs",
			"Enabled": true,
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ]
		},
		{
			"Name": "WebSocketNetworking",
			"Enabled": true
		},
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		}
	]
}
