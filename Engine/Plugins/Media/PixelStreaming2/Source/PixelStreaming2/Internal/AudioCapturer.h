// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/MultithreadedPatching.h"
#include "AudioDeviceManager.h"
#include "AudioProducer.h"
#include "Misc/CoreDelegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "SampleBuffer.h"
#include "TickableTask.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FAudioCapturer;

	class FMixAudioTask : public FPixelStreamingTickableTask
	{
	public:
		UE_API FMixAudioTask(TSharedPtr<FAudioCapturer> Capturer);

		virtual ~FMixAudioTask() = default;

		// Begin FPixelStreamingTickableTask
		UE_API virtual void			  Tick(float DeltaMs) override;
		UE_API virtual const FString& GetName() const override;
		// End FPixelStreamingTickableTask

	protected:
		TWeakPtr<FAudioCapturer> Capturer;
	};

	class FAudioCapturer : public Audio::FPatchMixer, public TSharedFromThis<FAudioCapturer>
	{
	public:
		template <typename T>
		static TSharedPtr<T> Create(const int32 InSampleRate = 48000, const int32 InNumChannels = 2, const float InSampleSizeInSeconds = 0.5f);

		virtual ~FAudioCapturer() = default;

		// AudioProducer lifecycle methods triggered by engine delegates
		UE_API void CreateAudioProducer(Audio::FDeviceId AudioDeviceId);
		UE_API void RemoveAudioProducer(Audio::FDeviceId AudioDeviceId);

		// Methods for managing custom audio producers that will have their audio mixed in with engine audio
		UE_API void AddAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer);
		UE_API void RemoveAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer);

		// Called when mixed audio has been produced (and optionally dumped) and is ready to be sent for encoding
		UE_API virtual void PushAudio(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate);

		/**
		 * This is broadcast each time audio is captured. Tracks should bind to this and push the audio into the track
		 */
		DECLARE_TS_MULTICAST_DELEGATE_FourParams(FOnAudioBuffer, const int16_t*, int32, int32, const int32);
		FOnAudioBuffer OnAudioBuffer;

		/* */
		void ProcessAudio();
		/* */

	protected:
		UE_API FAudioCapturer(const int32 SampleRate = 48000, const int32 NumChannels = 2, const float SampleSizeInSeconds = 0.5f);

		// Required to be called by derived classes if the class is not constructed with FAudioCapturer::Create.
		// It initializes members that require TSharedPtr to FAudioCapturer
		UE_API void Initialize();

		UE_API void OnDebugDumpAudioChanged(IConsoleVariable* Var);
		UE_API void OnEnginePreExit();
		UE_API void WriteDebugAudio();

		/* */
		UE_API int32 GetSampleRate() const;
		UE_API int32  GetNumChannels() const;
		UE_API int32 GetMaxBufferSize() const;
		/* */

	protected:
		TSharedPtr<FMixAudioTask> MixerTask;

		// Audio producers created by the engine that capture audio from specific devices
		TMap<Audio::FDeviceId, TSharedPtr<FAudioProducer>> EngineAudioProducers;
		// Audio producers created by the user that push audio from arbitrary sources
		TMap<TSharedPtr<IPixelStreaming2AudioProducer>, TSharedPtr<FAudioProducer>> UserAudioProducers;

		int32	  SampleRate;
		int32	  NumChannels;
		float SampleSizeSeconds;

		Audio::TSampleBuffer<int16_t> DebugDumpAudioBuffer;
	};

	template <typename T>
	TSharedPtr<T> FAudioCapturer::Create(const int32 InSampleRate, const int32 InNumChannels, const float InSampleSizeInSeconds)
	{
		static_assert(std::is_base_of_v<FAudioCapturer, T>);
		TSharedPtr<T> AudioCapturer(new T(InSampleRate, InNumChannels, InSampleSizeInSeconds));
		AudioCapturer->Initialize();

		FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddSP(AudioCapturer.ToSharedRef(), &T::CreateAudioProducer);
		FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddSP(AudioCapturer.ToSharedRef(), &T::RemoveAudioProducer);

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnDebugDumpAudioChanged.AddSP(AudioCapturer.ToSharedRef(), &T::OnDebugDumpAudioChanged);

			TWeakPtr<T> WeakAudioMixingCapturer = AudioCapturer;
			FCoreDelegates::OnEnginePreExit.AddLambda([WeakAudioMixingCapturer]() {
				if (TSharedPtr<T> AudioCapturer = WeakAudioMixingCapturer.Pin())
				{
					AudioCapturer->OnEnginePreExit();
				}
			});
		}

		return AudioCapturer;
	}
} // namespace UE::PixelStreaming2

#undef UE_API
