// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio.h"
#include "DSP/MultithreadedPatching.h"
#include "IPixelStreaming2AudioProducer.h"
#include "ISubmixBufferListener.h"
#include "UtilsAudio.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FAudioCapturer;

	/**
	 * An audio input capable of listening to UE submix's as well as receiving user audio via the OnPushedAudio method.
	 * Any received audio will be pushed into the FPatchInput interface for mixing
	 */
	class FAudioProducer : public ISubmixBufferListener, public Audio::FPatchInput
	{
	public:
		static UE_API TSharedPtr<FAudioProducer> Create(Audio::FDeviceId AudioDeviceId, int32 TargetSampleRate, int32 TargetNumChannels, const Audio::FPatchOutputStrongPtr& PatchOutput);
		static UE_API TSharedPtr<FAudioProducer> Create(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer, int32 TargetSampleRate, int32 TargetNumChannels, const Audio::FPatchOutputStrongPtr& PatchOutput);
		virtual ~FAudioProducer() = default;

		// ISubmixBufferListener interface
		UE_API virtual void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) override;

		// Listener for audio pushed from custom IPixelStreaming2AudioProducer implementations
		UE_API void OnPushedAudio(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate);

	protected:
		UE_API FAudioProducer(int32 TargetSampleRate, int32 TargetNumChannels, const Audio::FPatchOutputStrongPtr& PatchOutput);

	protected:
		const int32 TargetSampleRate;
		const int32 TargetNumChannels;
		bool bIsMuted;

	private:
		void OnRawAudio(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate);
	};
} // namespace UE::PixelStreaming2

#undef UE_API
