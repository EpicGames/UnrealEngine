// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2AudioSink.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FAudioSink : public IPixelStreaming2AudioSink
	{
	public:
		// Note: destructor will call destroy on any attached audio consumers
		UE_API virtual ~FAudioSink();

		UE_API virtual void AddAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>& AudioConsumer) override;
		UE_API virtual void RemoveAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>& AudioConsumer) override;

		UE_API bool HasAudioConsumers();

		UE_API void SetMuted(bool bIsMuted);

		UE_API void OnAudioData(int16_t* AudioData, uint32 NumFrames, uint32 NumChannels, uint32 SampleRate);

	protected:
		TArray<TWeakPtrVariant<IPixelStreaming2AudioConsumer>> AudioConsumers;

	private:
		FCriticalSection AudioConsumersCS;
		bool			 bIsMuted = false;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
