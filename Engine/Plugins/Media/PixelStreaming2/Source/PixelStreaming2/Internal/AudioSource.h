// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FAudioSource
	{
	public:
		virtual ~FAudioSource() = default;

		UE_API virtual void OnAudioBuffer(const int16_t* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate);

		UE_API void SetMuted(bool bIsMuted);

	protected:
		bool bIsMuted = false;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
