// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2DDynamic.h"
#include "IPixelStreaming2VideoConsumer.h"

#include "PixelStreaming2MediaTexture.generated.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FPixelStreaming2MediaTextureResource;
} // namespace UE::PixelStreaming2

/**
 * A Texture Object that can be used in materials etc. that takes updates from webrtc frames.
 */
UCLASS(MinimalAPI, NotBlueprintType, NotBlueprintable, HideDropdown, HideCategories = (ImportSettings, Compression, Texture, Adjustments, Compositing, LevelOfDetail, Object), META = (DisplayName = "PixelStreaming Media Texture"))
class UPixelStreaming2MediaTexture : public UTexture2DDynamic
{
	GENERATED_UCLASS_BODY()

public:
	// IPixelStreamingVideoConsumer implementation
	UE_API void ConsumeFrame(FTextureRHIRef Frame);

	UE_API const FTextureResource* GetResource() const;

protected:
	// UObject overrides.
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	// UTexture implementation
	UE_API virtual FTextureResource* CreateResource() override;

private:
	UE_API void InitializeResources();

	// updates the internal texture resource after each frame.
	UE_API void UpdateTextureReference(FRHICommandList& RHICmdList, FTextureRHIRef Reference);

	FCriticalSection RenderSyncContext;
	// NOTE: This type has to be a raw ptr because UE will manage the lifetime of it
	UE::PixelStreaming2::FPixelStreaming2MediaTextureResource* CurrentResource;
};

#undef UE_API
