// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTarget2D.h"
#include "IPixelStreaming2VideoProducer.h"

#include "PixelStreaming2VideoProducer.generated.h"

#define UE_API PIXELSTREAMING2_API

/**
 * The base video producer blueprint class
 */
UCLASS(MinimalAPI, NotBlueprintType, NotBlueprintable, Category = "PixelStreaming2", META = (DisplayName = "Video Producer"))
class UPixelStreaming2VideoProducerBase : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<IPixelStreaming2VideoProducer> GetVideoProducer() { return VideoProducer; }

protected:
	TSharedPtr<IPixelStreaming2VideoProducer> VideoProducer;
};

/**
 * A video producer that streams the Unreal Engine's back buffer
 */
UCLASS(MinimalAPI, NotBlueprintType, NotBlueprintable, Category = "PixelStreaming2", META = (DisplayName = "Back Buffer Video Producer"))
class UPixelStreaming2VideoProducerBackBuffer : public UPixelStreaming2VideoProducerBase
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual TSharedPtr<IPixelStreaming2VideoProducer> GetVideoProducer() override;
};

/**
 * A video producer that streams the Unreal Engine's back buffer but uses the MediaIO Framework to capture the frame rather than Pixel Capture
 */
UCLASS(MinimalAPI, NotBlueprintType, NotBlueprintable, Category = "PixelStreaming2", META = (DisplayName = "Media Capture Video Producer"))
class UPixelStreaming2VideoProducerMediaCapture : public UPixelStreaming2VideoProducerBase
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual TSharedPtr<IPixelStreaming2VideoProducer> GetVideoProducer() override;
};

/**
 * A video producer that streams the specified render target
 */
UCLASS(MinimalAPI, NotBlueprintType, NotBlueprintable, Category = "PixelStreaming2", META = (DisplayName = "Render Target Video Producer"))
class UPixelStreaming2VideoProducerRenderTarget : public UPixelStreaming2VideoProducerBase
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Video, AssetRegistrySearchable)
	TObjectPtr<UTextureRenderTarget2D> Target;

	UE_API virtual TSharedPtr<IPixelStreaming2VideoProducer> GetVideoProducer() override;
};

#undef UE_API
