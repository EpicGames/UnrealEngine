// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprints/PixelStreaming2InputComponent.h"
#include "ThreadSafeMap.h"

namespace UE::PixelStreaming2
{
	extern PIXELSTREAMING2_API TThreadSafeMap<uintptr_t, UPixelStreaming2Input*> InputComponents;
} // namespace UE::PixelStreaming2
