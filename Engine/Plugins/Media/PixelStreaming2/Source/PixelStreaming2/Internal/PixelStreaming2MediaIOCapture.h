// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StripedMap.h"
#include "MediaCapture.h"
#include "Slate/SceneViewport.h"
#include "VideoProducer.h"

#include "PixelStreaming2MediaIOCapture.generated.h"

#define UE_API PIXELSTREAMING2_API

UCLASS(MinimalAPI, BlueprintType)
class UPixelStreaming2MediaIOCapture : public UMediaCapture
{
	GENERATED_BODY()

	//~ Begin UMediaCapture interface
public:
	/**
	 * GPU copy methods
	 */
	UE_API virtual void OnRHIResourceCaptured_RenderingThread(
		FRHICommandListImmediate&							   RHICmdList,
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		FTextureRHIRef										   InTexture) override;

	/**
	 * Custom conversion operation for Mac
	 */
	UE_API virtual void OnCustomCapture_RenderingThread(
		FRDGBuilder&										   GraphBuilder,
		const FCaptureBaseData&								   InBaseData,
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
		FRDGTextureRef										   InSourceTexture,
		FRDGTextureRef										   OutputTexture,
		const FRHICopyTextureInfo&							   CopyInfo,
		FVector2D											   CropU,
		FVector2D											   CropV) override;

	UE_API virtual bool InitializeCapture() override;
	UE_API virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool ShouldCaptureRHIResource() const { return bDoGPUCopy; }
	UE_API virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	// We override the texture flags because on Mac we want the texture to have the CPU_Readback flag
	UE_API virtual ETextureCreateFlags GetOutputTextureFlags() const override;
	UE_API virtual void WaitForGPU(FRHITexture* InRHITexture) override;
	UE_API virtual TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> GetCaptureFrameUserData_GameThread() override;
	//~ End UMediaCapture interface

	TSharedPtr<FSceneViewport>				GetViewport() const { return SceneViewport.Pin(); }
	UE_API virtual void							ViewportResized(FViewport* Viewport, uint32 ResizeCode);
	bool									WasViewportResized() const { return bViewportResized; }
	void									SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> InVideoProducer) { VideoProducer = InVideoProducer; }
	TWeakPtr<IPixelStreaming2VideoProducer> GetVideoProducer() { return VideoProducer; }

	DECLARE_MULTICAST_DELEGATE(FOnCaptureViewportInitialized);
	FOnCaptureViewportInitialized OnCaptureViewportInitialized;

private:
	UE_API void HandleCapturedFrame(FTextureRHIRef InTexture, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData);
	UE_API void UpdateCaptureResolution(int32 Width, int32 Height);

private:
	TWeakPtr<FSceneViewport>				SceneViewport;
	TWeakPtr<IPixelStreaming2VideoProducer> VideoProducer;

	/* We track whether the viewport has been resized since we created this capturer as resize means restart capturer. */
	bool bViewportResized = false;

	/* Tracks the captured resolution, we use this to determine if resize events contain a different resolution than what we previously captured. */
	TUniquePtr<FIntPoint> CaptureResolution;

	/* Whether we want the UMediaCapture to read back to frame into cpu memory or not. */
	bool bDoGPUCopy = true;

	TStripedMap<32, uintptr_t, FGPUFenceRHIRef> Fences;
};

#undef UE_API
