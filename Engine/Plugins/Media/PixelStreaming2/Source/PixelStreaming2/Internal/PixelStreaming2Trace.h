// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Trace.h"

UE_TRACE_CHANNEL_EXTERN(PixelStreaming2Channel, PIXELSTREAMING2_API);
