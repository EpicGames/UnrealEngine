// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SoundGenerator.h"

#include <atomic>

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	/**
	 * An `ISoundGenerator` implementation to pump some audio from EpicRtc into this synth component
	 */
	class FSoundGenerator : public ::ISoundGenerator
	{
	public:
		UE_API FSoundGenerator();
		virtual ~FSoundGenerator() = default;

		// Called when a new buffer is required.
		UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;

		// Returns the number of samples to render per callback
		UE_API virtual int32 GetDesiredNumSamplesToRenderPerCallback() const;

		// Optional. Called on audio generator thread right when the generator begins generating.
		UE_API virtual void OnBeginGenerate();

		// Optional. Called on audio generator thread right when the generator ends generating.
		UE_API virtual void OnEndGenerate();

		// Optional. Can be overridden to end the sound when generating is finished.
		UE_API virtual bool IsFinished() const;

		UE_API void AddAudio(const int16* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames);

		UE_API bool IsInitialized();
		UE_API void  EmptyBuffers();
		UE_API void  Initialize(const FSoundGeneratorInitParams& InitParams);

	private:
		FSoundGeneratorInitParams Params;
		TArray<float>			  Buffer;
		FCriticalSection		  CriticalSection;

	public:
		std::atomic<bool> bInitialized = false;
		std::atomic<bool> bGeneratingAudio = false;
		std::atomic<bool> bShouldGenerateAudio = false;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
