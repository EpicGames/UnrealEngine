// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Tickable.h"
#include "IPixelStreaming2Streamer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	/**
	 * An interval timer that triggers reconnects when `IPixelStreaming2Streamer` has it session disconnect.
	 * Timer takes a `TWeakPtr<IPixelStreaming2Streamer>`, meaning the streamer object must exist for the timer to do anything.
	 * Note: this timer runs on the game thread and its reconnect interval is controlled by the CVar
	 * UPixelStreaming2PluginSettings::CVarSignalingReconnectInterval
	 */
	class FStreamerReconnectTimer : public FTickableGameObject
	{
	public:
		FStreamerReconnectTimer() = default;
		virtual ~FStreamerReconnectTimer() = default;
		UE_API void Start(TWeakPtr<IPixelStreaming2Streamer> InWeakStreamer);
		UE_API void Stop();
		UE_API void Reset();

		// Begin FTickableGameObject
		UE_API virtual void		Tick(float DeltaTime) override;
		virtual bool		IsTickableInEditor() const override { return true; }
		virtual bool		IsTickableWhenPaused() const override { return true; }
		FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(PixelStreaming2StreamerReconnectTimer, STATGROUP_Tickables); }
		// End FTickableGameObject

		DECLARE_EVENT(FStreamerReconnectTimer, FOnExceededMaxReconnectAttempts);
		FOnExceededMaxReconnectAttempts OnExceededMaxReconnectAttempts;

	private:
		TWeakPtr<IPixelStreaming2Streamer> WeakStreamer;

		bool							   bEnabled = false;
		uint64							   NumReconnectAttempts = 0;
		uint64							   LastReconnectCycles = 0;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
