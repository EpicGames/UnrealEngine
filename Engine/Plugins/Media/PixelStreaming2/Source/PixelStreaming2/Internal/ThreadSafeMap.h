// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Misc/ScopeLock.h"

namespace UE::PixelStreaming2
{
	/** A TThreadSafeMapBase specialization that only allows a single value associated with each key whilst being thread safe.*/
	template <typename InKeyType, typename InValueType, typename SetAllocator = FDefaultSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, false>>
	class TThreadSafeMap : protected TSortableMapBase<InKeyType, InValueType, SetAllocator, KeyFuncs>
	{
		template <typename, typename>
		friend class TScriptThreadSafeMap;

		static_assert(!KeyFuncs::bAllowDuplicateKeys, "TThreadSafeMap cannot be instantiated with a KeyFuncs which allows duplicate keys");

	public:
		typedef InKeyType	 KeyType;
		typedef InValueType	 ValueType;
		typedef SetAllocator SetAllocatorType;
		typedef KeyFuncs	 KeyFuncsType;

		typedef TSortableMapBase<KeyType, ValueType, SetAllocator, KeyFuncs> Super;
		typedef typename Super::KeyInitType									 KeyInitType;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.8, "KeyConstPointerType is deprecated - use KeyInitType instead.")
		typedef typename Super::KeyConstPointerType							 KeyConstPointerType;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TThreadSafeMap() = default;
		TThreadSafeMap(TThreadSafeMap&&) = default;
		TThreadSafeMap(const TThreadSafeMap&) = default;
		TThreadSafeMap& operator=(TThreadSafeMap&&) = default;
		TThreadSafeMap& operator=(const TThreadSafeMap&) = default;

		/** Constructor for moving elements from a TThreadSafeMap with a different SetAllocator */
		template <typename OtherSetAllocator>
		TThreadSafeMap(TThreadSafeMap<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
			: Super(MoveTemp(Other))
		{
		}

		/** Constructor for copying elements from a TThreadSafeMap with a different SetAllocator */
		template <typename OtherSetAllocator>
		TThreadSafeMap(const TThreadSafeMap<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
			: Super(Other)
		{
		}

		/** Constructor which gets its elements from a native initializer list */
		TThreadSafeMap(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
		{
			FScopeLock Lock(&Mutex);

			this->Reserve((int32)InitList.size());
			for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
			{
				this->Add(Element.Key, Element.Value);
			}
		}

		/** Assignment operator for moving elements from a TThreadSafeMap with a different SetAllocator */
		template <typename OtherSetAllocator>
		TThreadSafeMap& operator=(TThreadSafeMap<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& Other)
		{
			// Move Pairs so they they are destroyed outside of the ScopeLock
			TSet<TPair<KeyType, ValueType>, KeyFuncs, SetAllocator> MovedPairs;
			{
				FScopeLock Lock(&Mutex);
				MovedPairs = MoveTemp(Super::Pairs);

				(Super&)* this = MoveTemp(Other);
			}
			MovedPairs.Empty();
			return *this;
		}

		/** Assignment operator for copying elements from a TThreadSafeMap with a different SetAllocator */
		template <typename OtherSetAllocator>
		TThreadSafeMap& operator=(const TThreadSafeMap<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& Other)
		{
			// Move Pairs so they they are destroyed outside of the ScopeLock
			TSet<TPair<KeyType, ValueType>, KeyFuncs, SetAllocator> MovedPairs;
			{
				FScopeLock Lock(&Mutex);
				MovedPairs = MoveTemp(Super::Pairs);

				(Super&)* this = Other;
			}
			MovedPairs.Empty();
			return *this;
		}

		/** Assignment operator which gets its elements from a native initializer list */
		TThreadSafeMap& operator=(std::initializer_list<TPairInitializer<const KeyType&, const ValueType&>> InitList)
		{
			FScopeLock Lock(&Mutex);

			this->Empty((int32)InitList.size());
			for (const TPairInitializer<const KeyType&, const ValueType&>& Element : InitList)
			{
				this->Add(Element.Key, Element.Value);
			}
			return *this;
		}

		/**
		 * Remove the pair with the specified key and copies the value
		 * that was removed to the ref parameter
		 *
		 * @param Key The key to search for
		 * @param OutRemovedValue If found, the value that was removed (not modified if the key was not found)
		 * @return whether or not the key was found
		 */
		FORCEINLINE bool RemoveAndCopyValue(KeyInitType Key, ValueType& OutRemovedValue)
		{
			FScopeLock Lock(&Mutex);

			const FSetElementId PairId = Super::Pairs.FindId(Key);
			if (!PairId.IsValidId())
			{
				return false;
			}

			OutRemovedValue = MoveTempIfPossible(Super::Pairs[PairId].Value);
			Super::Pairs.Remove(PairId);
			return true;
		}

		/** See RemoveAndCopyValue() and class documentation section on ByHash() functions */
		template <typename ComparableKey>
		FORCEINLINE bool RemoveAndCopyValueByHash(uint32 KeyHash, const ComparableKey& Key, ValueType& OutRemovedValue)
		{
			FScopeLock Lock(&Mutex);

			const FSetElementId PairId = Super::Pairs.FindIdByHash(KeyHash, Key);
			if (!PairId.IsValidId())
			{
				return false;
			}

			OutRemovedValue = MoveTempIfPossible(Super::Pairs[PairId].Value);
			Super::Pairs.Remove(PairId);
			return true;
		}

		/**
		 * Find a pair with the specified key, removes it from the map, and returns the value part of the pair.
		 *
		 * If no pair was found, an exception is thrown.
		 *
		 * @param Key the key to search for
		 * @return whether or not the key was found
		 */
		FORCEINLINE ValueType FindAndRemoveChecked(KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			const FSetElementId PairId = Super::Pairs.FindId(Key);
			check(PairId.IsValidId());
			ValueType Result = MoveTempIfPossible(Super::Pairs[PairId].Value);
			Super::Pairs.Remove(PairId);
			return Result;
		}

		/**
		 * Move all items from another map into our map (if any keys are in both,
		 * the value from the other map wins) and empty the other map.
		 *
		 * @param OtherMap The other map of items to move the elements from.
		 */
		template <typename OtherSetAllocator>
		void Append(TThreadSafeMap<KeyType, ValueType, OtherSetAllocator, KeyFuncs>&& OtherMap)
		{
			FScopeLock Lock(&Mutex);

			this->Reserve(this->Num() + OtherMap.Num());
			for (auto& Pair : OtherMap)
			{
				this->Add(MoveTempIfPossible(Pair.Key), MoveTempIfPossible(Pair.Value));
			}

			OtherMap.Reset();
		}

		/**
		 * Add all items from another map to our map (if any keys are in both,
		 * the value from the other map wins).
		 *
		 * @param OtherMap The other map of items to add.
		 */
		template <typename OtherSetAllocator>
		void Append(const TThreadSafeMap<KeyType, ValueType, OtherSetAllocator, KeyFuncs>& OtherMap)
		{
			FScopeLock Lock(&Mutex);

			this->Reserve(this->Num() + OtherMap.Num());
			for (auto& Pair : OtherMap)
			{
				this->Add(Pair.Key, Pair.Value);
			}
		}

		FORCEINLINE ValueType& operator[](KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->FindChecked(Key);
		}

		FORCEINLINE const ValueType& operator[](KeyInitType Key) const
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->FindChecked(Key);
		}

		// Note: Do not call a map method inside a visitor or you will deadlock
		template <typename T>
		void Apply(T&& Visitor)
		{
			FScopeLock Lock(&Mutex);
			for (typename Super::ElementSetType::TIterator It(Super::Pairs); It; ++It)
			{
				Visitor(It->Key, It->Value);
			}
		}

		// Note: Do not call a map method inside a visitor or you will deadlock
		template <typename T>
		void ApplyUntil(T&& Visitor)
		{
			FScopeLock Lock(&Mutex);
			for (typename Super::ElementSetType::TIterator It(Super::Pairs); It; ++It)
			{
				if (Visitor(It->Key, It->Value))
				{
					break;
				}
			}
		}

		FORCEINLINE ValueType& FindOrAdd(KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->FindOrAdd(Key);
		}

		FORCEINLINE ValueType* Find(KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Find(Key);
		}

		FORCEINLINE ValueType FindRef(KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->FindRef(Key);
		}

		FORCEINLINE int32 Remove(KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Remove(Key);
		}

		FORCEINLINE bool IsEmpty()
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->IsEmpty();
		}

		FORCEINLINE void Empty(int32 ExpectedNumElements = 0)
		{
			// Move Pairs so they they are destroyed outside of the ScopeLock
			TSet<TPair<KeyType, ValueType>, KeyFuncs, SetAllocator> MovedPairs;
			{
				FScopeLock Lock(&Mutex);
				MovedPairs = MoveTemp(Super::Pairs);
			}
			MovedPairs.Empty(ExpectedNumElements);
		}

		template <typename InSetKeyFuncs, typename InSetAllocator>
		int32 GetKeys(TSet<KeyType, InSetKeyFuncs, InSetAllocator>& OutKeys)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->GetKeys(OutKeys);
		}

		[[nodiscard]] FORCEINLINE bool Contains(KeyInitType Key)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Contains(Key);
		}

		FORCEINLINE ValueType& Add(const KeyType& InKey, const ValueType& InValue)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Emplace(InKey, InValue);
		}

		FORCEINLINE ValueType& Add(const KeyType& InKey, ValueType&& InValue)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Emplace(InKey, MoveTempIfPossible(InValue));
		}

		FORCEINLINE ValueType& Add(KeyType&& InKey, const ValueType& InValue)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Emplace(MoveTempIfPossible(InKey), InValue);
		}

		FORCEINLINE ValueType& Add(KeyType&& InKey, ValueType&& InValue)
		{
			FScopeLock Lock(&Mutex);

			return static_cast<Super*>(this)->Emplace(MoveTempIfPossible(InKey), MoveTempIfPossible(InValue));
		}

	private:
		mutable FCriticalSection Mutex;
	};

} // namespace UE::PixelStreaming2