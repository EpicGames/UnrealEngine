// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	template <typename T>
	using TUniqueTaskPtr = TUniquePtr<T, struct FPixelStreamingTaskDeleter>;

	/**
	 * Base class for a tickable task. Inheriting from this class ensures that your task can be ticked by the PixelStreaming thread.
	 * Note it is important to destroy the task when it is no longer in use to save the tick thread.
	 */
	class FPixelStreamingTickableTask : public TSharedFromThis<FPixelStreamingTickableTask>
	{
	public:
		/**
		 * Classes derived from FPixelStreamingTickableTask must construct themselves using this Create method.
		 * Using this Create method ensures the class is fully constructed at the time it is added to
		 * the PixelStreaming thread.
		 */
		template <typename T, typename... Args>
		static TSharedPtr<T> Create(Args&&... InArgs);

	public:
		virtual ~FPixelStreamingTickableTask() = default;

		virtual void Tick(float DeltaMs) { /* Purposeful no-op to avoid pure virtual call if task is ticked mid construction. */ };

		virtual const FString& GetName() const = 0;

	protected:
		FPixelStreamingTickableTask() = default;

	private:
		UE_API void Register();
		UE_API void Unregister();

		// Allow FPixelStreamingTaskDeleter to access the Unregiser method
		friend FPixelStreamingTaskDeleter;
	};

	/**
	 * TUniquePtr custom deleter that handle automatic unregistering of the task.
	 * This ensure that the task won't try to be ticked mid-deletion
	 */
	struct FPixelStreamingTaskDeleter
	{
		template <typename T>
		void operator()(T* Ptr) const
		{
			static_assert(std::is_base_of_v<FPixelStreamingTickableTask, T>);
			Ptr->Unregister();
			delete Ptr;
		}
	};

	template <typename T, typename... Args>
	TSharedPtr<T> FPixelStreamingTickableTask::Create(Args&&... InArgs)
	{
		static_assert(std::is_base_of_v<FPixelStreamingTickableTask, T>);

		TSharedPtr<T> Task(new T(Forward<Args>(InArgs)...), FPixelStreamingTaskDeleter());
		Task->Register();

		return Task;
	}

} // namespace UE::PixelStreaming2

#undef UE_API
