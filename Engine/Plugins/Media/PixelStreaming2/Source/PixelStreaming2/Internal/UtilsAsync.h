// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"

namespace UE::PixelStreaming2
{
	template <typename T>
	void DoOnGameThread(T&& Func)
	{
		if (IsInGameThread())
		{
			Func();
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, [Func]() { Func(); });
		}
	}

	template <typename T>
	void DoOnGameThreadAndWait(uint32 Timeout, T&& Func)
	{
		if (IsInGameThread())
		{
			Func();
		}
		else
		{
			FEvent* TaskEvent = FPlatformProcess::GetSynchEventFromPool();
			AsyncTask(ENamedThreads::GameThread, [Func, TaskEvent]() {
				Func();
				TaskEvent->Trigger();
			});
			TaskEvent->Wait(Timeout);
			FPlatformProcess::ReturnSynchEventToPool(TaskEvent);
		}
	}
} // namespace UE::PixelStreaming2
