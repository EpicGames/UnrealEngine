// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioResampler.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/ConvertDeinterleave.h"
#include "DSP/FloatArrayMath.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	template <typename T>
	struct TRawAudio
	{
		T*	  Data;
		int32 NumSamples;
        int32 SampleRate;
		int32 NumChannels;
        float Gain = 1.f;
	};

	struct FProcessedAudio
	{
		Audio::FAlignedFloatBuffer& Data;
        int32 NumSamples;
        int32 SampleRate;
		int32 NumChannels;
        float Gain = 1.f;
	};

	template <typename T>
	bool ProcessAudio(const TRawAudio<T>& RawAudio, FProcessedAudio& OutProcessedAudio)
	{
		static_assert(std::is_same_v<std::decay_t<T>, int16> || std::is_same_v<std::decay_t<T>, float>, "Unsupported audio sample type");

        if (RawAudio.Data == nullptr || RawAudio.NumChannels <= 0 || RawAudio.SampleRate <= 0 || RawAudio.NumSamples <= 0 || RawAudio.Gain <= 0.f)
        {
            return false;
        }

        if (OutProcessedAudio.SampleRate <= 0 || OutProcessedAudio.NumChannels <= 0 || OutProcessedAudio.Gain < 0.f)
        {
            return false;
        }

		TArray<float> RawAudioData;
		RawAudioData.SetNumUninitialized(RawAudio.NumSamples);
		if constexpr (std::is_same_v<std::decay_t<T>, int16>)
		{
			Audio::ArrayPcm16ToFloat(MakeArrayView(RawAudio.Data, RawAudio.NumSamples), RawAudioData);
		}
		else
		{
			FMemory::Memcpy(RawAudioData.GetData(), RawAudio.Data, RawAudio.NumSamples * sizeof(float));
		}

		if (RawAudio.SampleRate == OutProcessedAudio.SampleRate && RawAudio.NumChannels == OutProcessedAudio.NumChannels && RawAudio.Gain == OutProcessedAudio.Gain)
		{
			// No processing needed, just copy the data
            int32 NumOutSamples = RawAudioData.Num();
            OutProcessedAudio.Data.SetNumUninitialized(NumOutSamples);
            FMemory::Memcpy(OutProcessedAudio.Data.GetData(), RawAudioData.GetData(), NumOutSamples * sizeof(float));
			OutProcessedAudio.NumSamples = NumOutSamples;
			return true;
		}

		TUniquePtr<Audio::IConvertDeinterleave> Converter = Audio::IConvertDeinterleave::Create({ .NumInputChannels = RawAudio.NumChannels, .NumOutputChannels = OutProcessedAudio.NumChannels });
		if (!Converter)
		{
			return false;
		}

		Audio::FMultichannelBuffer MultichannelBuffer;
		Audio::SetMultichannelBufferSize(OutProcessedAudio.NumChannels, RawAudio.NumSamples / RawAudio.NumChannels, MultichannelBuffer);
		// ProcessAudio also handles the up/down mixing of channels
		Converter->ProcessAudio(RawAudioData, MultichannelBuffer);

		for (int32 ChannelIndex = 0; ChannelIndex < OutProcessedAudio.NumChannels; ChannelIndex++)
		{
			// Operate on the channels individually
			Audio::FAlignedFloatBuffer& ChannelBuffer = MultichannelBuffer[ChannelIndex];

			// Resample if required
			if (RawAudio.SampleRate != OutProcessedAudio.SampleRate)
			{
				Audio::FResamplingParameters ResamplerParameters = {
					.ResamplerMethod = Audio::EResamplingMethod::Linear,
					.NumChannels = 1,
					.SourceSampleRate = static_cast<float>(RawAudio.SampleRate),
					.DestinationSampleRate = static_cast<float>(OutProcessedAudio.SampleRate),
					.InputBuffer = ChannelBuffer
				};

				Audio::FAlignedFloatBuffer ResamplerOutputData;
				ResamplerOutputData.AddUninitialized(Audio::GetOutputBufferSize(ResamplerParameters));

				Audio::FResamplerResults ResamplerResults;
				ResamplerResults.OutBuffer = &ResamplerOutputData;

				if (!Audio::Resample(ResamplerParameters, ResamplerResults))
				{
					return false;
				}

				int32 NumSamplesGenerated = ResamplerResults.OutputFramesGenerated;
				ChannelBuffer.SetNumZeroed(NumSamplesGenerated);
				FMemory::Memcpy(ChannelBuffer.GetData(), ResamplerOutputData.GetData(), NumSamplesGenerated * sizeof(float));
			}

            if (RawAudio.Gain != OutProcessedAudio.Gain)
			{
				// This approach does loop over the channel buffer twice but can take advantage of SIMD. May be quicker?
				Audio::ArrayMultiplyByConstantInPlace(ChannelBuffer, OutProcessedAudio.Gain / RawAudio.Gain);
				Audio::ArrayClampInPlace(ChannelBuffer, -1.f, 1.f);
			}
		}

		Audio::FAlignedFloatBuffer InterleavedBuffer;
		Audio::ArrayInterleave(MultichannelBuffer, InterleavedBuffer);

        int32 NumOutSamples = InterleavedBuffer.Num();
        OutProcessedAudio.Data.SetNumUninitialized(NumOutSamples);
        FMemory::Memcpy(OutProcessedAudio.Data.GetData(), InterleavedBuffer.GetData(), NumOutSamples * sizeof(float));
		OutProcessedAudio.NumSamples = NumOutSamples;

		return true;
	}
} // namespace UE::PixelStreaming2

#undef UE_API