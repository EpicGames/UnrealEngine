// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/VideoDecoder.h"
#if PLATFORM_WINDOWS
	#include "DynamicRHI.h"
	#include "Video/Resources/D3D/VideoResourceD3D.h"
	#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#elif PLATFORM_LINUX
	#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#elif PLATFORM_APPLE
	#include "Video/Resources/Metal/VideoResourceMetal.h"
#endif
#include "Video/Resources/VideoResourceCPU.h"

namespace UE::PixelStreaming2
{
	/**
	 * As windows supports many RHIs and many codecs, we need to check at runtime if the current codec and RHI is compatible.
	 *
	 * To remove nested switch-cases, this function is templated to take a video encoder config of the target codec. eg:
	 *
	 * IsHardwareEncoderSupported<FVideoEncoderConfigH264>();
	 * OR
	 * IsHardwareEncoderSupported<FVideoEncoderConfigAV1>();
	 * OR
	 * ...
	 */
	template <typename TCodec>
	bool IsHardwareEncoderSupported()
	{
		TSharedRef<FAVInstance> const Instance = MakeShared<FAVInstance>();
#if PLATFORM_WINDOWS
		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		switch (RHIType)
		{
			case ERHIInterfaceType::D3D11:
				return FVideoEncoder::IsSupported<FVideoResourceD3D11, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::D3D12:
				return FVideoEncoder::IsSupported<FVideoResourceD3D12, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::Vulkan:
				return FVideoEncoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			default:
				return false;
		}
#elif PLATFORM_LINUX
		return FVideoEncoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#elif PLATFORM_APPLE
		return FVideoEncoder::IsSupported<FVideoResourceMetal, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#else
		return false;
#endif
	}

	template <typename TCodec>
	bool IsSoftwareEncoderSupported()
	{
		TSharedRef<FAVInstance> const Instance = MakeShared<FAVInstance>();
		return FVideoEncoder::IsSupported<FVideoResourceCPU, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
	}

	template <typename TCodec>
	bool IsEncoderSupported()
	{
		return IsHardwareEncoderSupported<TCodec>() || IsSoftwareEncoderSupported<TCodec>();
	}

	/**
	 * As windows supports many RHIs and many codecs, we need to check at runtime if the current codec and RHI is compatible.
	 *
	 * To remove nested switch-cases, this function is templated to take a video decoder config of the target codec. eg:
	 *
	 * IsHardwareDecoderSupported<FVideoDecoderConfigVP8>();
	 * OR
	 * IsHardwareDecoderSupported<FVideoDecoderConfigAV1>();
	 * OR
	 * ...
	 */
	template <typename TCodec>
	bool IsHardwareDecoderSupported()
	{
		TSharedRef<FAVInstance> const Instance = MakeShared<FAVInstance>();
#if PLATFORM_WINDOWS
		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		switch (RHIType)
		{
			case ERHIInterfaceType::D3D11:
				return FVideoDecoder::IsSupported<FVideoResourceD3D11, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::D3D12:
				return FVideoDecoder::IsSupported<FVideoResourceD3D12, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			case ERHIInterfaceType::Vulkan:
				return FVideoDecoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
			default:
				return false;
		}
#elif PLATFORM_LINUX
		return FVideoDecoder::IsSupported<FVideoResourceVulkan, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#elif PLATFORM_APPLE
		return FVideoDecoder::IsSupported<FVideoResourceMetal, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
#else
		return false;
#endif
	}

	template <typename TCodec>
	bool IsSoftwareDecoderSupported()
	{
		TSharedRef<FAVInstance> const Instance = MakeShared<FAVInstance>();
		return FVideoDecoder::IsSupported<FVideoResourceCPU, TCodec>(FAVDevice::GetHardwareDevice(), Instance);
	}

	template <typename TCodec>
	bool IsDecoderSupported()
	{
		return IsHardwareDecoderSupported<TCodec>() || IsSoftwareDecoderSupported<TCodec>();
	}
} // namespace UE::PixelStreaming2