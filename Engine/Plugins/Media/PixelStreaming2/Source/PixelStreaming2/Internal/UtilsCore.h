// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <VersionHelpers.h>
THIRD_PARTY_INCLUDES_END
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace UE::PixelStreaming2
{
	
	// HACK (aidan.possemiers) the AVCodecs API surface wants a SharedPtr for the encoded data but EpicRtc already owns that and we don't want AVCodecs to delete it
	template<typename T>
	struct TFakeDeleter
	{
		void operator()(T* Object) const
		{
		}
	};

	inline bool IsStreamingSupported()
	{
		// Pixel Streaming does not make sense without an RHI so we don't run in commandlets without one.
		if (IsRunningCommandlet() && !IsAllowCommandletRendering())
		{
			return false;
		}

		return true;
	}

	inline bool IsPlatformSupported()
	{
		bool bCompatible = true;

#if PLATFORM_WINDOWS
		bool bWin8OrHigher = IsWindows8OrGreater();
		if (!bWin8OrHigher)
		{
			bCompatible = false;
			UE_CALL_ONCE([]() {
				FString ErrorString(TEXT("Failed to initialize Pixel Streaming plugin because minimum requirement is Windows 8"));
				FText	ErrorText = FText::FromString(ErrorString);
				FText	TitleText = FText::FromString(TEXT("Pixel Streaming Plugin"));
				FMessageDialog::Open(EAppMsgType::Ok, ErrorText, TitleText);
			});
		}
#endif
		return bCompatible;
	}
} // namespace UE::PixelStreaming2