// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreaming2PluginSettings.h"
#include "UtilsCore.h"

namespace UE::PixelStreaming2
{
	// List of video codecs supported by PS2
	const TStaticArray<EVideoCodec, 4> SupportedVideoCodecs = { EVideoCodec::H264, EVideoCodec::AV1, EVideoCodec::VP8, EVideoCodec::VP9 };

	constexpr uint32_t NumSimulcastLayers = 3;
	// Each subsequent layer is 1/ScalingFactor the size of the previous
	constexpr uint32_t ScalingFactor = 2;

	/**
	 * A struct representing the simulcast paramaters of a single simulcast layer used by PixelStreaming2.
	 * Specifically, each layer has a `Scaling`, `MinBitrate` and `MaxBitrate`.
	 */
	struct FPixelStreaming2SimulcastLayer
	{
		float Scaling;
		int	  MinBitrate;
		int	  MaxBitrate;
	};

	inline TArray<FPixelStreaming2SimulcastLayer> GetSimulcastParameters()
	{
		TArray<FPixelStreaming2SimulcastLayer> SimulcastParams;

		if (UPixelStreaming2PluginSettings::CVarEncoderEnableSimulcast.GetValueOnAnyThread())
		{
			int MinBps = UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread();
			// We have to halve the maximum here due to the layer assignments max bitrates effectively summing to 2
			// 1/3 + 2/3 + 3/3
			int MaxBps = UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread() / 2;

			int OneThird = MaxBps / 3;
			int TwoThird = 2 * MaxBps / 3;
			// Bitrates assignment per layer
			// 0: 2/3 -> 3/3
			// 1: 1/3 -> 2/3
			// 2: 0 -> 1/3
			TArray<TTuple<int, int>> Bitrates = {
				{ TwoThird, MaxBps },
				{ OneThird, TwoThird },
				{ MinBps, OneThird }
			};

			for (int i = 0; i < NumSimulcastLayers; i++)
			{
				// By convention, the first layer should be lowest resolution, working its way up to highest. However,
				// as we add all layers to a p2p connection, we want the first layer to be the highest resolution as 
				// the first layer is the one that will be used
				float Scaling = ScalingFactor * i;

				// clang-format off
				SimulcastParams.Add({
					.Scaling = Scaling > 0 ? Scaling : 1.f,
					.MinBitrate = Bitrates[i].Get<0>(),
					.MaxBitrate = Bitrates[i].Get<1>(),
				});
				// clang-format on
			}
		}
		else
		{
			// clang-format off
			SimulcastParams.Add({ 
				.Scaling = 1.f,
				.MinBitrate = UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread(),
				.MaxBitrate = UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread() 
			});
			// clang-format on
		}

		return SimulcastParams;
	}
} // namespace UE::PixelStreaming2