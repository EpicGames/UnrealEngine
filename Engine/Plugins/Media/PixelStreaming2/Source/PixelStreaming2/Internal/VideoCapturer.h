// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureCapturerSource.h"
#include "PixelCaptureCapturerMultiFormat.h"
#include "VideoProducer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	/**
	 * The start of PixelCapture pipeline. Frames enter the system when `OnFrameCaptured` is called.
	 * This class creates the underlying PixelCapture `FPixelCaptureCapturer` that handles frame capture when `RequestFormat` is called.
	 */
	class FVideoCapturer : public IPixelCaptureCapturerSource, public TSharedFromThis<FVideoCapturer>
	{
	public:
		static UE_API TSharedPtr<FVideoCapturer> Create(bool bIsSRGB = false);
		UE_API virtual ~FVideoCapturer();

		// Begin IPixelCaptureCapturerSource Interface
		UE_API virtual TSharedPtr<FPixelCaptureCapturer> CreateCapturer(int32 FinalFormat, FIntPoint OutputResolution) override;
		// End IPixelCaptureCapturerSource Interface

		bool IsReady() const { return bReady; }

		UE_API void SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> InVideoProducer);

		TSharedPtr<IPixelStreaming2VideoProducer> GetVideoProducer() { return VideoProducer; }

		UE_API TSharedPtr<IPixelCaptureOutputFrame> WaitForFormat(int32 Format, TOptional<FIntPoint> Resolution = TOptional<FIntPoint>(), uint32 MaxWaitTime = 5000);

		UE_API TSharedPtr<IPixelCaptureOutputFrame> RequestFormat(int32 Format, TOptional<FIntPoint> Resolution = TOptional<FIntPoint>());

		UE_API void ResetFrameCapturer();

		UE_API void OnFrame(const IPixelCaptureInputFrame& InputFrame);

		/**
		 * This is broadcast each time a frame exits the adapt process. Used to synchronize framerates with input rates.
		 * This should be called once per frame taking into consideration all the target formats and layers within the frame.
		 */
		DECLARE_MULTICAST_DELEGATE(FOnFrameCaptured);
		FOnFrameCaptured OnFrameCaptured;

		/**
		 * Each time the frame capturer is created. This could be triggered by a multitude of situations such as cvar changes, frame
		 * resizes, etc
		 */
		DECLARE_MULTICAST_DELEGATE(FOnFrameCapturerCreated);
		FOnFrameCapturerCreated OnFrameCapturerCreated;

	protected:
		UE_API FVideoCapturer(bool bIsSRGB);

		int32 LastFrameWidth = -1;
		int32 LastFrameHeight = -1;
		int32 LastFrameType = -1;
		bool  bReady = false;
		bool  bIsSRGB = false;

		TSharedPtr<IPixelStreaming2VideoProducer>	 VideoProducer;
		TSharedPtr<FPixelCaptureCapturerMultiFormat> FrameCapturer;
		FDelegateHandle								 CaptureCompleteHandle;
		FDelegateHandle								 FramePushedHandle;

		FDelegateHandle SimulcastEnabledChangedHandle;
		FDelegateHandle CaptureUseFenceChangedHandle;
		FDelegateHandle UseMediaCaptureChangedHandle;

		UE_API void CreateFrameCapturer();
		UE_API void OnSimulcastEnabledChanged(IConsoleVariable* Var);
		UE_API void OnCaptureUseFenceChanged(IConsoleVariable* Var);
		UE_API void OnUseMediaCaptureChanged(IConsoleVariable* Var);
		UE_API void OnCaptureComplete();
	};

} // namespace UE::PixelStreaming2

#undef UE_API
