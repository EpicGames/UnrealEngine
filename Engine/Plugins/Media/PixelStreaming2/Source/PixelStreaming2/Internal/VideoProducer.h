// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2VideoProducer.h"
#include "MediaCapture.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	namespace VideoProducerIdentifiers
	{
		PIXELSTREAMING2_API extern const FString FVideoProducerBase;
		PIXELSTREAMING2_API extern const FString FVideoProducerBackBuffer;
		PIXELSTREAMING2_API extern const FString FVideoProducerMediaCapture;
		PIXELSTREAMING2_API extern const FString FVideoProducerPIEViewport;
		PIXELSTREAMING2_API extern const FString FVideoProducerRenderTarget;
	} // namespace VideoProducerIdentifiers

	class FVideoProducerUserData : public FMediaCaptureUserData, public FPixelCaptureUserData
	{
	public:
		FVideoProducerUserData() = default;

		uint64	ProductionBeginCycles = 0;
		uint64	ProductionEndCycles = 0;
		FString ProducerName = TEXT("");
	};

	/**
	 * The base video producer implementation used by all other PS2 video producers as well as in tests.
	 */
	class FVideoProducerBase : public IPixelStreaming2VideoProducer
	{
	public:
		static UE_API TSharedPtr<FVideoProducerBase> Create();
		virtual ~FVideoProducerBase() = default;

		UE_API virtual FString ToString() override;

		virtual EVideoProducerCapabilities GetCapabilities() { return EVideoProducerCapabilities::Default; }

	protected:
		FVideoProducerBase() = default;

		UE_API virtual ETextureCreateFlags GetTexCreateFlags();
	};

} // namespace UE::PixelStreaming2

#undef UE_API
