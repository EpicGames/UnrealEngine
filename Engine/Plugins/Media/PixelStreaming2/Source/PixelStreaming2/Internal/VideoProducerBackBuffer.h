// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoProducer.h"
#include "Widgets/SWindow.h"
#include "Slate/SlateViewportProvider.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	/**
	 * Use this if you want to send the contents of the UE backbuffer.
	 */
	class FVideoProducerBackBuffer : public FVideoProducerBase
	{
	public:
		static UE_API TSharedPtr<FVideoProducerBackBuffer> Create();
		UE_API virtual ~FVideoProducerBackBuffer();

		UE_API virtual FString ToString() override;

	private:
		FVideoProducerBackBuffer() = default;

		UE_API void OnBackBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);

		FDelegateHandle DelegateHandle;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
