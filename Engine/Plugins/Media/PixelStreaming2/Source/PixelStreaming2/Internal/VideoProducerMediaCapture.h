// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"
#include "PixelStreaming2MediaIOCapture.h"


#include "VideoProducerMediaCapture.generated.h"

#define UE_API PIXELSTREAMING2_API

UCLASS(BlueprintType)
class UPixelStreaming2MediaIOOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	virtual FIntPoint						 GetRequestedSize() const override { return UMediaOutput::RequestCaptureSourceSize; }
	virtual EPixelFormat					 GetRequestedPixelFormat() const override { return EPixelFormat::PF_B8G8R8A8; }
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override { return EMediaCaptureConversionOperation::CUSTOM; }

private:
	UPROPERTY(Transient)
	TObjectPtr<UPixelStreaming2MediaIOCapture> MediaCapture = nullptr;
};

namespace UE::PixelStreaming2
{

	/**
	 * Use this if you want to send media capture frames as video input.
	 */
	class FVideoProducerMediaCapture : public FVideoProducerBase, public TSharedFromThis<FVideoProducerMediaCapture>
	{
	public:
		/**
		 * @brief Creates a MediaIO capture of the active viewport and starts capturing as soon as possible.
		 * @return A video input backed by the created MediaIO capture that sends frames from the active viewport.
		 */
		static UE_API TSharedPtr<FVideoProducerMediaCapture> CreateActiveViewportCapture();

		/**
		 * @brief Creates a video input where the user can specify their own MediaIO output and capture.
		 * This method does not does not configure or start capturing, this is left to the user.
		 * Use this constructor if you know how to configure the MediaIOCapture yourself or don't want to capture the active viewport.
		 * @param MediaCapture The custom MediaIOCapture that will pass its captured frames as video input.
		 * @return A video input backed by the passed in MediaIO capture.
		 */
		static UE_API TSharedPtr<FVideoProducerMediaCapture> Create(TObjectPtr<UPixelStreaming2MediaIOCapture> MediaCapture);

		UE_API virtual ~FVideoProducerMediaCapture();

		UE_API virtual FString ToString() override;

		virtual EVideoProducerCapabilities GetCapabilities() { return EVideoProducerCapabilities::ProducesPreprocessedFrames; }

	protected:
		UE_API void StartActiveViewportCapture();
		UE_API void LateStartActiveViewportCapture();

	private:
		FVideoProducerMediaCapture() = default;
		UE_API FVideoProducerMediaCapture(TObjectPtr<UPixelStreaming2MediaIOCapture> MediaCapture);

		UE_API void OnCaptureActiveViewportStateChanged();

	private:
		TObjectPtr<UPixelStreaming2MediaIOCapture> MediaCapture = nullptr;

		TOptional<FDelegateHandle> OnFrameEndDelegateHandle;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
