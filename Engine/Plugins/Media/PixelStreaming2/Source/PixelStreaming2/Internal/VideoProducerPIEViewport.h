// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "VideoProducerViewportBase.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	/**
	 * An extension of the back buffer input that can handle PIE sessions. Primarily to be used in blueprints
	 */
	class FVideoProducerPIEViewport : public FVideoProducerViewportBase
	{
	public:
		static UE_API TSharedPtr<FVideoProducerPIEViewport> Create();
		virtual ~FVideoProducerPIEViewport() = default;

		UE_API virtual FString ToString() override;

	protected:
		UE_API virtual bool ShouldCaptureViewport() override;

	private:
		FVideoProducerPIEViewport() = default;
 
		UE_API void OnPreTick(float DeltaTime);
	};
} // namespace UE::PixelStreaming2

#undef UE_API
