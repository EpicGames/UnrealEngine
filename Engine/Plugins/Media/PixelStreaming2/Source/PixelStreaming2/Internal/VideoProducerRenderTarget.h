// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoProducer.h"

#define UE_API PIXELSTREAMING2_API

class UTextureRenderTarget2D;

namespace UE::PixelStreaming2
{
	/**
	 * Use this if you want to send the contents of a render target.
	 */
	class FVideoProducerRenderTarget : public FVideoProducerBase
	{
	public:
		static UE_API TSharedPtr<FVideoProducerRenderTarget> Create(UTextureRenderTarget2D* Target);
		UE_API virtual ~FVideoProducerRenderTarget();

		UE_API virtual FString ToString() override;

	private:
		UE_API FVideoProducerRenderTarget(UTextureRenderTarget2D* InTarget);
		UE_API void OnEndFrameRenderThread();

		UTextureRenderTarget2D* Target = nullptr;
		FDelegateHandle			DelegateHandle;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
