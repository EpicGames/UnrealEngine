// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "Slate/SlateViewportProvider.h"
#include "VideoProducer.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	/**
	 * A base video producer for streaming a target viewport by way of cropping the entire backbuffer to the size of the viewport widget. 
	 * 
	 * 
	 * Implementations MUST set the TargetViewport and TargetWidget and then call CalculateCaptureRegion at least once to configure the captured region.
	 * Additionally, implementations MUST override ShouldCaptureViewport with an optional check to prevent OnBackBufferReadyToPresent from sending frames.
	 */
	class FVideoProducerViewportBase : public FVideoProducerBase
	{
	public:
		virtual ~FVideoProducerViewportBase() = default;

		UE_API virtual EVideoProducerCapabilities GetCapabilities() override;

    protected:
        FVideoProducerViewportBase() = default;

        // Viewport specific implementations should override this method with their own check
        UE_API virtual bool ShouldCaptureViewport();

        UE_API void CalculateCaptureRegion(TSharedRef<SViewport> Viewport, TSharedRef<SWindow> Window);
		UE_API void OnBackBufferReadyToPresent(SWindow& Window, ISlateViewportProvider& ViewportProvider);

        FIntRect CaptureRect;
		FCriticalSection CaptureRectCS;

        TWeakPtr<SViewport> TargetViewport;
		TWeakPtr<SWindow> TargetWindow;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
