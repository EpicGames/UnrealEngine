// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "IPixelStreaming2VideoSink.h"
#include "Rendering/SlateRenderer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FVideoSink : public IPixelStreaming2VideoSink
	{
	public:
		UE_API virtual ~FVideoSink();

		UE_API virtual void AddVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>& VideoConsumer) override;
		UE_API virtual void RemoveVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>& VideoConsumer) override;

		UE_API bool HasVideoConsumers();

		UE_API void SetMuted(bool bIsMuted);

		UE_API void OnVideoData(FTextureRHIRef Frame);

	protected:
		FCriticalSection									   VideoConsumersCS;
		TArray<TWeakPtrVariant<IPixelStreaming2VideoConsumer>> VideoConsumers;

		bool bIsMuted = false;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
