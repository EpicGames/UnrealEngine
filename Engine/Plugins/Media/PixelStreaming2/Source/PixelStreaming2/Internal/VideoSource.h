// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "VideoCapturer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FVideoSource
	{
	public:
		virtual ~FVideoSource() = default;

		UE_API virtual void PushFrame();
		UE_API virtual void ForceKeyFrame();
		UE_API virtual void SetMuted(bool bIsMuted);

	protected:
		bool bIsMuted = false;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
