// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoCapturer.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
	class FVideoSource;
	class FFrameRunnable;

	class FVideoSourceGroup : public TSharedFromThis<FVideoSourceGroup>
	{
	public:
		static UE_API TSharedPtr<FVideoSourceGroup> Create(TSharedPtr<FVideoCapturer> InVideoCapturer);
		UE_API ~FVideoSourceGroup();

		UE_API void  SetFPS(int32 InFramesPerSecond);
		UE_API int32 GetFPS();

		UE_API void SetDecoupleFramerate(bool bDecouple);

		UE_API void AddVideoSource(TSharedPtr<FVideoSource> VideoSource);
		UE_API void RemoveVideoSource(const FVideoSource* ToRemove);
		UE_API void RemoveAllVideoSources();

		UE_API void Start();
		UE_API void Stop();
		UE_API void PushFrame();

		UE_API void ForceKeyFrame();

	private:
		UE_API FVideoSourceGroup();

		UE_API void StartThread();
		UE_API void StopThread();

		UE_API void OnFrameCaptured();

		UE_API void OnWebRtcFpsChanged(IConsoleVariable* Var);
		UE_API void OnDecoupleFramerateChanged(IConsoleVariable* Var);

		bool bFPSOverride = false;
		bool bDecoupleOverride = false;

		bool  bRunning = false;
		bool  bDecoupleFramerate = false;
		int32 FramesPerSecond = 30;

		TArray<TSharedPtr<FVideoSource>> VideoSources;

		FDelegateHandle FrameDelegateHandle;
		FDelegateHandle FpsDelegateHandle;
		FDelegateHandle DecoupleDelegateHandle;

		TSharedPtr<FRunnableThread> FrameThread = nullptr;	 // constant FPS tick thread
		TSharedPtr<FFrameRunnable>	FrameRunnable = nullptr; // constant fps runnable

		mutable FCriticalSection CriticalSection;
		mutable FCriticalSection RunnableCriticalSection;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
