// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;

namespace UnrealBuildTool.Rules
{
    public class PixelStreaming2 : ModuleRules
    {
        public PixelStreaming2(ReadOnlyTargetRules Target) : base(Target)
        {
			bRequiresPlatformSDK = true;
			
            // This is so for game projects using our public headers don't have to include extra modules they might not know about.
            PublicDependencyModuleNames.AddRange(new string[]
            {
                "AudioMixer",
                "AudioPlatformConfiguration",
                "AVCodecsCore",
                "AVCodecsCoreRHI",
                "CoreUObject",
                "Json",
                "MediaIOCore",
                "MediaUtils",
                "PixelCapture",
                "PixelStreaming2Input",
                "PixelStreaming2Core",
                "RHI",
                "Renderer",
                "RenderCore",
                "SignalProcessing",
                "Slate",
                "SlateCore",
                "TraceLog",
                "WebRTC" // Used by pixelcapture :facepalm: 
            });

            PrivateDependencyModuleNames.AddRange(new string[]
            {
                "Core",
                "Engine",
                "PixelStreaming2Settings"
            });

            if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
				{
                    // Needed for the PIE viewport video producer
					"UnrealEd",
				});
			}

            if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
            {
                PrivateDependencyModuleNames.Add("VulkanRHI");
                AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan", "CUDA");
            }


            if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
            {
                PrivateDependencyModuleNames.Add("D3D11RHI");
                PrivateDependencyModuleNames.Add("D3D12RHI");

                AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");
            }

            if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
            {
                AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
            }
        }
    }
}
