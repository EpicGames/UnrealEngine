// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCapturer.h"

#include "AudioDevice.h"
#include "Engine/Engine.h"
#include "Logging.h"
#include "Misc/CoreDelegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "Sound/SampleBufferIO.h"

namespace UE::PixelStreaming2
{
	/***************************************************
	 *
	 ****************************************************/
	FMixAudioTask::FMixAudioTask(TSharedPtr<FAudioCapturer> Capturer)
		: Capturer(Capturer)
	{
	}

	void FMixAudioTask::Tick(float DeltaMs)
	{
		if (!Capturer.IsValid())
		{
			return;
		}

		TSharedPtr<FAudioCapturer> PinnedCapturer = Capturer.Pin();
		PinnedCapturer->ProcessAudio();
	}

	const FString& FMixAudioTask::GetName() const
	{
		static FString TaskName = TEXT("MixAudioTask");
		return TaskName;
	}

	/***************************************************
	 *
	 ****************************************************/

	void FAudioCapturer::Initialize()
	{
		// If No engine. Possibly running editor tests
		if (GEngine)
		{
			// subscribe to audio data
			MixerTask = FPixelStreamingTickableTask::Create<FMixAudioTask>(AsShared());
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get(); AudioDeviceManager != nullptr)
			{
				TWeakPtr<FAudioCapturer> LocalAudioCapturer = AsWeak();
				AudioDeviceManager->IterateOverAllDevices([LocalAudioCapturer](Audio::FDeviceId AudioDeviceId, FAudioDevice*) {
					if (TSharedPtr<FAudioCapturer> Pin = LocalAudioCapturer.Pin())
					{
						Pin->CreateAudioProducer(AudioDeviceId);
					}
				});
			}
		}
	}

	FAudioCapturer::FAudioCapturer(const int32 SampleRate, const int32 NumChannels, const float SampleSizeInSeconds)
		: SampleRate(SampleRate)
		, NumChannels(NumChannels)
		, SampleSizeSeconds(SampleSizeInSeconds)
	{
	}

	void FAudioCapturer::CreateAudioProducer(Audio::FDeviceId AudioDeviceId)
	{
		// The lifetimes of audio producers created by the engine are our responsibility
		TSharedPtr<FAudioProducer> AudioInput = FAudioProducer::Create(AudioDeviceId, GetSampleRate(), GetNumChannels(), MakeShared<Audio::FPatchOutput, ESPMode::ThreadSafe>(GetMaxBufferSize()));
		EngineAudioProducers.Add(AudioDeviceId, AudioInput);

		AddNewInput(*AudioInput.Get());
	}

	void FAudioCapturer::RemoveAudioProducer(Audio::FDeviceId AudioDeviceId)
	{
		TSharedPtr<FAudioProducer> AudioInput;
		if (!EngineAudioProducers.RemoveAndCopyValue(AudioDeviceId, AudioInput) || !AudioInput.IsValid())
		{
			return;
		}

		RemovePatch(*AudioInput.Get());
	}

	void FAudioCapturer::AddAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer)
	{
		TSharedPtr<FAudioProducer> AudioInput = FAudioProducer::Create(AudioProducer, GetSampleRate(), GetNumChannels(), MakeShared<Audio::FPatchOutput, ESPMode::ThreadSafe>(GetMaxBufferSize()));
		UserAudioProducers.Add(AudioProducer, AudioInput);

		AddNewInput(*AudioInput.Get());
	}

	void FAudioCapturer::RemoveAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer)
	{
		TSharedPtr<FAudioProducer> AudioInput;
		if (!UserAudioProducers.RemoveAndCopyValue(AudioProducer, AudioInput) || !AudioInput.IsValid())
		{
			return;
		}

		RemovePatch(*AudioInput.Get());
	}

	void FAudioCapturer::PushAudio(const float* AudioData, int32 InNumSamples, int32 InNumChannels, int32 InSampleRate)
	{
		Audio::TSampleBuffer<int16_t> Buffer(AudioData, InNumSamples, InNumChannels, InSampleRate);

		OnAudioBuffer.Broadcast(Buffer.GetData(), Buffer.GetNumSamples(), Buffer.GetNumChannels(), Buffer.GetSampleRate());
	}

	void FAudioCapturer::OnDebugDumpAudioChanged(IConsoleVariable* Var)
	{
		if (!Var->GetBool())
		{
			WriteDebugAudio();
		}
	}

	void FAudioCapturer::OnEnginePreExit()
	{
		// If engine is exiting but the dump cvar is true, we need to manually trigger a write
		if (UPixelStreaming2PluginSettings::CVarDebugDumpAudio.GetValueOnAnyThread())
		{
			WriteDebugAudio();
		}
	}

	void FAudioCapturer::WriteDebugAudio()
	{
		// Only write audio if we actually have some
		if (DebugDumpAudioBuffer.GetSampleDuration() <= 0.f)
		{
			return;
		}

		Audio::FSoundWavePCMWriter Writer;
		FString					   FilePath = TEXT("");
		Writer.SynchronouslyWriteToWavFile(DebugDumpAudioBuffer, TEXT("PixelStreamingMixedAudio"), TEXT(""), &FilePath);
		UE_LOGFMT(LogPixelStreaming2, Log, "Saving audio sample to: {0}", FilePath);
		DebugDumpAudioBuffer.Reset();
	}

	int32 FAudioCapturer::GetSampleRate() const
	{
		return SampleRate;
	}

	int32 FAudioCapturer::GetNumChannels() const
	{
		return NumChannels;
	}

	int32 FAudioCapturer::GetMaxBufferSize() const
	{
		return NumChannels * SampleRate * SampleSizeSeconds;
	}

	void FAudioCapturer::ProcessAudio()
	{
		int32 TargetNumSamples = MaxNumberOfSamplesThatCanBePopped();
		if (TargetNumSamples < 0)
		{
			return;
		}

		Audio::VectorOps::FAlignedFloatBuffer MixingBuffer;
		MixingBuffer.SetNumUninitialized(GetMaxBufferSize());

		int32 NumSamplesPopped = PopAudio(MixingBuffer.GetData(), TargetNumSamples, false /* bUseLatestAudio */);
		if (NumSamplesPopped == 0)
		{
			return;
		}

		if (UPixelStreaming2PluginSettings::CVarDebugDumpAudio.GetValueOnAnyThread())
		{
			// Note: TSampleBuffer takes in AudioData as float* and internally converts to int16
			Audio::TSampleBuffer<int16_t> Buffer(MixingBuffer.GetData(), NumSamplesPopped, GetNumChannels(), GetSampleRate());
			DebugDumpAudioBuffer.Append(Buffer.GetData(), Buffer.GetNumSamples(), Buffer.GetNumChannels(), Buffer.GetSampleRate());
		}

		PushAudio(MixingBuffer.GetData(), NumSamplesPopped, GetNumChannels(), GetSampleRate());
	}
} // namespace UE::PixelStreaming2