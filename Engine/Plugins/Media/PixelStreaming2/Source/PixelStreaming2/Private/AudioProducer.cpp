// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioProducer.h"

#include "AudioCapturer.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Logging.h"
#include "UtilsAudio.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FAudioProducer> FAudioProducer::Create(Audio::FDeviceId InAudioDeviceId, int32 TargetSampleRate, int32 TargetNumChannels, const Audio::FPatchOutputStrongPtr& PatchOutput)
	{
		TSharedPtr<FAudioProducer> Listener = TSharedPtr<FAudioProducer>(new FAudioProducer(TargetSampleRate, TargetNumChannels, PatchOutput));
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get(); AudioDeviceManager != nullptr)
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetAudioDeviceRaw(InAudioDeviceId); AudioDevice != nullptr)
			{
				AudioDevice->RegisterSubmixBufferListener(Listener.ToSharedRef(), AudioDevice->GetMainSubmixObject());
			}
		}

		return Listener;
	}

	TSharedPtr<FAudioProducer> FAudioProducer::Create(TSharedPtr<IPixelStreaming2AudioProducer> InAudioProducer, int32 TargetSampleRate, int32 TargetNumChannels, const Audio::FPatchOutputStrongPtr& PatchOutput)
	{
		TSharedPtr<FAudioProducer> Listener = TSharedPtr<FAudioProducer>(new FAudioProducer(TargetSampleRate, TargetNumChannels, PatchOutput));
		InAudioProducer->OnAudioPushed.AddSP(Listener.ToSharedRef(), &FAudioProducer::OnPushedAudio);

		return Listener;
	}

	FAudioProducer::FAudioProducer(int32 TargetSampleRate, int32 TargetNumChannels, const Audio::FPatchOutputStrongPtr& PatchOutput)
		: FPatchInput(PatchOutput)
		, TargetSampleRate(TargetSampleRate)
		, TargetNumChannels(TargetNumChannels)
		, bIsMuted(false)
	{
	}

	void FAudioProducer::OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock)
	{
		OnRawAudio(AudioData, NumSamples, NumChannels, SampleRate);
	}

	void FAudioProducer::OnPushedAudio(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate)
	{
		OnRawAudio(AudioData, NumSamples, NumChannels, SampleRate);
	}

	void FAudioProducer::OnRawAudio(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate)
	{
		if (bIsMuted)
		{
			return;
		}

		TRawAudio<const float> RawAudio = {
			.Data = AudioData,
			.NumSamples = NumSamples,
			.SampleRate = SampleRate,
			.NumChannels = NumChannels
		};

		Audio::FAlignedFloatBuffer ProcessedAudioData;
		FProcessedAudio ProcessedAudio = {
			.Data = ProcessedAudioData,
			.NumSamples = 0,
			.SampleRate = TargetSampleRate,
			.NumChannels = TargetNumChannels,
			.Gain = UPixelStreaming2PluginSettings::CVarWebRTCAudioGain.GetValueOnAnyThread()
		};

		if (!ProcessAudio(RawAudio, ProcessedAudio))
		{
			UE_LOGFMT(LogPixelStreaming2, Warning, "FAudioProducer::OnRawAudio: Failed to process audio data.");
			return;
		}

		PushAudio(ProcessedAudio.Data.GetData(), ProcessedAudio.NumSamples);		
	}
} // namespace UE::PixelStreaming2