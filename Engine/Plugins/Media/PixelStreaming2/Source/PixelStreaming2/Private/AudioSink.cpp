// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSink.h"

#include "PixelStreaming2Trace.h"

namespace UE::PixelStreaming2
{

	FAudioSink::~FAudioSink()
	{
		FScopeLock Lock(&AudioConsumersCS);
		for (auto Iter = AudioConsumers.CreateIterator(); Iter; ++Iter)
		{
			TWeakPtrVariant<IPixelStreaming2AudioConsumer> AudioConsumer = *Iter;
			Iter.RemoveCurrent();
			if (TStrongPtrVariant<IPixelStreaming2AudioConsumer> PinnedAudioConsumer = AudioConsumer.Pin(); PinnedAudioConsumer.IsValid())
			{
				PinnedAudioConsumer.Get()->OnAudioConsumerRemoved();
			}
		}
	}

	void FAudioSink::AddAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>& AudioConsumer)
	{
		FScopeLock Lock(&AudioConsumersCS);

		if (!AudioConsumers.Contains(AudioConsumer))
		{
			AudioConsumers.Add(AudioConsumer);
			if (TStrongPtrVariant<IPixelStreaming2AudioConsumer> PinnedAudioConsumer = AudioConsumer.Pin(); PinnedAudioConsumer.IsValid())
			{
				PinnedAudioConsumer.Get()->OnAudioConsumerAdded();
			}
		}
	}

	void FAudioSink::RemoveAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>& AudioConsumer)
	{
		FScopeLock Lock(&AudioConsumersCS);
		if (AudioConsumers.Contains(AudioConsumer))
		{
			AudioConsumers.Remove(AudioConsumer);

			if (TStrongPtrVariant<IPixelStreaming2AudioConsumer> PinnedAudioConsumer = AudioConsumer.Pin(); PinnedAudioConsumer.IsValid())
			{
				PinnedAudioConsumer.Get()->OnAudioConsumerRemoved();
			}
		}
	}

	bool FAudioSink::HasAudioConsumers()
	{
		return AudioConsumers.Num() > 0;
	}

	void FAudioSink::OnAudioData(int16_t* AudioData, uint32 NumFrames, uint32 NumChannels, uint32 SampleRate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FAudioSink::OnData", PixelStreaming2Channel);
		// This data is populated from the internals of WebRTC, basically each audio track sent from the browser has its RTP audio source received and decoded.
		// The sample rate and number of channels here has absolutely no relationship with PixelStreaming2AudioDeviceModule.
		// The sample rate and number of channels here is determined adaptively by WebRTC's NetEQ class that selects sample rate/number of channels
		// based on network conditions and other factors.
		if (!HasAudioConsumers() || bIsMuted || IsEngineExitRequested())
		{
			return;
		}

		// Iterate audio consumers and pass this data to their buffers
		FScopeLock Lock(&AudioConsumersCS);
		for (TWeakPtrVariant<IPixelStreaming2AudioConsumer> AudioConsumer : AudioConsumers)
		{
			if (TStrongPtrVariant<IPixelStreaming2AudioConsumer> PinnedAudioConsumer = AudioConsumer.Pin(); PinnedAudioConsumer.IsValid())
			{
				PinnedAudioConsumer.Get()->ConsumeRawPCM(AudioData, SampleRate, NumChannels, NumFrames);
			}
		}
	}

	void FAudioSink::SetMuted(bool bInIsMuted)
	{
		bIsMuted = bInIsMuted;
	}

} // namespace UE::PixelStreaming2