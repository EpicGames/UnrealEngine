// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSource.h"

namespace UE::PixelStreaming2
{
	void FAudioSource::OnAudioBuffer(const int16_t* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate)
	{
	}

	void FAudioSource::SetMuted(bool bInIsMuted)
	{
		bIsMuted = bInIsMuted;
	}
} // namespace UE::PixelStreaming2