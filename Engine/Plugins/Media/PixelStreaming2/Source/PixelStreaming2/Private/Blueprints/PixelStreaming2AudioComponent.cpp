// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2AudioComponent.h"

#include "CoreMinimal.h"
#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "SampleBuffer.h"
#include "SoundGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2AudioComponent)

/**
 * Component that receives audio from a remote webrtc connection and outputs it into UE using a "synth component".
 */
UPixelStreaming2AudioComponent::UPixelStreaming2AudioComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PlayerToHear(FString())
	, bAutoFindPeer(true)
	, AudioSink(nullptr)
	, SoundGenerator(MakeShared<UE::PixelStreaming2::FSoundGenerator, ESPMode::ThreadSafe>())
{
	PreferredBufferLength = 512u;
	NumChannels = 2;
	PrimaryComponentTick.bCanEverTick = true;
	SetComponentTickEnabled(true);
	bAutoActivate = true;
};

ISoundGeneratorPtr UPixelStreaming2AudioComponent::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	SoundGenerator->Initialize(InParams);
	Initialize(InParams.SampleRate);
	return SoundGenerator;
}

void UPixelStreaming2AudioComponent::OnBeginGenerate()
{
	SoundGenerator->bGeneratingAudio = true;
}

void UPixelStreaming2AudioComponent::OnEndGenerate()
{
	SoundGenerator->bGeneratingAudio = false;
}

void UPixelStreaming2AudioComponent::BeginDestroy()
{
	Super::BeginDestroy();
	Reset();
	SoundGenerator = nullptr;
}

bool UPixelStreaming2AudioComponent::ListenTo(FString PlayerToListenTo)
{
	IPixelStreaming2Module& PixelStreaming2Module = IPixelStreaming2Module::Get();
	if (!PixelStreaming2Module.IsReady())
	{
		return false;
	}
	return StreamerListenTo(PixelStreaming2Module.GetDefaultStreamerID(), PlayerToListenTo);
}

bool UPixelStreaming2AudioComponent::StreamerListenTo(FString StreamerId, FString PlayerToListenTo)
{
	if (!IPixelStreaming2Module::IsAvailable())
	{
		UE_LOGF(LogPixelStreaming2, Verbose, "Pixel Streaming audio component could not listen to anything because Pixel Streaming module is not loaded. This is expected on dedicated servers.");
		return false;
	}

	IPixelStreaming2Module& PixelStreaming2Module = IPixelStreaming2Module::Get();
	if (!PixelStreaming2Module.IsReady())
	{
		return false;
	}

	PlayerToHear = PlayerToListenTo;

	if (StreamerId == FString())
	{
		FString DefaultStreamerId = PixelStreaming2Module.GetDefaultStreamerID();

		TArray<FString> StreamerIds = PixelStreaming2Module.GetStreamerIds();
		if (StreamerIds.Contains(DefaultStreamerId))
		{
			StreamerToHear = DefaultStreamerId;
		}
		else if (StreamerIds.Num() > 0)
		{
			StreamerToHear = StreamerIds[0];
		}
	}
	else
	{
		StreamerToHear = StreamerId;
	}

	TSharedPtr<IPixelStreaming2Streamer> Streamer = PixelStreaming2Module.FindStreamer(StreamerToHear);
	if (!Streamer)
	{
		return false;
	}

	TWeakPtr<IPixelStreaming2AudioSink> CandidateSink = WillListenToAnyPlayer() ? Streamer->GetUnlistenedAudioSink() : Streamer->GetPeerAudioSink(FString(PlayerToHear));

	if (!CandidateSink.IsValid())
	{
		return false;
	}

	AudioSink = CandidateSink;

	if (TSharedPtr<IPixelStreaming2AudioSink> PinnedSink = AudioSink.Pin())
	{
		PinnedSink->AddAudioConsumer(TWeakPtrVariant<IPixelStreaming2AudioConsumer>(this));
	}

	return true;
}

void UPixelStreaming2AudioComponent::Reset()
{
	PlayerToHear = FString();
	StreamerToHear = FString();
	if (SoundGenerator)
	{
		SoundGenerator->bShouldGenerateAudio = false;
		SoundGenerator->EmptyBuffers();
	}

	if (TSharedPtr<IPixelStreaming2AudioSink> PinnedSink = AudioSink.Pin())
	{
		PinnedSink->RemoveAudioConsumer(TWeakPtrVariant<IPixelStreaming2AudioConsumer>(this));
	}

	AudioSink = nullptr;
}

bool UPixelStreaming2AudioComponent::IsListeningToPlayer()
{
	return SoundGenerator->bShouldGenerateAudio;
}

bool UPixelStreaming2AudioComponent::WillListenToAnyPlayer()
{
	return PlayerToHear == FString();
}

void UPixelStreaming2AudioComponent::ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t InNumChannels, size_t NumFrames)
{
	// Sound generator has not been initialized yet.
	if (!SoundGenerator || !SoundGenerator->IsInitialized())
	{
		return;
	}

	SoundGenerator->AddAudio(AudioData, SampleRate, InNumChannels, NumFrames);
}

void UPixelStreaming2AudioComponent::OnAudioConsumerAdded()
{
	SoundGenerator->bShouldGenerateAudio = true;
	Start();
}

void UPixelStreaming2AudioComponent::OnAudioConsumerRemoved()
{
	Reset();
}

void UPixelStreaming2AudioComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{

	bool bPixelStreaming2Loaded = IPixelStreaming2Module::IsAvailable();

	// Early out if running in commandlet
	if (IsRunningCommandlet())
	{
		return;
	}

	// if auto connect turned off don't bother
	if (!bAutoFindPeer)
	{
		return;
	}

	// if listening to a peer don't auto connect
	if (IsListeningToPlayer())
	{
		return;
	}

	if (StreamerListenTo(StreamerToHear, PlayerToHear))
	{
		UE_LOGF(LogPixelStreaming2, Log, "PixelStreaming2 audio component found a WebRTC peer to listen to.");
	}
}
