// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SynthComponent.h"
#include "IPixelStreaming2AudioConsumer.h"
#include "IPixelStreaming2AudioSink.h"
#include "PixelStreaming2AudioComponent.generated.h"

namespace UE::PixelStreaming2
{
	class FSoundGenerator;
} // namespace UE::PixelStreaming2

/**
 * Allows in-engine playback of incoming WebRTC audio from a particular Pixel Streaming player/peer using their mic in the browser.
 * Note: Each audio component associates itself with a particular Pixel Streaming player/peer (using the the Pixel Streaming player id).
 */
UCLASS(Blueprintable, ClassGroup = (PixelStreaming2), meta = (BlueprintSpawnableComponent))
class UPixelStreaming2AudioComponent : public USynthComponent, public IPixelStreaming2AudioConsumer
{
	GENERATED_BODY()

protected:
	UPixelStreaming2AudioComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin USynthComponent interface
	virtual void			   OnBeginGenerate() override;
	virtual void			   OnEndGenerate() override;
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;
	//~ End USynthComponent interface

	//~ Begin UObject interface
	virtual void BeginDestroy() override;
	//~ End UObject interface

	//~ Begin UActorComponent interface
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent interface

public:
	/**
	 *   The Pixel Streaming streamer of the player that we wish to listen to.
	 *   If this is left blank this component will use the default Pixel Streaming streamer
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Audio Component")
	FString StreamerToHear;

	/**
	 *   The Pixel Streaming player/peer whose audio we wish to listen to.
	 *   If this is left blank this component will listen to the first non-listened to peer that connects after this component is ready.
	 *   Note: that when the listened to peer disconnects this component is reset to blank and will once again listen to the next non-listened to peer that connects.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Audio Component")
	FString PlayerToHear;

	/**
	 *  If not already listening to a player/peer will try to attach for listening to the "PlayerToHear" each tick.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Audio Component")
	bool bAutoFindPeer;

private:
	TWeakPtr<IPixelStreaming2AudioSink>									  AudioSink;
	TSharedPtr<UE::PixelStreaming2::FSoundGenerator, ESPMode::ThreadSafe> SoundGenerator;

public:
	// Listen to a specific player on the default streamer. If the player is not found this component will be silent.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	bool ListenTo(FString PlayerToListenTo);

	// Listen to a specific player. If the player is not found this component will be silent.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	bool StreamerListenTo(FString StreamerId, FString PlayerToListenTo);

	// True if listening to a connected WebRTC peer through Pixel Streaming.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	bool IsListeningToPlayer();

	bool WillListenToAnyPlayer();

	// Stops listening to any connected player/peer and resets internal state so component is ready to listen again.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Audio Component")
	void Reset();

	//~ Begin IPixelStreaming2AudioConsumer interface
	void ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames);
	void OnAudioConsumerAdded();
	void OnAudioConsumerRemoved();
	//~ End IPixelStreaming2AudioConsumer interface
};