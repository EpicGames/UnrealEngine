// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Blueprints.h"

#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "Misc/FileHelper.h"
#include "PixelStreaming2AudioComponent.h"
#include "PixelStreaming2Utils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2Blueprints)

TSharedPtr<IPixelStreaming2Streamer> GetStreamer(FString StreamerId)
{
	TSharedPtr<IPixelStreaming2Streamer> Streamer = IPixelStreaming2Module::Get().FindStreamer(StreamerId);
	if (Streamer)
	{
		return Streamer;
	}
	else
	{
		UE_LOGF(LogPixelStreaming2, Error, "Unable to get streamer with id %ls", *StreamerId);
		return nullptr;
	}
}

void UPixelStreaming2Blueprints::SendResponse(FString Descriptor)
{
	StreamerSendResponse(GetDefaultStreamerID(), Descriptor);
}

	
void UPixelStreaming2Blueprints::StreamerSendResponse(FString StreamerId, FString Descriptor)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		Streamer->SendAllPlayersMessage(TEXT("Response"), Descriptor);
	}
}

void UPixelStreaming2Blueprints::SendFileAsByteArray(TArray<uint8> ByteArray, FString MimeType, FString FileExtension)
{
	StreamerSendFileAsByteArray(GetDefaultStreamerID(), ByteArray, MimeType, FileExtension);
}

void UPixelStreaming2Blueprints::StreamerSendFileAsByteArray(FString StreamerId, TArray<uint8> ByteArray, FString MimeType, FString FileExtension)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		const TArray64<uint8> LargeByteArray(ByteArray);
		Streamer->SendFileData(LargeByteArray, MimeType, FileExtension);
	}
}

void UPixelStreaming2Blueprints::SendFile(FString FilePath, FString MimeType, FString FileExtension)
{
	StreamerSendFile(GetDefaultStreamerID(), FilePath, MimeType, FileExtension);
}

void UPixelStreaming2Blueprints::StreamerSendFile(FString StreamerId, FString FilePath, FString MimeType, FString FileExtension)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		TArray64<uint8> ByteData;
		bool			bSuccess = FFileHelper::LoadFileToArray(ByteData, *FilePath);
		if (bSuccess)
		{
			Streamer->SendFileData(ByteData, MimeType, FileExtension);
		}
		else
		{
			UE_LOGF(LogPixelStreaming2, Error, "FileHelper failed to load file data");
		}
	}
}

void UPixelStreaming2Blueprints::ForceKeyFrame()
{
	StreamerForceKeyFrame(GetDefaultStreamerID());
}

void UPixelStreaming2Blueprints::StreamerForceKeyFrame(FString StreamerId)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		Streamer->ForceKeyFrame();
	}
}

void UPixelStreaming2Blueprints::FreezeFrame(UTexture2D* Texture)
{
	StreamerFreezeStream(GetDefaultStreamerID(), Texture);
}

void UPixelStreaming2Blueprints::StreamerFreezeStream(FString StreamerId, UTexture2D* Texture)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		Streamer->FreezeStream(Texture);
	}
}

void UPixelStreaming2Blueprints::UnfreezeFrame()
{
	StreamerUnfreezeStream(GetDefaultStreamerID());
}

void UPixelStreaming2Blueprints::StreamerUnfreezeStream(FString StreamerId)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		Streamer->UnfreezeStream();
	}
}

void UPixelStreaming2Blueprints::KickPlayer(FString PlayerId)
{
	StreamerKickPlayer(GetDefaultStreamerID(), PlayerId);
}

void UPixelStreaming2Blueprints::StreamerKickPlayer(FString StreamerId, FString PlayerId)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		Streamer->KickPlayer(PlayerId);
	}
}

TArray<FString> UPixelStreaming2Blueprints::GetConnectedPlayers()
{
	return StreamerGetConnectedPlayers(GetDefaultStreamerID());
}

TArray<FString> UPixelStreaming2Blueprints::StreamerGetConnectedPlayers(FString StreamerId)
{
	if (TSharedPtr<IPixelStreaming2Streamer> Streamer = GetStreamer(StreamerId))
	{
		return Streamer->GetConnectedPlayers();
	}
	else
	{
		UE_LOGF(LogPixelStreaming2, Error, "No streamer found with specified id - %ls. Returning no connected players.", *StreamerId);
		return {};
	}
}

FString UPixelStreaming2Blueprints::GetDefaultStreamerID()
{
	return IPixelStreaming2Module::Get().GetDefaultStreamerID();
}

void UPixelStreaming2Blueprints::GetJsonStringValue(FString Descriptor, FString FieldName, FString& StringValue, bool& Success)
{
	UE::PixelStreaming2::ExtractJsonFromDescriptor(Descriptor, FieldName, StringValue, Success);
}

void UPixelStreaming2Blueprints::AddJsonStringValue(const FString& Descriptor, FString FieldName, FString StringValue, FString& NewDescriptor, bool& Success)
{
	UE::PixelStreaming2::ExtendJsonWithField(Descriptor, FieldName, StringValue, NewDescriptor, Success);
}

UPixelStreaming2Delegates* UPixelStreaming2Blueprints::GetDelegates()
{
	return UPixelStreaming2Delegates::Get();
}
