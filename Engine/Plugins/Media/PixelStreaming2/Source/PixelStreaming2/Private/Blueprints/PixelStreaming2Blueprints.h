// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPixelStreaming2Module.h"
#include "PixelStreaming2Module.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Containers/Array.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2Blueprints.generated.h"

UCLASS()
class UPixelStreaming2Blueprints : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void SendResponse(FString Descriptor);


	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerSendResponse(FString StreamerId, FString Descriptor);

	/**
	 * Send a specified byte array over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param   ByteArray       The raw data that will be sent over the data channel
	 * @param   MimeType        The mime type of the file. Used for reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void SendFileAsByteArray(TArray<uint8> ByteArray, FString MimeType, FString FileExtension);

	/**
	 * Send a specified byte array over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param	StreamerId		The streamer use when sending the data
	 * @param   ByteArray       The raw data that will be sent over the data channel
	 * @param   MimeType        The mime type of the file. Used for reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerSendFileAsByteArray(FString StreamerId, TArray<uint8> ByteArray, FString MimeType, FString FileExtension);

	/**
	 * Send a specified file over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param   FilePath        The path to the file that will be sent
	 * @param   MimeType        The mime type of the file. Used for file reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void SendFile(FString Filepath, FString MimeType, FString FileExtension);

	/**
	 * Send a specified file over the WebRTC peer connection data channel. The extension and mime type are used for file reconstruction on the front end
	 *
	 * @param	StreamerId		The streamer use when sending the data
	 * @param   FilePath        The path to the file that will be sent
	 * @param   MimeType        The mime type of the file. Used for file reconstruction on the front end
	 * @param   FileExtension   The file extension. Used for file reconstruction on the front end
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerSendFile(FString StreamerId, FString Filepath, FString MimeType, FString FileExtension);

	/**
	 * Force a key frame to be sent to the default streamer (if there is one).
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void ForceKeyFrame();

	/**
	 * Force a key frame to be sent to the specified streamer.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerForceKeyFrame(FString StreamerId);

	/**
	 * Freeze the video stream of the default streamer (if there is one).
	 * @param   Texture         The freeze frame to display. If null then show the last frame that was streamed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void FreezeFrame(UTexture2D* Texture);

	/**
	 * Unfreeze the video stream of the default streamer (if there is one).
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void UnfreezeFrame();

	/**
	 * Freeze the video stream of the specified video stream.
	 * @param	StreamerId		The id of the streamer to freeze.
	 * @param   Texture         The freeze frame to display. If null then show the last frame that was streamed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerFreezeStream(FString StreamerId, UTexture2D* Texture);

	/**
	 * Unfreeze the video stream of the specified streamer.
	 * @param StreamerId		The id of the streamer to unfreeze.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerUnfreezeStream(FString StreamerId);

	/**
	 * Kick a player that is connected to the default streamer (if there is a default streamer).
	 * @param   PlayerId         The ID of the player to kick.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void KickPlayer(FString PlayerId);

	/**
	 * Kick a player connected to the specified streamer.
	 * @param	StreamerId		The streamer which the player belongs
	 * @param   PlayerId        The ID of the player to kick.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static void StreamerKickPlayer(FString StreamerId, FString PlayerId);

	/**
	 * @brief Get the connected players
	 *
	 * @return TArray<FString> The connected players
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static TArray<FString> GetConnectedPlayers();

	/**
	 * @brief Get the connected players
	 *
	 * @param StreamerId	The streamer whose list of players you wish to get
	 * @return TArray<FString> The connected players
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static TArray<FString> StreamerGetConnectedPlayers(FString StreamerId);

	/**
	 * Get the default Streamer ID
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static FString GetDefaultStreamerID();

	/**
	 * Helper function to extract a string field from a JSON descriptor of a
	 * UI interaction given its field name.
	 * The field name may be hierarchical, delimited by a period. For example,
	 * to access the Width value of a Resolution command above you should use
	 * "Resolution.Width" to get the width value.
	 * @param Descriptor - The UI interaction JSON descriptor.
	 * @param FieldName - The name of the field to look for in the JSON.
	 * @param StringValue - The string value associated with the field name.
	 * @param Success - True if the field exists in the JSON data.
	 */
	UFUNCTION(BlueprintPure, Category = "Pixel Streaming 2")
	static void GetJsonStringValue(FString Descriptor, FString FieldName, FString& StringValue, bool& Success);

	/**
	 * Helper function to add a string field to a JSON descriptor. This produces
	 * a new descriptor which may then be chained to add further string fields.
	 * @param Descriptor - The initial JSON descriptor which may be blank initially.
	 * @param FieldName - The name of the field to add to the JSON.
	 * @param StringValue - The string value associated with the field name.
	 * @param NewDescriptor - The JSON descriptor with the string field added.
	 * @param Success - True if the string field could be added successfully.
	 */
	UFUNCTION(BlueprintPure, Category = "Pixel Streaming 2")
	static void AddJsonStringValue(const FString& Descriptor, FString FieldName, FString StringValue, FString& NewDescriptor, bool& Success);

	// PixelStreaming2Delegates
	/**
	 * Get the singleton. This allows application-specific blueprints to bind
	 * to delegates of interest.
	 */
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming 2")
	static UPixelStreaming2Delegates* GetDelegates();
};