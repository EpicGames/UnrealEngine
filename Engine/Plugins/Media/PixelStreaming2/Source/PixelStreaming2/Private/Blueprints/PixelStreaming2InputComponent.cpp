// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/PixelStreaming2InputComponent.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "PixelStreaming2Common.h"
#include "PixelStreaming2Utils.h"
#include "PixelStreaming2InputEnums.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2InputComponent)

UPixelStreaming2Input::UPixelStreaming2Input(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
	PrimaryComponentTick.bCanEverTick = true;
	SetComponentTickEnabled(true);
}

void UPixelStreaming2Input::BeginPlay()
{
	Super::BeginPlay();

	UE::PixelStreaming2::InputComponents.Add(reinterpret_cast<uintptr_t>(this), this);
}

void UPixelStreaming2Input::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	UE::PixelStreaming2::InputComponents.Remove(reinterpret_cast<uintptr_t>(this));
}

void UPixelStreaming2Input::SendPixelStreaming2Response(const FString& Descriptor)
{
	IPixelStreaming2Module::Get().ForEachStreamer([&Descriptor, this](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
		TSharedPtr<IPixelStreaming2InputHandler> Handler = Streamer->GetInputHandler().Pin();
		if (!Handler)
		{
			UE_LOGF(LogPixelStreaming2, Error, "Pixel Streaming input handler was null when sending response message.");
			return;
		}
		Streamer->SendAllPlayersMessage(EPixelStreaming2FromStreamerMessage::Response, Descriptor);
	});
}

void UPixelStreaming2Input::GetJsonStringValue(FString Descriptor, FString FieldName, FString& StringValue, bool& Success)
{
	UE::PixelStreaming2::ExtractJsonFromDescriptor(Descriptor, FieldName, StringValue, Success);
}

void UPixelStreaming2Input::AddJsonStringValue(const FString& Descriptor, FString FieldName, FString StringValue, FString& NewDescriptor, bool& Success)
{
	UE::PixelStreaming2::ExtendJsonWithField(Descriptor, FieldName, StringValue, NewDescriptor, Success);
}
