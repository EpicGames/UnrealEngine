// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/PixelStreaming2MediaTexture.h"

#include "Async/Async.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "RenderUtils.h"
#include "TextureResource.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2MediaTexture)

constexpr int32 DEFAULT_WIDTH = 1920;
constexpr int32 DEFAULT_HEIGHT = 1080;

namespace UE::PixelStreaming2
{
	/**
	 * The actual texture resource for a UPixelStreaming2MediaTexture. Contains the RHI Texture and
	 * Sampler information.
	 */
	class FPixelStreaming2MediaTextureResource : public FTextureResource
	{
	public:
		FPixelStreaming2MediaTextureResource(TWeakObjectPtr<UPixelStreaming2MediaTexture> Owner)
			: MediaTexture(Owner)
		{
		}

		virtual ~FPixelStreaming2MediaTextureResource() override
		{
			TextureRHI.SafeRelease();
		}

		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			TStrongObjectPtr<UPixelStreaming2MediaTexture> PinnedMediaTexture = MediaTexture.Pin();
			if (PinnedMediaTexture.IsValid())
			{
				FSamplerStateInitializerRHI SamplerStateInitializer(
					(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(PinnedMediaTexture.Get()),
					AM_Border, AM_Border, AM_Wrap);
				SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
			}
		}

		virtual void ReleaseRHI() override
		{
			TextureRHI.SafeRelease();

			TStrongObjectPtr<UPixelStreaming2MediaTexture> PinnedMediaTexture = MediaTexture.Pin();
			if (PinnedMediaTexture.IsValid())
			{
				RHIClearTextureReference(PinnedMediaTexture->TextureReference.TextureReferenceRHI);
			}
		}

		virtual uint32 GetSizeX() const override
		{
			return TextureRHI.IsValid() ? TextureRHI->GetSizeXYZ().X : 0;
		}

		virtual uint32 GetSizeY() const override
		{
			return TextureRHI.IsValid() ? TextureRHI->GetSizeXYZ().Y : 0;
		}

		SIZE_T GetResourceSize()
		{
			return CalcTextureSize(GetSizeX(), GetSizeY(), EPixelFormat::PF_A8R8G8B8, 1);
		}

	private:
		TWeakObjectPtr<UPixelStreaming2MediaTexture> MediaTexture;
	};
} // namespace UE::PixelStreaming2

UPixelStreaming2MediaTexture::UPixelStreaming2MediaTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetResource(nullptr);
}

void UPixelStreaming2MediaTexture::BeginDestroy()
{
	SetResource(nullptr);

	Super::BeginDestroy();
}

void UPixelStreaming2MediaTexture::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (CurrentResource)
	{
		CumulativeResourceSize.AddUnknownMemoryBytes(CurrentResource->GetResourceSize());
	}
}

FTextureResource* UPixelStreaming2MediaTexture::CreateResource()
{
	if (CurrentResource)
	{
		SetResource(nullptr);
		CurrentResource = nullptr;
	}

	CurrentResource = new UE::PixelStreaming2::FPixelStreaming2MediaTextureResource(this);
	InitializeResources();

	return CurrentResource;
}

void UPixelStreaming2MediaTexture::ConsumeFrame(FTextureRHIRef Frame)
{
	AsyncTask(ENamedThreads::GetRenderThread(), [this, Frame]() {
		FScopeLock Lock(&RenderSyncContext);

		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		UpdateTextureReference(RHICmdList, Frame);
	});
}

const FTextureResource* UPixelStreaming2MediaTexture::GetResource() const
{
	return CurrentResource;
}

void UPixelStreaming2MediaTexture::InitializeResources()
{
	ENQUEUE_RENDER_COMMAND(FPixelStreamingMediaTextureUpdateTextureReference)
	([this](FRHICommandListImmediate& RHICmdList) {
		// Set the default video texture to reference nothing
		FTextureRHIRef ShaderTexture2D;
		FTextureRHIRef RenderableTexture;

		FRHITextureCreateDesc RenderTargetTextureDesc =
			FRHITextureCreateDesc::Create2D(TEXT(""), DEFAULT_WIDTH, DEFAULT_HEIGHT, EPixelFormat::PF_B8G8R8A8)
				.SetClearValue(FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f)))
				.SetFlags(ETextureCreateFlags::ShaderResource | TexCreate_RenderTargetable)
				.SetInitialState(ERHIAccess::SRVMask);

		RenderableTexture = RHICmdList.CreateTexture(RenderTargetTextureDesc);
		ShaderTexture2D = RenderableTexture;

		CurrentResource->TextureRHI = RenderableTexture;

		RHICmdList.UpdateTextureReference(TextureReference.TextureReferenceRHI, CurrentResource->TextureRHI);
	});
}

void UPixelStreaming2MediaTexture::UpdateTextureReference(FRHICommandList& RHICmdList, FTextureRHIRef Reference)
{
	if (CurrentResource)
	{
		if (Reference.IsValid() && CurrentResource->TextureRHI != Reference)
		{
			CurrentResource->TextureRHI = Reference;
			RHICmdList.UpdateTextureReference(TextureReference.TextureReferenceRHI, CurrentResource->TextureRHI);
		}
		else if (!Reference.IsValid())
		{
			if (CurrentResource)
			{
				InitializeResources();

				// Make sure RenderThread is executed before continuing
				FlushRenderingCommands();
			}
		}
	}
}
