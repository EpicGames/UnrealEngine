// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2StreamerComponent.h"

#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2InputHandler.h"
#include "VideoProducerMediaCapture.h"
#include "Engine/GameEngine.h"
#include "Slate/SceneViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2StreamerComponent)

UPixelStreaming2StreamerComponent::UPixelStreaming2StreamerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPixelStreaming2StreamerComponent::BeginPlay()
{
	Super::BeginPlay();
	if (Streamer)
	{
		SetupStreamerInput();
	}
}

void UPixelStreaming2StreamerComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
	if (Streamer)
	{
		IPixelStreaming2Module::Get().DeleteStreamer(Streamer);
	}
}

FString UPixelStreaming2StreamerComponent::GetId()
{
	if (Streamer)
	{
		return Streamer->GetId();
	}
	return "";
}

void UPixelStreaming2StreamerComponent::StartStreaming()
{
	if (!Streamer)
	{
		CreateStreamer();
	}

	if (VideoProducer && VideoProducer->GetVideoProducer())
	{
		Streamer->SetVideoProducer(VideoProducer->GetVideoProducer());
	}
	else
	{
		Streamer->SetVideoProducer(UE::PixelStreaming2::FVideoProducerMediaCapture::CreateActiveViewportCapture());
	}

	Streamer->SetCoupleFramerate(CoupleFramerate);

	if (StreamFPSOverride > 0)
	{
		Streamer->SetStreamFPS(StreamFPSOverride);
	}

	if (UsePixelStreamingURL)
	{
		FString ServerURL;
		FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingURL="), ServerURL);
		Streamer->SetConnectionURL(ServerURL);
	}
	else
	{
		Streamer->SetConnectionURL(SignallingServerURL);
	}
	Streamer->StartStreaming();
}

void UPixelStreaming2StreamerComponent::StopStreaming()
{
	if (Streamer)
	{
		Streamer->StopStreaming();
	}
}

bool UPixelStreaming2StreamerComponent::IsStreaming()
{
	return Streamer && Streamer->IsStreaming();
}

void UPixelStreaming2StreamerComponent::ForceKeyFrame()
{
	if (Streamer)
	{
		Streamer->ForceKeyFrame();
	}
}

void UPixelStreaming2StreamerComponent::FreezeStream(UTexture2D* Texture)
{
	if (Streamer)
	{
		Streamer->FreezeStream(Texture);
	}
}

void UPixelStreaming2StreamerComponent::UnfreezeStream()
{
	if (Streamer)
	{
		Streamer->UnfreezeStream();
	}
}

void UPixelStreaming2StreamerComponent::SendAllPlayersMessage(FString MessageType, const FString& Descriptor)
{
	if (Streamer)
	{
		Streamer->SendAllPlayersMessage(MessageType, Descriptor);
	}
}

void UPixelStreaming2StreamerComponent::SendPlayerMessage(FString PlayerId, FString MessageType, const FString& Descriptor)
{
	if (Streamer)
	{
		Streamer->SendPlayerMessage(PlayerId, MessageType, Descriptor);
	}
}

void UPixelStreaming2StreamerComponent::CreateStreamer()
{
	Streamer = IPixelStreaming2Module::Get().CreateStreamer(StreamerId);
	Streamer->OnStreamingStarted().AddUObject(this, &UPixelStreaming2StreamerComponent::StreamingStarted);
	Streamer->OnStreamingStopped().AddUObject(this, &UPixelStreaming2StreamerComponent::StreamingStopped);

	SetupStreamerInput();
}

void UPixelStreaming2StreamerComponent::StreamingStarted(IPixelStreaming2Streamer*)
{
	OnStreamingStarted.Broadcast();
}

void UPixelStreaming2StreamerComponent::StreamingStopped(IPixelStreaming2Streamer*)
{
	OnStreamingStopped.Broadcast();
}

void UPixelStreaming2StreamerComponent::SetupStreamerInput()
{
	if (!GIsEditor)
	{
		// default to the scene viewport if we have a game engine
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			TSharedPtr<FSceneViewport>				 TargetViewport = GameEngine->SceneViewport;
			TSharedPtr<IPixelStreaming2InputHandler> InputHandler = Streamer->GetInputHandler().Pin();
			if (TargetViewport.IsValid() && InputHandler.IsValid())
			{
				InputHandler->SetTargetViewport(TargetViewport->GetViewportWidget());
				InputHandler->SetTargetWindow(TargetViewport->FindWindow());
			}
		}
	}
}
