// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprints/PixelStreaming2VideoProducer.h"
#include "IPixelStreaming2Streamer.h"
#include "Components/ActorComponent.h"

#include "PixelStreaming2StreamerComponent.generated.h"

UCLASS(BlueprintType, Blueprintable, Category = "PixelStreaming2", META = (DisplayName = "Streamer Component", BlueprintSpawnableComponent))
class UPixelStreaming2StreamerComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	FString GetId();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void StartStreaming();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void StopStreaming();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	bool IsStreaming();

	DECLARE_EVENT(UPixelStreaming2StreamerComponent, FStreamingStartedEvent);
	FStreamingStartedEvent OnStreamingStarted;

	DECLARE_EVENT(UPixelStreaming2StreamerComponent, FStreamingStoppedEvent);
	FStreamingStoppedEvent OnStreamingStopped;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnInputReceived, FString /* Peer id */, uint8 /* Type */, TArray<uint8> /* Message */);
	FOnInputReceived OnInputReceived;

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void ForceKeyFrame();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void FreezeStream(UTexture2D* Texture);

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void UnfreezeStream();

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void SendAllPlayersMessage(FString MessageType, const FString& Descriptor);

	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	void SendPlayerMessage(FString PlayerId, FString MessageType, const FString& Descriptor);

	UPROPERTY(EditAnywhere, Category = "PixelStreaming2")
	FString StreamerId = "Streamer Component";

	UPROPERTY(EditAnywhere, Category = "PixelStreaming2")
	FString SignallingServerURL = "ws://127.0.0.1:8888";

	UPROPERTY(EditAnywhere, Category = "PixelStreaming2")
	bool UsePixelStreamingURL = false;

	UPROPERTY(EditAnywhere, Category = "PixelStreaming2")
	int32 StreamFPSOverride = -1;

	UPROPERTY(EditAnywhere, Category = "PixelStreaming2")
	bool CoupleFramerate = false;

	UPROPERTY(EditAnywhere, Category = "PixelStreaming2")
	TObjectPtr<UPixelStreaming2VideoProducerBase> VideoProducer = nullptr;

private:
	TSharedPtr<IPixelStreaming2Streamer> Streamer;

	void CreateStreamer();
	void SetupStreamerInput();

	void StreamingStarted(IPixelStreaming2Streamer*);
	void StreamingStopped(IPixelStreaming2Streamer*);
};
