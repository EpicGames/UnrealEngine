// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2VideoComponent.h"

#include "CoreMinimal.h"
#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2VideoComponent)

/**
 * Component that recieves video from a remote webrtc connection and outputs it into UE using a "synth component".
 */
UPixelStreaming2VideoComponent::UPixelStreaming2VideoComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PlayerToWatch(FString())
	, bAutoFindPeer(true)
	, VideoSink(nullptr)
	, bIsWatchingPlayer(false)
{
	PrimaryComponentTick.bCanEverTick = true;
	SetComponentTickEnabled(true);
	bAutoActivate = true;
};

void UPixelStreaming2VideoComponent::BeginDestroy()
{
	Super::BeginDestroy();
	Reset();
}

bool UPixelStreaming2VideoComponent::Watch(FString PlayerId)
{
	IPixelStreaming2Module& PixelStreaming2Module = IPixelStreaming2Module::Get();
	if (!PixelStreaming2Module.IsReady())
	{
		return false;
	}
	return StreamerWatch(PixelStreaming2Module.GetDefaultStreamerID(), PlayerId);
}

bool UPixelStreaming2VideoComponent::StreamerWatch(FString StreamerId, FString PlayerId)
{
	if (!IPixelStreaming2Module::IsAvailable())
	{
		UE_LOGF(LogPixelStreaming2, Verbose, "Pixel Streaming video component could not watch anything because Pixel Streaming module is not loaded. This is expected on dedicated servers.");
		return false;
	}

	IPixelStreaming2Module& PixelStreaming2Module = IPixelStreaming2Module::Get();
	if (!PixelStreaming2Module.IsReady())
	{
		return false;
	}

	PlayerToWatch = PlayerId;

	if (StreamerId == FString())
	{
		FString DefaultStreamerId = PixelStreaming2Module.GetDefaultStreamerID();

		TArray<FString> StreamerIds = PixelStreaming2Module.GetStreamerIds();
		if (StreamerIds.Contains(DefaultStreamerId))
		{
			StreamerToWatch = DefaultStreamerId;
		}
		else if (StreamerIds.Num() > 0)
		{
			StreamerToWatch = StreamerIds[0];
		}
	}
	else
	{
		StreamerToWatch = StreamerId;
	}

	TSharedPtr<IPixelStreaming2Streamer> Streamer = PixelStreaming2Module.FindStreamer(StreamerToWatch);
	if (!Streamer)
	{
		return false;
	}

	TWeakPtr<IPixelStreaming2VideoSink> CandidateSink = WillWatchAnyPlayer() ? Streamer->GetUnwatchedVideoSink() : Streamer->GetPeerVideoSink(FString(PlayerToWatch));

	if (!CandidateSink.IsValid())
	{
		return false;
	}

	VideoSink = CandidateSink;

	if (TSharedPtr<IPixelStreaming2VideoSink> PinnedSink = VideoSink.Pin(); PinnedSink)
	{
		PinnedSink->AddVideoConsumer(TWeakPtrVariant<IPixelStreaming2VideoConsumer>(this));
	}

	return true;
}

void UPixelStreaming2VideoComponent::Reset()
{
	PlayerToWatch = FString();
	StreamerToWatch = FString();
	bIsWatchingPlayer = false;

	if (TSharedPtr<IPixelStreaming2VideoSink> PinnedSink = VideoSink.Pin(); PinnedSink)
	{
		PinnedSink->RemoveVideoConsumer(TWeakPtrVariant<IPixelStreaming2VideoConsumer>(this));
	}

	VideoSink = nullptr;
}

bool UPixelStreaming2VideoComponent::IsWatchingPlayer()
{
	return bIsWatchingPlayer;
}

bool UPixelStreaming2VideoComponent::WillWatchAnyPlayer()
{
	return PlayerToWatch == FString();
}

void UPixelStreaming2VideoComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	bool bPixelStreaming2Loaded = IPixelStreaming2Module::IsAvailable();

	// Early out if running in commandlet
	if (IsRunningCommandlet())
	{
		return;
	}

	// if auto connect turned off don't bother
	if (!bAutoFindPeer)
	{
		return;
	}

	// if watching a peer don't auto connect
	if (IsWatchingPlayer())
	{
		return;
	}

	if (StreamerWatch(StreamerToWatch, PlayerToWatch))
	{
		UE_LOGF(LogPixelStreaming2, Log, "PixelStreaming2 video component found a WebRTC peer to watch.");
	}
}

void UPixelStreaming2VideoComponent::ConsumeFrame(FTextureRHIRef Frame) 
{
	if (TObjectPtr<UPixelStreaming2MediaTexture> LocalVideoConsumer = VideoConsumer; LocalVideoConsumer)
	{
		LocalVideoConsumer->ConsumeFrame(Frame);
	}
}

void UPixelStreaming2VideoComponent::OnVideoConsumerAdded() 
{
	bIsWatchingPlayer = true;
}

void UPixelStreaming2VideoComponent::OnVideoConsumerRemoved() 
{
	Reset();
}
