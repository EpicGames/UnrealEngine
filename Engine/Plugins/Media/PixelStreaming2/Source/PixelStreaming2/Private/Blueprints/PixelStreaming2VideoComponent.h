// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprints/PixelStreaming2MediaTexture.h"
#include "Components/SceneComponent.h"
#include "CoreMinimal.h"
#include "IPixelStreaming2VideoConsumer.h"
#include "IPixelStreaming2VideoSink.h"
#include "RHIResources.h"

#include "PixelStreaming2VideoComponent.generated.h"

/**
 * Allows in-engine playback of incoming WebRTC video from a particular Pixel Streaming player/peer using their camera in the browser.
 * Note: Each video component associates itself with a particular Pixel Streaming player/peer (using the the Pixel Streaming player id).
 */
UCLASS(Blueprintable, ClassGroup = (PixelStreaming2), meta = (BlueprintSpawnableComponent))
class UPixelStreaming2VideoComponent : public USceneComponent, public IPixelStreaming2VideoConsumer
{
	GENERATED_BODY()

protected:
	UPixelStreaming2VideoComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject interface
	virtual void BeginDestroy() override;
	//~ End UObject interface

	//~ Begin UActorComponent interface
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent interface

	//~ Begin IPixelStreaming2VideoConsumer interface
	virtual void ConsumeFrame(FTextureRHIRef Frame) override;
	virtual void OnVideoConsumerAdded() override;
	virtual void OnVideoConsumerRemoved() override;
	//~ End IPixelStreaming2VideoConsumer interface

public:
	/**
	 *   The Pixel Streaming streamer of the player that we wish to watch.
	 *   If this is left blank this component will use the default Pixel Streaming streamer
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Video Component")
	FString StreamerToWatch;

	/**
	 *   The Pixel Streaming player/peer whose video we wish to watch.
	 *   If this is left blank this component will watch the first non-watched to peer that connects after this component is ready.
	 *   Note: that when the watched peer disconnects this component is reset to blank and will once again watch the next non-watched to peer that connects.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Video Component")
	FString PlayerToWatch;

	/**
	 *  If not already watching a player/peer will try to attach for watching the "PlayerToWatch" each tick.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pixel Streaming Video Component")
	bool bAutoFindPeer;

	/**
	 * A sink for the video data received once this connection has finished negotiating.
	 */
	UPROPERTY(EditAnywhere, Category = "Pixel Streaming Video Component", META = (DisplayName = "Pixel Streaming Video Consumer", AllowPrivateAccess = true))
	TObjectPtr<UPixelStreaming2MediaTexture> VideoConsumer = nullptr;

private:
	TWeakPtr<IPixelStreaming2VideoSink> VideoSink;

	bool bIsWatchingPlayer;

public:
	// Watch a specific player on the default streamer. If the player is not found this component won't produce video.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Video Component")
	bool Watch(FString PlayerId);

	// Watch a specific player. If the player is not found this component won't produce video.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Video Component")
	bool StreamerWatch(FString StreamerId, FString PlayerId);

	// True if watching a connected WebRTC peer through Pixel Streaming.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Video Component")
	bool IsWatchingPlayer();

	bool WillWatchAnyPlayer();

	// Stops watching any connected player/peer and resets internal state so component is ready to watch again.
	UFUNCTION(BlueprintCallable, Category = "Pixel Streaming Video Component")
	void Reset();
};