// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/PixelStreaming2VideoProducer.h"

#include "VideoProducerBackBuffer.h"
#include "VideoProducerMediaCapture.h"
#include "VideoProducerPIEViewport.h"
#include "VideoProducerRenderTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2VideoProducer)

/**
 * ---------- UPixelStreaming2VideoProducerBase -------------------
 */
UPixelStreaming2VideoProducerBase::UPixelStreaming2VideoProducerBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/**
 * ---------- UPixelStreaming2VideoProducerBackBuffer -------------------
 */
UPixelStreaming2VideoProducerBackBuffer::UPixelStreaming2VideoProducerBackBuffer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<IPixelStreaming2VideoProducer> UPixelStreaming2VideoProducerBackBuffer::GetVideoProducer()
{
	if (!VideoProducer)
	{
		// detect if we're in PIE mode or not
		bool bIsGame = false;
		FParse::Bool(FCommandLine::Get(), TEXT("game"), bIsGame);
		if (GIsEditor && !bIsGame)
		{
			VideoProducer = UE::PixelStreaming2::FVideoProducerPIEViewport::Create();
		}
		else
		{
			VideoProducer = UE::PixelStreaming2::FVideoProducerBackBuffer::Create();
		}
	}

	return VideoProducer;
}

/**
 * ---------- UPixelStreaming2VideoProducerMediaCapture -------------------
 */
UPixelStreaming2VideoProducerMediaCapture::UPixelStreaming2VideoProducerMediaCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<IPixelStreaming2VideoProducer> UPixelStreaming2VideoProducerMediaCapture::GetVideoProducer()
{
	if (!VideoProducer)
	{
		// detect if we're in PIE mode or not
		bool bIsGame = false;
		FParse::Bool(FCommandLine::Get(), TEXT("game"), bIsGame);
		if (GIsEditor && !bIsGame)
		{
			VideoProducer = UE::PixelStreaming2::FVideoProducerPIEViewport::Create();
		}
		else
		{
			VideoProducer = UE::PixelStreaming2::FVideoProducerMediaCapture::CreateActiveViewportCapture();
		}
	}

	return VideoProducer;
}

/**
 * ---------- UPixelStreaming2VideoProducerRenderTarget -------------------
 */
UPixelStreaming2VideoProducerRenderTarget::UPixelStreaming2VideoProducerRenderTarget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<IPixelStreaming2VideoProducer> UPixelStreaming2VideoProducerRenderTarget::GetVideoProducer()
{
	if (!VideoProducer)
	{
		VideoProducer = UE::PixelStreaming2::FVideoProducerRenderTarget::Create(Target);
	}
	return VideoProducer;
}
