// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Delegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2Delegates)

UPixelStreaming2Delegates* UPixelStreaming2Delegates::Singleton = nullptr;

UPixelStreaming2Delegates* UPixelStreaming2Delegates::Get()
{
	if (Singleton == nullptr && !IsEngineExitRequested())
	{
		Singleton = NewObject<UPixelStreaming2Delegates>();
		Singleton->AddToRoot();
	}
	return Singleton;
}

UPixelStreaming2Delegates::~UPixelStreaming2Delegates()
{
	Singleton = nullptr;
}
