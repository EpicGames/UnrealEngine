// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2MediaIOCapture.h"

#include "Logging.h"
#include "MediaShaders.h"
#include "PixelCaptureInputFrameRHI.h"
#include "ScreenPass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2MediaIOCapture)

BEGIN_SHADER_PARAMETER_STRUCT(FWriteFenceParameters, )
SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
END_SHADER_PARAMETER_STRUCT()

void UPixelStreaming2MediaIOCapture::OnRHIResourceCaptured_RenderingThread(
	FRHICommandListImmediate& /* RHICmdList */,
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	FTextureRHIRef										   InTexture)
{
	HandleCapturedFrame(InTexture, InUserData);
}

void UPixelStreaming2MediaIOCapture::OnCustomCapture_RenderingThread(
	FRDGBuilder&										   GraphBuilder,
	const FCaptureBaseData&								   InBaseData,
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,
	FRDGTextureRef										   InSourceTexture,
	FRDGTextureRef										   OutputTexture,
	const FRHICopyTextureInfo&							   CopyInfo,
	FVector2D											   CropU,
	FVector2D											   CropV)
{
	bool bRequiresFormatConversion = InSourceTexture->Desc.Format != OutputTexture->Desc.Format;
	if (InSourceTexture->Desc.Format == OutputTexture->Desc.Format
		&& InSourceTexture->Desc.Extent.X == OutputTexture->Desc.Extent.X
		&& InSourceTexture->Desc.Extent.Y == OutputTexture->Desc.Extent.Y)
	{
		// The formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			InSourceTexture,
			OutputTexture,
			FRDGDrawTextureInfo());
	}
	else
	{
#if PLATFORM_APPLE
		// Create a staging texture that is the same size and format as the final.
		FRDGTextureRef			   StagingTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y), OutputTexture->Desc.Format, OutputTexture->Desc.ClearValue, ETextureCreateFlags::RenderTargetable), TEXT("PixelStreamingMediaIOCapture Staging"));
		FScreenPassTextureViewport StagingViewport(StagingTexture);
#endif

		FScreenPassTextureViewport InputViewport(InSourceTexture);
		FScreenPassTextureViewport OutputViewport(OutputTexture);

		FGlobalShaderMap*			 GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		int32										  MediaConversionOperation = 0; // None
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(MediaConversionOperation);

		// Rectangle area to use from source
		const FIntRect ViewRect(FIntPoint(0, 0), InSourceTexture->Desc.Extent);

		TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
		FModifyAlphaSwizzleRgbaPS::FParameters*	 PixelShaderParameters = PixelShader->AllocateAndSetParameters(
			 GraphBuilder,
			 InSourceTexture,
#if PLATFORM_APPLE
			StagingTexture
#else
			OutputTexture
#endif
		);

		FRHIBlendState*		   BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("PixelStreaming2MediaIOCapture Swizzle"),
			FScreenPassViewInfo(),
#if PLATFORM_APPLE
			StagingViewport,
#else
			OutputViewport,
#endif

			InputViewport,
			VertexShader,
			PixelShader,
			PixelShaderParameters);

#if PLATFORM_APPLE
		// Now we can be certain the formats are the same and size are the same. simple copy
		AddDrawTexturePass(
			GraphBuilder,
			GetGlobalShaderMap(GMaxRHIFeatureLevel),
			StagingTexture,
			OutputTexture,
			FRDGDrawTextureInfo());
#endif
	}

	if (UseExperimentalScheduling() && ShouldCaptureRHIResource())
	{
		TWeakObjectPtr<UPixelStreaming2MediaIOCapture> WeakThis(this);

		FWriteFenceParameters* PassParams = GraphBuilder.AllocParameters<FWriteFenceParameters>();
		PassParams->Texture = OutputTexture;

		// clang-format off
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("UPixelStreaming2MediaIOCapture WriteFence"), 
			PassParams,
			ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,								
			[WeakThis, PassParams](FRDGAsyncTask, FRHICommandList& RHICmdList) {
				if (TStrongObjectPtr<UPixelStreaming2MediaIOCapture> This = WeakThis.Pin())
				{
					FGPUFenceRHIRef Fence = GDynamicRHI->RHICreateGPUFence(TEXT("UPixelStreaming2MediaIOCapture Fence"));
					This->Fences.Add(reinterpret_cast<uintptr_t>(PassParams->Texture->GetRHI()), Fence);
					RHICmdList.WriteGPUFence(Fence);
				}
		});
		// clang-format on
	}
}

bool UPixelStreaming2MediaIOCapture::InitializeCapture()
{
	UE_LOGF(LogPixelStreaming2, Log, "Initializing Media IO capture for Pixel Streaming.");
	bViewportResized = false;
	bDoGPUCopy = true;

	SetState(EMediaCaptureState::Capturing);

	return true;
}

void UPixelStreaming2MediaIOCapture::StopCaptureImpl(bool bAllowPendingFrameToBeProcess)
{
	// Todo: Any cleanup on capture stop should happen here.
}

ETextureCreateFlags UPixelStreaming2MediaIOCapture::GetOutputTextureFlags() const
{
#if PLATFORM_APPLE
	return TexCreate_CPUReadback;
#else
	ETextureCreateFlags Flags = TexCreate_RenderTargetable | TexCreate_UAV;

	if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
	{
		Flags |= TexCreate_External;
	}
	else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		Flags |= TexCreate_Shared;
	}

	return Flags;
#endif
}

void UPixelStreaming2MediaIOCapture::WaitForGPU(FRHITexture* InRHITexture)
{
	constexpr float SleepTimeSeconds = 50 * 1E-6f;
	FGPUFenceRHIRef	 Fence;

	double MaxWaitTime = FPlatformTime::Seconds() + 0.1;
	while (!Fences.RemoveAndCopyValue(reinterpret_cast<uintptr_t>(InRHITexture), Fence))
	{
		if (FPlatformTime::Seconds() >= MaxWaitTime)
		{
			UE_LOGFMT(LogPixelStreaming2, Error, "UPixelStreaming2MediaIOCapture::WaitForGPU: Timed out waiting for fence to be added in pass!");
			return;
		}

		FPlatformProcess::SleepNoStats(SleepTimeSeconds);
	}

	if (!Fence.IsValid())
	{
		UE_LOGFMT(LogPixelStreaming2, Error, "UPixelStreaming2MediaIOCapture::WaitForGPU: Found fence is invalid!");
		return;
	}

	MaxWaitTime = FPlatformTime::Seconds() + 0.1;
	while (!Fence->Poll())
	{
		if (FPlatformTime::Seconds() >= MaxWaitTime)
		{
			UE_LOGFMT(LogPixelStreaming2, Error, "UPixelStreaming2MediaIOCapture::WaitForGPU: Timed out waiting for fence to be written!");
			return;
		}

		FPlatformProcess::SleepNoStats(SleepTimeSeconds);
	}
}

TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> UPixelStreaming2MediaIOCapture::GetCaptureFrameUserData_GameThread()
{
	TSharedPtr<UE::PixelStreaming2::FVideoProducerUserData> UserData = MakeShared<UE::PixelStreaming2::FVideoProducerUserData>();
	UserData->ProducerName = TEXT("FVideoProducerMediaCapture");
	UserData->ProductionBeginCycles = FPlatformTime::Cycles64();

	return UserData;
}

bool UPixelStreaming2MediaIOCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	SceneViewport = TWeakPtr<FSceneViewport>(InSceneViewport);
	OnCaptureViewportInitialized.Broadcast();

	// Listen for viewport resize events as resizes invalidate media capture, so we want to know when to reset capture
	InSceneViewport->ViewportResizedEvent.AddUObject(this, &UPixelStreaming2MediaIOCapture::ViewportResized);

	return true;
}

void UPixelStreaming2MediaIOCapture::ViewportResized(FViewport* Viewport, uint32 ResizeCode)
{
	bViewportResized = true;

	// If we have not captured a frame yet, we don't care about stopping the capture due to capture size mismatch
	if (!CaptureResolution)
	{
		return;
	}

	// If resolution of viewport is actually the same as the capture resolution no need to stop/restart capturing.
	if (Viewport->GetSizeXY() == *CaptureResolution)
	{
		return;
	}

	if (GetState() == EMediaCaptureState::Capturing)
	{
		UE_LOGF(LogPixelStreaming2, Warning, "Stopping PixelStreaming MediaIO capture because viewport was resized.");
		StopCapture(false);
	}
}

void UPixelStreaming2MediaIOCapture::HandleCapturedFrame(FTextureRHIRef InTexture, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData)
{
	TSharedPtr<IPixelStreaming2VideoProducer> VideoProducerPtr = VideoProducer.Pin();
	if (VideoProducerPtr)
	{
		TSharedPtr<UE::PixelStreaming2::FVideoProducerUserData> UserData = StaticCastSharedPtr<UE::PixelStreaming2::FVideoProducerUserData>(InUserData);
		UserData->ProductionEndCycles = FPlatformTime::Cycles64();
		UpdateCaptureResolution(InTexture->GetDesc().Extent.X, InTexture->GetDesc().Extent.Y);
		VideoProducerPtr->PushFrame(FPixelCaptureInputFrameRHI(InTexture, UserData));
	}
}

void UPixelStreaming2MediaIOCapture::UpdateCaptureResolution(int32 Width, int32 Height)
{
	if (!CaptureResolution)
	{
		CaptureResolution = MakeUnique<FIntPoint>();
	}
	CaptureResolution->X = Width;
	CaptureResolution->Y = Height;
}
