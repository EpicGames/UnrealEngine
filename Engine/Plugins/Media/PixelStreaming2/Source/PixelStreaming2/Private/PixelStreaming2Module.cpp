// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Module.h"

#include "Blueprints/PixelStreaming2InputComponent.h"
#include "Engine/Engine.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "PixelStreaming2Common.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreamingThread.h"
#include "UtilsCoder.h"
#include "UtilsCore.h"
#include "VideoProducer.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"

namespace UE::PixelStreaming2
{
	TThreadSafeMap<uintptr_t, UPixelStreaming2Input*> InputComponents;

	FPixelStreaming2Module* FPixelStreaming2Module::PixelStreaming2Module = nullptr;

	FPixelStreaming2Module* FPixelStreaming2Module::GetModule()
	{
		if (!PixelStreaming2Module)
		{
			PixelStreaming2Module = FModuleManager::Get().LoadModulePtr<FPixelStreaming2Module>("PixelStreaming2");
		}

		return PixelStreaming2Module;
	}

	void FPixelStreaming2Module::StartupModule()
	{
		if (!IsStreamingSupported())
		{
			return;
		}

		if (!FSlateApplication::IsInitialized())
		{
			return;
		}

		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		// Validate RHI against what PS2 supports
		if (!(
			RHIType == ERHIInterfaceType::D3D11 
			|| RHIType == ERHIInterfaceType::D3D12 
			|| RHIType == ERHIInterfaceType::Vulkan 
			|| RHIType == ERHIInterfaceType::Metal
#if PLATFORM_ANDROID
			|| RHIType == ERHIInterfaceType::OpenGL
#endif
			))
		{
	#if !WITH_DEV_AUTOMATION_TESTS
			UE_LOGF(LogPixelStreaming2, Warning, "Only D3D11/D3D12/Vulkan/Metal/OpenGL(Android only) Dynamic RHI is supported. Detected %ls", GDynamicRHI != nullptr ? GDynamicRHI->GetName() : TEXT("[null]"));
	#endif
			return;
		}

		// Initialize PixelStreaming thread. Handles tasks like audio pushing and conference ticking
		PixelStreamingThread = GetOrCreatePixelStreamingThread();

		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddLambda([this]() {
			if (!IsPlatformSupported())
			{
				return;
			}

			if (!ensure(GEngine != nullptr))
			{
				return;
			}

			// Make sure streaming is stopped before modules are unloaded.
			FCoreDelegates::OnEnginePreExit.AddLambda([this]() {
				StopStreaming();
			});

			FApp::SetUnfocusedVolumeMultiplier(1.0f);

			// HACK (Eden.Harris): Until or if we ever find a workaround for fencing, we need to ensure capture always uses a fence.
			// If we don't then we get frequent and intermittent stuttering as textures are rendered to while being encoded.
			// From testing NVENC + CUDA pathway seems acceptable without a fence in most cases so we use the faster, unsafer path there.
			if (IsRHIDeviceAMD())
			{
				if (!UPixelStreaming2PluginSettings::CVarCaptureUseFence.GetValueOnAnyThread())
				{
					UE_LOGFMT(LogPixelStreaming2, Warning, "AMD GPU Device detected, setting PixelStreaming2.CaptureUseFence to true to avoid screen tearing in stream.");
				}

				UPixelStreaming2PluginSettings::CVarCaptureUseFence.AsVariable()->Set(true);
			}

			bModuleReady = true;
			ReadyEvent.Broadcast(*this);

			// This is called after the ready event is broadcast so that other modules have a chance to initialize
			// themselves before streamer creation
			if (!GIsEditor && UPixelStreaming2PluginSettings::CVarInitializeDefaultStreamer.GetValueOnAnyThread())
			{
				// We don't want to create the default streamer when using the editor
				InitDefaultStreamer();

				if (UPixelStreaming2PluginSettings::CVarAutoStartStream.GetValueOnAnyThread())
				{
					DefaultStreamer->StartStreaming();
				}
			}

			// Extra Initialisations post loading console commands
			TWeakPtr<IPixelStreaming2Streamer> WeakDefaultStreamer = DefaultStreamer;
			IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("PixelStreaming.StartStreaming"),
				TEXT("Start all streaming sessions"),
				FConsoleCommandDelegate::CreateLambda([WeakDefaultStreamer]() {
					if (TSharedPtr<IPixelStreaming2Streamer> PinnedDefaultStreamer = WeakDefaultStreamer.Pin(); PinnedDefaultStreamer)
					{
						PinnedDefaultStreamer->StartStreaming();
					}
			}));
		
			IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("PixelStreaming.StopStreaming"),
				TEXT("End any existing streaming sessions."),
				FConsoleCommandDelegate::CreateLambda([WeakDefaultStreamer]() {
					if (TSharedPtr<IPixelStreaming2Streamer> PinnedDefaultStreamer = WeakDefaultStreamer.Pin(); PinnedDefaultStreamer)
					{
						PinnedDefaultStreamer->StopStreaming();
					}
			}));
		});
	}

	void FPixelStreaming2Module::ShutdownModule()
	{
		if (!IsStreamingSupported())
		{
			return;
		}

		// We explicitly call release on streamer so WebRTC gets shutdown before our module is deleted
		// additionally the streamer does a bunch of delegate calls and unbinds which seem to have issues
		// when called during engine destruction rather than here.
		DefaultStreamer.Reset();

		Streamers.Empty();

		// Reset thread must be called before tasks to ensure it does not attempt to run any partially destroyed tasks from the AudioMixingCapturer
		PixelStreamingThread.Reset();
	}

	IPixelStreaming2Module::FReadyEvent& FPixelStreaming2Module::OnReady()
	{
		return ReadyEvent;
	}

	bool FPixelStreaming2Module::IsReady()
	{
		return bModuleReady;
	}

	bool FPixelStreaming2Module::StartStreaming()
	{
		bool bStreamingStarted = false;
		Streamers.Apply([&bStreamingStarted](FString StreamerId, TWeakPtr<IPixelStreaming2Streamer> WeakStreamer) {
			if (TSharedPtr<IPixelStreaming2Streamer> PinnedStreamer = WeakStreamer.Pin())
			{
				PinnedStreamer->StartStreaming();
				bStreamingStarted = true;
			}
		});

		return bStreamingStarted;
	}

	void FPixelStreaming2Module::StopStreaming()
	{
		Streamers.Apply([](FString StreamerId, TWeakPtr<IPixelStreaming2Streamer> WeakStreamer) {
			if (TSharedPtr<IPixelStreaming2Streamer> PinnedStreamer = WeakStreamer.Pin())
			{
				PinnedStreamer->StopStreaming();
			}
		});
	}

	TSharedPtr<IPixelStreaming2Streamer> FPixelStreaming2Module::CreateStreamer(const FString& StreamerId, TOptional<FString> Type)
	{
		TSharedPtr<IPixelStreaming2Streamer> ExistingStreamer = FindStreamer(StreamerId);
		if (ExistingStreamer)
		{
			return ExistingStreamer;
		}

		IPixelStreaming2StreamerFactory*	 Factory = IPixelStreaming2StreamerFactory::Get(Type.Get(UPixelStreaming2PluginSettings::CVarDefaultStreamerType.GetValueOnAnyThread()));
		TSharedPtr<IPixelStreaming2Streamer> NewStreamer = Factory->CreateNewStreamer(StreamerId);
		NewStreamer->Initialize();
		// Any time we create a new streamer, populate it's signalling server URL with whatever is in the ini, console or command line
		if (!UPixelStreaming2PluginSettings::CVarConnectionURL.GetValueOnAnyThread().IsEmpty())
		{
			NewStreamer->SetConnectionURL(UPixelStreaming2PluginSettings::CVarConnectionURL.GetValueOnAnyThread());
		}

		Streamers.Add(StreamerId, NewStreamer);

		return NewStreamer;
	}

	TSharedPtr<IPixelStreaming2VideoProducer> FPixelStreaming2Module::CreateVideoProducer()
	{
		return nullptr;
	}

	TSharedPtr<IPixelStreaming2AudioProducer> FPixelStreaming2Module::CreateAudioProducer()
	{
		return nullptr;
	}

	TArray<FString> FPixelStreaming2Module::GetStreamerIds()
	{
		TSet<FString> KeySet;
		{
			Streamers.GetKeys(KeySet);
		}

		TArray<FString> KeyArray;
		for (const FString& Key : KeySet)
		{
			KeyArray.Add(Key);
		}
		return KeyArray;
	}

	TSharedPtr<IPixelStreaming2Streamer> FPixelStreaming2Module::FindStreamer(const FString& StreamerId)
	{
		if (Streamers.Contains(StreamerId))
		{
			return Streamers[StreamerId].Pin();
		}
		return nullptr;
	}

	TSharedPtr<IPixelStreaming2Streamer> FPixelStreaming2Module::DeleteStreamer(const FString& StreamerId)
	{
		TSharedPtr<IPixelStreaming2Streamer> ToBeDeleted;
		if (Streamers.Contains(StreamerId))
		{
			ToBeDeleted = Streamers[StreamerId].Pin();
			Streamers.Remove(StreamerId);
		}
		return ToBeDeleted;
	}

	void FPixelStreaming2Module::DeleteStreamer(TSharedPtr<IPixelStreaming2Streamer> ToBeDeleted)
	{
		FString StreamerIdToRemove;
		Streamers.ApplyUntil([&StreamerIdToRemove, &ToBeDeleted](FString StreamerId, TWeakPtr<IPixelStreaming2Streamer> WeakStreamer) {
			if (WeakStreamer.Pin() == ToBeDeleted)
			{
				StreamerIdToRemove = StreamerId;
				return true;
			}
			return false;
		});
		if (!StreamerIdToRemove.IsEmpty())
		{
			Streamers.Remove(StreamerIdToRemove);
		}
	}

	FString FPixelStreaming2Module::GetDefaultStreamerID()
	{
		return UPixelStreaming2PluginSettings::CVarDefaultStreamerID.GetValueOnAnyThread();
	}

	FString FPixelStreaming2Module::GetDefaultConnectionURL()
	{
		return UPixelStreaming2PluginSettings::CVarConnectionURL.GetValueOnAnyThread();
	}

	void FPixelStreaming2Module::ForEachStreamer(const TFunction<void(TSharedPtr<IPixelStreaming2Streamer>)>& Func)
	{
		TSet<FString> KeySet;
		Streamers.GetKeys(KeySet);

		for (auto&& StreamerId : KeySet)
		{
			if (TSharedPtr<IPixelStreaming2Streamer> Streamer = FindStreamer(StreamerId))
			{
				Func(Streamer);
			}
		}
	}

	TSharedPtr<FPixelStreamingThread> FPixelStreaming2Module::GetOrCreatePixelStreamingThread()
	{
		if (!PixelStreamingThread)
		{
			PixelStreamingThread = MakeShared<FPixelStreamingThread>();
		}

		return PixelStreamingThread;
	}

	void FPixelStreaming2Module::InitDefaultStreamer()
	{
		UE_LOGFMT(LogPixelStreaming2, Log, "Initializing default streamer. ID: [{0}], Type: [{1}]", GetDefaultStreamerID(), UPixelStreaming2PluginSettings::CVarDefaultStreamerType.GetValueOnAnyThread());
		DefaultStreamer = CreateStreamer(GetDefaultStreamerID(), UPixelStreaming2PluginSettings::CVarDefaultStreamerType.GetValueOnAnyThread());
	}
} // namespace UE::PixelStreaming2

IMPLEMENT_MODULE(UE::PixelStreaming2::FPixelStreaming2Module, PixelStreaming2)
