// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2Module.h"
#include "ThreadSafeMap.h"

class UPixelStreaming2Input;

namespace UE::PixelStreaming2
{
	/**
	 * This plugin allows the back buffer to be sent as a compressed video across a network.
	 */
	class FPixelStreaming2Module : public IPixelStreaming2Module
	{
	public:
		static FPixelStreaming2Module* GetModule();

		virtual ~FPixelStreaming2Module() = default;

		virtual FReadyEvent&							  OnReady() override;
		virtual bool									  IsReady() override;
		virtual bool									  StartStreaming() override;
		virtual void									  StopStreaming() override;
		virtual TSharedPtr<IPixelStreaming2Streamer>	  CreateStreamer(const FString& StreamerId, TOptional<FString> Type = TOptional<FString>()) override;
		virtual TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducer() override;
		virtual TSharedPtr<IPixelStreaming2AudioProducer> CreateAudioProducer() override;
		virtual TArray<FString>							  GetStreamerIds() override;
		virtual TSharedPtr<IPixelStreaming2Streamer>	  FindStreamer(const FString& StreamerId) override;
		virtual TSharedPtr<IPixelStreaming2Streamer>	  DeleteStreamer(const FString& StreamerId) override;
		virtual void									  DeleteStreamer(TSharedPtr<IPixelStreaming2Streamer> ToBeDeleted) override;
		virtual FString									  GetDefaultStreamerID() override;
		virtual FString									  GetDefaultConnectionURL() override;
		virtual void									  ForEachStreamer(const TFunction<void(TSharedPtr<IPixelStreaming2Streamer>)>& Func) override;

		TSharedPtr<class FPixelStreamingThread> GetOrCreatePixelStreamingThread();

	private:
		// Begin IModuleInterface
		void StartupModule() override;
		void ShutdownModule() override;
		// End IModuleInterface

	private:
		void InitDefaultStreamer();

	private:
		bool		bModuleReady = false;
		FReadyEvent ReadyEvent;

		// FPixelStreamingThread must exist before any AudioTask and AudioMixingCapturer(Which contains a audio task) to ensure it is destroyed last
		TSharedPtr<class FPixelStreamingThread> PixelStreamingThread;

		TSharedPtr<IPixelStreaming2Streamer> DefaultStreamer;

		TThreadSafeMap<FString, TWeakPtr<IPixelStreaming2Streamer>> Streamers;

		static FPixelStreaming2Module* PixelStreaming2Module;
	};
} // namespace UE::PixelStreaming2