// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Trace.h"

#include "Trace/Trace.inl"

UE_TRACE_CHANNEL_DEFINE(PixelStreaming2Channel, "Traces PixelStreaming2 operations including video/audio streaming, video source/sink management, thread \
operations, capture pipeline, and streaming performance metrics for debugging client streaming sessions.");
