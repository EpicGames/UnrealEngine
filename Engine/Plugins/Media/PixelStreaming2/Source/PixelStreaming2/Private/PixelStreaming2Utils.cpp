// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Utils.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Policies/CondensedJsonPrintPolicy.h"

namespace UE::PixelStreaming2
{
	void ExtendJsonWithField(const FString& Descriptor, FString FieldName, FString StringValue, FString& OutNewDescriptor, bool& OutSuccess)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);

		if (!Descriptor.IsEmpty())
		{
			TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Descriptor);
			if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
			{
				OutSuccess = false;
				return;
			}
		}

		TSharedRef<FJsonValueString> JsonValueObject = MakeShareable(new FJsonValueString(StringValue));
		JsonObject->SetField(FieldName, JsonValueObject);

		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&OutNewDescriptor);
		OutSuccess = FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
	}

	void ExtractJsonFromDescriptor(FString Descriptor, FString FieldName, FString& OutStringValue, bool& OutSuccess)
	{
		/**
		 * ExtractJsonFromDescriptor supports parsing nested objects, as well as keys that contain a period.
		 * eg
		 * { "Encoder.MinQP": val }
		 * vs
		 * { "Encoder": { "MinQP": val }}
		 */
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);

		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Descriptor);
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			const TSharedPtr<FJsonObject>* JsonObjectPtr = &JsonObject;

			// Create a copy of our field name
			FString OriginalFieldName = FieldName;
			if (FieldName.Contains(TEXT(".")))
			{
				TArray<FString> FieldComponents;
				FieldName.ParseIntoArray(FieldComponents, TEXT("."));
				FieldName = FieldComponents.Pop();

				for (const FString& FieldComponent : FieldComponents)
				{
					if (!(*JsonObjectPtr)->TryGetObjectField(FieldComponent, JsonObjectPtr))
					{
						// If we can't get the nested object, try parse the key as a whole just incase the key contains a period
						OutSuccess = (*JsonObjectPtr)->TryGetStringField(OriginalFieldName, OutStringValue);
						return;
					}
				}
			}

			OutSuccess = (*JsonObjectPtr)->TryGetStringField(FieldName, OutStringValue);
		}
		else
		{
			OutSuccess = false;
		}
	}

} // namespace UE::PixelStreaming2
