// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2VideoProducers.h"

#include "VideoProducerBackBuffer.h"
#include "VideoProducerRenderTarget.h"
#include "VideoProducerPIEViewport.h"

namespace UE::PixelStreaming2
{
    TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducerBackBuffer()
    {
        return FVideoProducerBackBuffer::Create();
    }

    TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducerRenderTarget(UTextureRenderTarget2D* Target)
    {
        return FVideoProducerRenderTarget::Create(Target);
    }

    TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducerPIEViewport()
    {
        return FVideoProducerPIEViewport::Create();
    }
}