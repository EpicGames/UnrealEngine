// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingThread.h"

#include "Async/EventCount.h"
#include "HAL/Event.h"
#include "HAL/Runnable.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/RunnableThread.h"
#include "Logging.h"
#include "Misc/SingleThreadRunnable.h"
#include "PixelStreaming2Trace.h"
#include "TickableTask.h"

namespace UE::PixelStreaming2
{
	static TWeakPtr<FPixelStreamingRunnable> PixelStreamingRunnable;

	/**
	 * The runnable. Handles ticking of all tasks
	 */
	class FPixelStreamingRunnable : public FRunnable, public FSingleThreadRunnable
	{
	public:
		bool IsSleeping()
		{
			return bIsSleeping;
		}

		// Begin FRunnable
		virtual bool Init() override
		{
			return true;
		}

		virtual uint32 Run() override
		{
			bIsRunning = true;

			while (bIsRunning)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::Run LoopStart", PixelStreaming2Channel);
				Tick();

				// Sleep 1ms
				FPlatformProcess::Sleep(0.001f);
			}

			return 0;
		}

		virtual void Stop() override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::Stop", PixelStreaming2Channel);
			bIsRunning = false;

			// Wake the thread to ensure it exits
			TaskEvent.Notify();
		}

		virtual void Exit() override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::Exit", PixelStreaming2Channel);
			bIsRunning = false;

			// Wake the thread to ensure it exits
			TaskEvent.Notify();
		}

		virtual FSingleThreadRunnable* GetSingleThreadInterface() override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::GetSingleThreadInterface", PixelStreaming2Channel);
			return this;
		}
		// End FRunnable

		// Begin FSingleThreadRunnable
		virtual void Tick() override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::Tick", PixelStreaming2Channel);

			const uint64 NowCycles = FPlatformTime::Cycles64();
			const double DeltaMs = FPlatformTime::ToMilliseconds64(NowCycles - LastTickCycles);
		
			bool bShouldSleep = false;
			// Generate a token for the task event. If any tasks are added or removed while ticking
			// it will prevent the sleep at the end from occuring
			FEventCountToken Token = TaskEvent.PrepareWait();
			{
				// Lock the tasks mutex inside an inner scope so that it is released before we sleep
				FScopeLock TaskLock(&TasksMutex);
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::StartTicking", PixelStreaming2Channel);
					FScopeLock NewTaskLock(&NewTasksMutex);
					for (auto& NewTask : NewTasks)
					{
						Tasks.Add(NewTask);
					}
					
					NewTasks.Empty();
					
					bIsTicking = true;
				}
				
				for (auto& Task : Tasks)
				{
					// A task may be nulled out due to deletion during our loop. Check for safety
					TSharedPtr<FPixelStreamingTickableTask> PinnedTask = Task.Pin();
					if (PinnedTask)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*PinnedTask->GetName(), PixelStreaming2Channel)
						PinnedTask->Tick(DeltaMs);
					}
				}
				
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::FinishTicking", PixelStreaming2Channel);
					if (bNeedsCleanup)
					{
						Tasks.RemoveAll([](const TWeakPtr<FPixelStreamingTickableTask> Entry) { return Entry == nullptr || !Entry.IsValid(); });
						bNeedsCleanup = false;
					}
					
					// Update the boolean to sleep if there's currently no tasks
					bShouldSleep = Tasks.Num() == 0;

					bIsTicking = false;
				}
			}

			// If there's currently no tasks, try and sleep
			if (bShouldSleep)
			{
				bIsSleeping = true;
				// If there's currently no tasks, but a task was added or removed during this loop, 
				// this wait will immediately return and continue
				TaskEvent.Wait(Token);
				bIsSleeping = false;
			}

			LastTickCycles = NowCycles;
		}
		// End FSingleThreadRunnable

		FPixelStreamingRunnable()
			: bIsTicking(false)
			, bNeedsCleanup(false)
			, bIsRunning(false)
			, bIsSleeping(false)
			, LastTickCycles(FPlatformTime::Cycles64())
		{
		}

		virtual ~FPixelStreamingRunnable() = default;

	private:
		void AddTask(TWeakPtr<FPixelStreamingTickableTask> Task)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::AddTask", PixelStreaming2Channel);
			FScopeLock NewTaskLock(&NewTasksMutex);
			NewTasks.Add(Task);
			TaskEvent.Notify();
		}

		void RemoveTask(FPixelStreamingTickableTask* Task)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingRunnable::RemoveTask", PixelStreaming2Channel);
			if (Task == nullptr)
			{
				return;
			}

			// Lock TaskLock before NewTaskLock to ensure deadlock does not happen when Tick and StartTicking lock.
			// Locking matches FTickableObjectBase locking in Tickable.cpp
			FScopeLock TaskLock(&TasksMutex);
			FScopeLock NewTaskLock(&NewTasksMutex);

			// Remove from pending list if it hasn't been registered
			NewTasks.RemoveAll([Task](const TWeakPtr<FPixelStreamingTickableTask> Entry) { 
				const TSharedPtr<FPixelStreamingTickableTask> Pin = Entry.Pin();
				return Entry == nullptr || !Pin || Pin.Get() == Task; 
			});

			// During ticking it is not safe to modify the set so null and mark for later
			if (bIsTicking)
			{
				bNeedsCleanup = true;
				for (TWeakPtr<FPixelStreamingTickableTask>& LoopTask : Tasks)
				{
					if (TSharedPtr<FPixelStreamingTickableTask> Pin = LoopTask.Pin(); Pin && Pin.Get() == Task)
					{
						LoopTask = nullptr;
					}
				}
			}
			else
			{
				Tasks.RemoveAll([Task](const TWeakPtr<FPixelStreamingTickableTask> Entry) {
					TSharedPtr<FPixelStreamingTickableTask> Pin = Entry.Pin();
					return Entry == nullptr || !Pin || Pin.Get() == Task;
				});
			}

			TaskEvent.Notify();
		}

	private:
		// Allow the FPixelStreamingTickableTask to access the private add and remove tasks
		friend FPixelStreamingTickableTask;

		// New tasks that have not yet been added to the Tasks list
		TArray<TWeakPtr<FPixelStreamingTickableTask>> NewTasks;
		// Lock for modifying new list 
		FCriticalSection NewTasksMutex;

		// Tasks to execute every tick
		TArray<TWeakPtr<FPixelStreamingTickableTask>> Tasks;
		// This critical section should be locked during entire tick process
		FCriticalSection TasksMutex;

		// Tasks can removed from any thread so this needs to be thread safe
		std::atomic<bool> bIsTicking;
		// Tasks can removed from any thread so this needs to be thread safe
		std::atomic<bool> bNeedsCleanup;
		// This thread can be stopped from another thread during shutdown so this needs to be thread safe
		std::atomic<bool> bIsRunning;
		//
		std::atomic<bool> bIsSleeping;

		// Event used to sleep the thread if no tasks exist or wake it up when a task is added/removed or the thread is stopped
		UE::FEventCount TaskEvent;

		uint64 LastTickCycles;
	};

	FPixelStreamingThread::FPixelStreamingThread()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingThread Constructor", PixelStreaming2Channel)
		Runnable = MakeShared<FPixelStreamingRunnable>();
		PixelStreamingRunnable = Runnable;
		Thread = TSharedPtr<FRunnableThread>(FRunnableThread::Create(Runnable.Get(), TEXT("Pixel Streaming PixelStreaming Thread")));
	}

	FPixelStreamingThread::~FPixelStreamingThread()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingThread Destructor", PixelStreaming2Channel);
		if (Thread)
		{
			Thread->Kill();
			Thread.Reset();
		}

		if (Runnable)
		{
			Runnable->Stop();
			Runnable.Reset();
		}
	}

	bool FPixelStreamingThread::IsSleeping()
	{
		if (Runnable)
		{
			return Runnable->IsSleeping();
		}	

		UE_LOGFMT(LogPixelStreaming2, Warning, "Tried to get IsSleeping but runnable is invalid!");
		return true;
	}

	/**
	 * ---------- FPixelStreamingTickableTask ---------------
	 */

	void FPixelStreamingTickableTask::Register()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingTickableTask::Register", PixelStreaming2Channel);
		TSharedPtr<FPixelStreamingRunnable> Runnable = PixelStreamingRunnable.Pin();
		if (Runnable)
		{
			Runnable->AddTask(AsWeak());
		}
	}

	void FPixelStreamingTickableTask::Unregister()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FPixelStreamingTickableTask::Unregister", PixelStreaming2Channel);
		TSharedPtr<FPixelStreamingRunnable> Runnable = PixelStreamingRunnable.Pin();
		if (Runnable)
		{
			Runnable->RemoveTask(this);
		}
	}

} // namespace UE::PixelStreaming2