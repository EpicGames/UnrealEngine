// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::PixelStreaming2
{
	/**
	 * The thread. Wraps both the RunnableThread and Runnable into a single point
	 */
	class FPixelStreamingThread
	{
	public:
		FPixelStreamingThread();
		~FPixelStreamingThread();

		bool IsSleeping();

	private:
		TSharedPtr<class FRunnableThread>		  Thread;
		TSharedPtr<class FPixelStreamingRunnable> Runnable;
	};
} // namespace UE::PixelStreaming2
