// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundGenerator.h"

#include "Logging.h"
#include "UtilsAudio.h"

namespace UE::PixelStreaming2
{
	FSoundGenerator::FSoundGenerator()
		: Params()
		, Buffer()
		, CriticalSection()
	{
	}

	void FSoundGenerator::Initialize(const FSoundGeneratorInitParams& InitParams)
	{
		Params = InitParams;
		bInitialized = true;
	}

	int32 FSoundGenerator::GetDesiredNumSamplesToRenderPerCallback() const
	{
		return Params.NumFramesPerCallback * Params.NumChannels;
	}

	void FSoundGenerator::OnBeginGenerate()
	{
		bGeneratingAudio = true;
	}

	void FSoundGenerator::OnEndGenerate()
	{
		bGeneratingAudio = false;
	}

	bool FSoundGenerator::IsFinished() const
	{
		return false;
	}

	bool FSoundGenerator::IsInitialized()
	{
		return bInitialized;
	}

	void FSoundGenerator::EmptyBuffers()
	{
		FScopeLock Lock(&CriticalSection);
		Buffer.Empty();
	}

	void FSoundGenerator::AddAudio(const int16* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames)
	{
		if (!bGeneratingAudio)
		{
			return;
		}

		TRawAudio<const int16> RawAudio = {
			.Data = AudioData,
			.NumSamples = static_cast<int32>(NumFrames * NumChannels),
			.SampleRate = SampleRate,
			.NumChannels = static_cast<int32>(NumChannels),
		};

		Audio::FAlignedFloatBuffer ProcessedAudioData;
		FProcessedAudio ProcessedAudio = {
			.Data = ProcessedAudioData,
			.NumSamples = 0,
			.SampleRate = static_cast<int32>(Params.SampleRate),
			.NumChannels = static_cast<int32>(Params.NumChannels),
		};

		if (!ProcessAudio(RawAudio, ProcessedAudio))
		{
			UE_LOGFMT(LogPixelStreaming2, Warning, "FSoundGenerator::AddAudio: Failed to process audio data.");
			return;
		}

		{
			// Critical Section as we are writing into the `Buffer` that `ISoundGenerator` is using on another thread.
			FScopeLock Lock(&CriticalSection);
			Buffer.Append(ProcessedAudioData.GetData(), ProcessedAudioData.Num());
		}
	}

	// Called when a new buffer is required.
	int32 FSoundGenerator::OnGenerateAudio(float* OutAudio, int32 NumRequestedSamples)
	{
		FScopeLock Lock(&CriticalSection);
		
		FMemory::Memzero(OutAudio, NumRequestedSamples * sizeof(float));

		// Not listening to peer, return zero'd buffer.
		if (!bShouldGenerateAudio || Buffer.Num() == 0)
		{
			return NumRequestedSamples;
		}

		int32 NumSamplesToCopy = FGenericPlatformMath::Min(NumRequestedSamples, Buffer.Num());

		// Copy from local buffer into OutAudio if we have enough samples
		FMemory::Memcpy(OutAudio, Buffer.GetData(), NumSamplesToCopy * sizeof(float));

		// Remove front NumSamplesToCopy from the local buffer
		Buffer.RemoveAt(0, NumSamplesToCopy, EAllowShrinking::No);

		return NumSamplesToCopy;
	}
} // namespace UE::PixelStreaming2
