// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamerReconnectTimer.h"

#include "Logging.h"
#include "PixelStreaming2PluginSettings.h"

namespace UE::PixelStreaming2
{
	void FStreamerReconnectTimer::Start(TWeakPtr<IPixelStreaming2Streamer> InWeakStreamer)
	{
		WeakStreamer = InWeakStreamer;
		bEnabled = true;
	}

	void FStreamerReconnectTimer::Stop()
	{
		bEnabled = false;
	}

	void FStreamerReconnectTimer::Reset()
	{
		NumReconnectAttempts = 0;
	}

	void FStreamerReconnectTimer::Tick(float DeltaTime)
	{
		if (IsEngineExitRequested())
		{
			return;
		}

		if (!bEnabled)
		{
			return;
		}

		TSharedPtr<IPixelStreaming2Streamer> Streamer = WeakStreamer.Pin();

		if (!Streamer)
		{
			return;
		}

		// Do not attempt a reconnect is we are already connected/streaming
		if (Streamer->IsStreaming())
		{
			return;
		}

		float ReconnectInterval = UPixelStreaming2PluginSettings::CVarSignalingReconnectInterval.GetValueOnAnyThread();

		if (ReconnectInterval <= 0.0f)
		{
			return;
		}

		uint64 CyclesNow = FPlatformTime::Cycles64();
		uint64 DeltaCycles = CyclesNow - LastReconnectCycles;
		float  DeltaSeconds = FPlatformTime::ToSeconds(DeltaCycles);

		// If enough time has elapsed, try a reconnect
		if (DeltaSeconds >= ReconnectInterval)
		{
			// Check if the next attempt to reconnect will exceed the maximum number of attempts
			if (UPixelStreaming2PluginSettings::CVarSignalingMaxReconnectAttempts.GetValueOnAnyThread() >= 0 && (NumReconnectAttempts + 1) > UPixelStreaming2PluginSettings::CVarSignalingMaxReconnectAttempts.GetValueOnAnyThread())
			{
				// Maxmimum exceeded so don't attempt it and instead stop the timer
				Stop();
				UE_LOGFMT(LogPixelStreaming2, Log, "Exceeded maximum reconnect attempts!", NumReconnectAttempts);
				OnExceededMaxReconnectAttempts.Broadcast();
				return;
			}

			NumReconnectAttempts++;
			UE_LOGFMT(LogPixelStreaming2, Log, "Streamer reconnecting... Attempt {0}", NumReconnectAttempts);
			Streamer->StartStreaming();
			LastReconnectCycles = CyclesNow;
		}
	}
} // namespace UE::PixelStreaming2