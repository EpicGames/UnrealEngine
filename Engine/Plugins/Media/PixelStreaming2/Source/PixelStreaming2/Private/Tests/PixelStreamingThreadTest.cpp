// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreamingThread.h"
#include "PixelStreaming2Module.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "TickableTask.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
    class FTestTask : public FPixelStreamingTickableTask
	{
	public:
		FTestTask(int TaskIndex)
            : Name(FString::Printf(TEXT("Task_%d"), TaskIndex))
        {
        }

		virtual ~FTestTask() = default;

		// Begin FPixelStreamingTickableTask
		virtual void Tick(float DeltaMs) override
        {
            TickCount++;
        }

		virtual const FString& GetName() const override
        {
            return Name;
        }
		// End FPixelStreamingTickableTask

        uint32 GetTickCount()
        {
            return TickCount;
        }

	private:
        std::atomic<uint32> TickCount;

        FString Name;
	};

    class FTestThreadRunnable : public FRunnable
    {
    public:
        FTestThreadRunnable(TFunction<void()> RunFunc)
            : RunFunc(RunFunc)
        {
        }

        virtual ~FTestThreadRunnable() = default;

        virtual uint32 Run() override
        {
            RunFunc();

            return 0;
        }

    private:
        TFunction<void()> RunFunc;
    };

    IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2ThreadTest, "System.Plugins.PixelStreaming2.FPS2ThreadTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2ThreadTest::RunTest(const FString& Parameters)
    {
        FPixelStreaming2Module* Module = FPixelStreaming2Module::GetModule();
        if (!Module)
        {
            return false;
        }

        TSharedPtr<FPixelStreamingThread> Thread = Module->GetOrCreatePixelStreamingThread();

        TestTrue(TEXT("Thread should be sleeping with no tasks"), Thread->IsSleeping());

        TSharedPtr<FTestTask> Task = FPixelStreamingTickableTask::Create<FTestTask>(0);
        FPlatformProcess::Sleep(0.1f); // Sleep to ensure thread has had a chance to wake up

        TestTrue(TEXT("Thread should not be sleeping"), !Thread->IsSleeping());
        TestTrue(TEXT("Task should have ticked at least once"), Task->GetTickCount() > 0);

        Task = nullptr;
        FPlatformProcess::Sleep(0.1f); // Sleep to ensure thread has had a chance to process removal and then sleep

        TestTrue(TEXT("Thread should be sleeping with no tasks"), Thread->IsSleeping());

		return true;
    }

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2ThreadConcurrencyTest, "System.Plugins.PixelStreaming2.FPS2ThreadConcurrencyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2ThreadConcurrencyTest::RunTest(const FString& Parameters)
	{
        FPixelStreaming2Module* Module = FPixelStreaming2Module::GetModule();
        if (!Module)
        {
            return false;
        }

        TSharedPtr<FPixelStreamingThread> Thread = Module->GetOrCreatePixelStreamingThread();

        TestTrue(TEXT("Thread should be sleeping with no tasks"), Thread->IsSleeping());

        constexpr int NumThreads = 4;
        constexpr int NumIterations = 50;

        TArray<FRunnableThread*> TestThreads;
        TArray<TSharedPtr<FTestThreadRunnable>> TestRunnables;

        FCriticalSection TasksMutex;
        TArray<TSharedPtr<FTestTask>> TestTasks;

        std::atomic<int> TaskIndex;
        TFunction<void()> RunnableFunc = [NumIterations, &TaskIndex, &TasksMutex, &TestTasks]()
        {
            for (int32 i = 0; i < NumIterations; ++i)
            {
                // Random sleep to simulate unpredictable timing
                FPlatformProcess::Sleep(FMath::FRandRange(0.001f, 0.01f));
                // Randomly decide to add or remove
                if (FMath::RandBool())
                {
                    // Add task
                    auto NewTask = FPixelStreamingTickableTask::Create<FTestTask>(TaskIndex++);
                    FScopeLock Lock(&TasksMutex);
                    TestTasks.Add(NewTask);
                }
                else
                {
                    // Remove a random task if any exist
                    FScopeLock Lock(&TasksMutex);
                    if (TestTasks.Num() > 0)
                    {
                        int32 Index = FMath::RandRange(0, TestTasks.Num() - 1);
                        TSharedPtr<FTestTask> ToRemove = TestTasks[Index];
                        TestTasks.RemoveAt(Index);
                    }
                }
            }
        };

        // Spawn NumThreads number of threads, each with a runnable that will add/remove tasks at random
        for (int ThreadIdx = 0; ThreadIdx < NumThreads; ThreadIdx++)
        {
            TSharedPtr<FTestThreadRunnable> TestRunnable = MakeShared<FTestThreadRunnable>(RunnableFunc);
            TestRunnables.Add(TestRunnable);
            TestThreads.Add(FRunnableThread::Create(TestRunnable.Get(), *FString::Printf(TEXT("TestThread_%d"), ThreadIdx)));
        }

        // Wait for the spawned threads to complete
        for (FRunnableThread* T : TestThreads)
        {
            T->WaitForCompletion();
            delete T;
        }

        // Remove runnable associated with threads
        TestRunnables.Empty();

        // Sleep to ensure activity on PixelStreamingThread has settled
        FPlatformProcess::Sleep(0.2f);

        // If there's still tasks valid, then the thread shouldn't be sleeping
        if (TestTasks.Num() > 0)
        {
            TestTrue(TEXT("Thread should not be sleeping"), !Thread->IsSleeping());
        }

        // Verify that tasks not removed have a valid tick count
        for (TSharedPtr<FTestTask>& Task : TestTasks)
        {
            TestTrue(TEXT("Task should have ticked at least once"), Task->GetTickCount() > 0);
        }

        // Empty the test tasks (unregistering them in the process)
        TestTasks.Empty();
        FPlatformProcess::Sleep(0.1f); // Sleep to ensure thread has had a chance to process removals and then sleep

        // With no tasks valid, the thread should now be sleeping
        TestTrue(TEXT("Thread should be sleeping with no tasks!"), Thread->IsSleeping());

		return true;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS
