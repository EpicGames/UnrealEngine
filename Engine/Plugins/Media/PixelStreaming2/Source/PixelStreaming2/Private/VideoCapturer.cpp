// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoCapturer.h"

#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureCapturerMediaCapture.h"
#include "PixelCaptureCapturerI420.h"
#include "PixelCaptureCapturerI420ToRHI.h"
#include "PixelCaptureCapturerNV12ToRHI.h"
#include "PixelCaptureCapturerRHI.h"
#include "PixelCaptureCapturerRHINoCopy.h"
#include "PixelCaptureCapturerRHIRDG.h"
#include "PixelCaptureCapturerRHIToI420CPU.h"
#include "PixelStreaming2PluginSettings.h"
#include "Logging.h"
#include "PixelStreaming2Trace.h"
#include "UtilsVideo.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FVideoCapturer> FVideoCapturer::Create(bool bInIsSRGB)
	{
		TSharedPtr<FVideoCapturer> VideoCapturer = TSharedPtr<FVideoCapturer>(new FVideoCapturer(bInIsSRGB));

		return VideoCapturer;
	}

	FVideoCapturer::FVideoCapturer(bool bIsSRGB)
		: bIsSRGB(bIsSRGB)
	{
		CreateFrameCapturer();

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			SimulcastEnabledChangedHandle = Delegates->OnSimulcastEnabledChanged.AddRaw(this, &FVideoCapturer::OnSimulcastEnabledChanged);
			CaptureUseFenceChangedHandle = Delegates->OnCaptureUseFenceChanged.AddRaw(this, &FVideoCapturer::OnCaptureUseFenceChanged);
			UseMediaCaptureChangedHandle = Delegates->OnUseMediaCaptureChanged.AddRaw(this, &FVideoCapturer::OnUseMediaCaptureChanged);
		}
	}

	FVideoCapturer::~FVideoCapturer()
	{
		if (VideoProducer)
		{
			VideoProducer->OnFramePushed.Remove(FramePushedHandle);
		}

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnSimulcastEnabledChanged.Remove(SimulcastEnabledChangedHandle);
			Delegates->OnCaptureUseFenceChanged.Remove(CaptureUseFenceChangedHandle);
			Delegates->OnUseMediaCaptureChanged.Remove(UseMediaCaptureChangedHandle);
		}
	}

	void FVideoCapturer::SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> InVideoProducer)
	{
		if (VideoProducer)
		{
			VideoProducer->OnFramePushed.Remove(FramePushedHandle);
		}

		VideoProducer = InVideoProducer;

		if (VideoProducer)
		{
			FramePushedHandle = VideoProducer->OnFramePushed.AddSP(AsShared(), &FVideoCapturer::OnFrame);
		}
	}

	void FVideoCapturer::OnFrame(const IPixelCaptureInputFrame& InputFrame)
	{
		if (InputFrame.GetType() != PixelCaptureBufferFormat::FORMAT_RHI
			&& InputFrame.GetType() != PixelCaptureBufferFormat::FORMAT_I420
			&& InputFrame.GetType() != PixelCaptureBufferFormat::FORMAT_NV12)
		{
			UE_LOGFMT(LogPixelStreaming2, Error, "Unsupported input format. Expected either a FPixelCaptureInputFrameRHI, FPixelCaptureInputFrameI420 or FPixelCaptureInputFrameNV12");
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming2 Video Input Frame", PixelStreaming2Channel);
		// if res change or format change, we need to re-create the capturer
		if ((LastFrameWidth != -1 && LastFrameHeight != -1)
			|| LastFrameType != InputFrame.GetType())
		{
			if (InputFrame.GetWidth() != LastFrameWidth || InputFrame.GetHeight() != LastFrameHeight)
			{
				CreateFrameCapturer();
			}
			else if (InputFrame.GetType() != LastFrameType)
			{
				CreateFrameCapturer();
			}
		}

		LastFrameWidth = InputFrame.GetWidth();
		LastFrameHeight = InputFrame.GetHeight();
		LastFrameType = InputFrame.GetType();
		bReady = true;

		FrameCapturer->Capture(InputFrame);
	}

	TSharedPtr<IPixelCaptureOutputFrame> FVideoCapturer::WaitForFormat(int32 Format, TOptional<FIntPoint> Resolution, uint32 MaxWaitTime)
	{
		if (FrameCapturer != nullptr)
		{
			if (!Resolution.IsSet())
			{
				Resolution = { LastFrameWidth, LastFrameHeight };
			}
			return FrameCapturer->WaitForFormat(Format, *Resolution, MaxWaitTime);
		}
		return nullptr;
	}

	TSharedPtr<IPixelCaptureOutputFrame> FVideoCapturer::RequestFormat(int32 Format, TOptional<FIntPoint> Resolution)
	{
		if (FrameCapturer != nullptr)
		{
			if (!Resolution.IsSet())
			{
				Resolution = { LastFrameWidth, LastFrameHeight };
			}
			return FrameCapturer->RequestFormat(Format, *Resolution);
		}
		return nullptr;
	}

	void FVideoCapturer::ResetFrameCapturer()
	{
		CreateFrameCapturer();
	}

	void FVideoCapturer::OnSimulcastEnabledChanged(IConsoleVariable* Var)
	{
		CreateFrameCapturer();
	}

	void FVideoCapturer::OnCaptureUseFenceChanged(IConsoleVariable* Var)
	{
		CreateFrameCapturer();
	}

	void FVideoCapturer::OnUseMediaCaptureChanged(IConsoleVariable* Var)
	{
		CreateFrameCapturer();
	}

	void FVideoCapturer::CreateFrameCapturer()
	{
		if (FrameCapturer != nullptr)
		{
			FrameCapturer->OnDisconnected();
			FrameCapturer->OnComplete.Remove(CaptureCompleteHandle);
			FrameCapturer = nullptr;
		}

		FrameCapturer = FPixelCaptureCapturerMultiFormat::Create(this);
		CaptureCompleteHandle = FrameCapturer->OnComplete.AddRaw(this, &FVideoCapturer::OnCaptureComplete);
	}

	void FVideoCapturer::OnCaptureComplete()
	{
		OnFrameCaptured.Broadcast();
	}

	TSharedPtr<FPixelCaptureCapturer> FVideoCapturer::CreateCapturer(int32 FinalFormat, FIntPoint OutputResolution)
	{
		OnFrameCapturerCreated.Broadcast();

		if (LastFrameType == PixelCaptureBufferFormat::FORMAT_RHI)
		{
			switch (FinalFormat)
			{
				case PixelCaptureBufferFormat::FORMAT_RHI:
				{
					bool bProducesPreprocessedFrames = VideoProducer ? EnumHasAnyFlags(VideoProducer->GetCapabilities(), EVideoProducerCapabilities::ProducesPreprocessedFrames) : false;
					if (bProducesPreprocessedFrames && OutputResolution.X == LastFrameWidth && OutputResolution.Y == LastFrameHeight)
					{
						// If we only have a single layer (and it's scale is 1), we can use the no copy capturer
						// as we know the output from the media capture will already be the correct format and scale
						return FPixelCaptureCapturerRHINoCopy::Create();
					}
					else
					{
						if (UPixelStreaming2PluginSettings::CVarUseMediaCapture.GetValueOnAnyThread())
						{
							return FPixelCaptureCapturerMediaCapture::Create({ .OutputResolution = OutputResolution, .bIsSRGB = bIsSRGB }, FinalFormat);
						}
						else
						{
							if (UPixelStreaming2PluginSettings::CVarCaptureUseFence.GetValueOnAnyThread())
							{
								return FPixelCaptureCapturerRHI::Create({ .OutputResolution = OutputResolution, .bIsSRGB = bIsSRGB });
							}
							else
							{
								return FPixelCaptureCapturerRHIRDG::Create({ .OutputResolution = OutputResolution, .bIsSRGB = bIsSRGB });
							}
						}
					}
				}
				case PixelCaptureBufferFormat::FORMAT_I420:
					if (UPixelStreaming2PluginSettings::CVarUseMediaCapture.GetValueOnAnyThread())
					{
						return FPixelCaptureCapturerMediaCapture::Create({ .OutputResolution = OutputResolution, .bIsSRGB = bIsSRGB }, FinalFormat);
					}
					else
					{
						return FPixelCaptureCapturerRHIToI420CPU::Create({ .OutputResolution = OutputResolution, .bIsSRGB = bIsSRGB });
					}
				default:
					UE_LOGFMT(LogPixelStreaming2, Error, "Unsupported final format ({0}) for RHI input format", FinalFormat);
					return nullptr;
			}
		}
		else if (LastFrameType == PixelCaptureBufferFormat::FORMAT_I420)
		{
			switch (FinalFormat)
			{
				case PixelCaptureBufferFormat::FORMAT_RHI:
					return FPixelCaptureCapturerI420ToRHI::Create({ .bIsSRGB = bIsSRGB }); // TODO (william.belcher): This capturer doesn't support scaling as part of the capture process
				case PixelCaptureBufferFormat::FORMAT_I420:
					return FPixelCaptureCapturerI420::Create();
				default:
					UE_LOGFMT(LogPixelStreaming2, Error, "Unsupported final format ({0}) for I420 input format", FinalFormat);
					return nullptr;
			}
		}
		else if (LastFrameType == PixelCaptureBufferFormat::FORMAT_NV12)
		{
			switch (FinalFormat)
			{
				case PixelCaptureBufferFormat::FORMAT_RHI:
					return FPixelCaptureCapturerNV12ToRHI::Create();
				default:
					UE_LOGFMT(LogPixelStreaming2, Error, "Unsupported final format ({0}) for NV12 input format", FinalFormat);
					return nullptr;
			}
		}
		else
		{
			// The video input will early out in OnFrame so we shouldn't even hit this, but log just in case
			UE_LOGFMT(LogPixelStreaming2, Error, "Unsupported input format. Expected either a FPixelCaptureInputFrameRHI, FPixelCaptureInputFrameI420 or FPixelCaptureInputFrameNV12!");
			return nullptr;
		}
	}
} // namespace UE::PixelStreaming2