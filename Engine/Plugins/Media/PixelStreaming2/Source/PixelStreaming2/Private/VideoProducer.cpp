// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducer.h"

namespace UE::PixelStreaming2
{
	namespace VideoProducerIdentifiers
	{
		const FString FVideoProducerBase = TEXT("The default video producer - override me");
		const FString FVideoProducerBackBuffer = TEXT("the Back Buffer");
		const FString FVideoProducerMediaCapture = TEXT("a Media Capture Video Input");
		const FString FVideoProducerPIEViewport = TEXT("the PIE Viewport");
		const FString FVideoProducerRenderTarget = TEXT("a Render Target");
	}

	TSharedPtr<FVideoProducerBase> FVideoProducerBase::Create()
	{
		return TSharedPtr<FVideoProducerBase>(new FVideoProducerBase());
	}

	FString FVideoProducerBase::ToString()
	{
		return VideoProducerIdentifiers::FVideoProducerBase;
	}

	ETextureCreateFlags FVideoProducerBase::GetTexCreateFlags()
	{
		ETextureCreateFlags Flags = TexCreate_RenderTargetable | TexCreate_UAV;

		if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
		{
			Flags |= TexCreate_External;
		}
		else if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11 || RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
		{
			Flags |= TexCreate_Shared;
		}

		return Flags;
	}
} // namespace UE::PixelStreaming2