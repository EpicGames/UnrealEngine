// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerBackBuffer.h"

#include "PixelCaptureInputFrameRHI.h"
#include "UtilsAsync.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateViewportProvider.h"

namespace UE::PixelStreaming2
{

	TSharedPtr<FVideoProducerBackBuffer> FVideoProducerBackBuffer::Create()
	{
		// this was added to fix packaging
		if (!FSlateApplication::IsInitialized())
		{
			return nullptr;
		}

		TSharedPtr<FVideoProducerBackBuffer> NewInput = TSharedPtr<FVideoProducerBackBuffer>(new FVideoProducerBackBuffer());
		TWeakPtr<FVideoProducerBackBuffer>	 WeakInput = NewInput;

		// Set up the callback on the game thread since FSlateApplication::Get() can only be used there
		UE::PixelStreaming2::DoOnGameThread([WeakInput]() {
			if (TSharedPtr<FVideoProducerBackBuffer> Input = WeakInput.Pin())
			{
				FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
				Input->DelegateHandle = Renderer->OnBackBufferReadyToPresent().AddSP(Input.ToSharedRef(), &FVideoProducerBackBuffer::OnBackBufferReady);
			}
		});

		return NewInput;
	}

	FVideoProducerBackBuffer::~FVideoProducerBackBuffer()
	{
		if (!IsEngineExitRequested())
		{
			UE::PixelStreaming2::DoOnGameThread([HandleCopy = DelegateHandle]() {
				FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().Remove(HandleCopy);
			});
		}
	}

	void FVideoProducerBackBuffer::OnBackBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
	{
		PushFrame(FPixelCaptureInputFrameRHI(ViewportProvider.GetBackBufferResource()));
	}

	FString FVideoProducerBackBuffer::ToString()
	{
		return VideoProducerIdentifiers::FVideoProducerBackBuffer;
	}

} // namespace UE::PixelStreaming2