// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerMediaCapture.h"

#include "Logging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VideoProducerMediaCapture)

namespace UE::PixelStreaming2
{

	TSharedPtr<FVideoProducerMediaCapture> FVideoProducerMediaCapture::CreateActiveViewportCapture()
	{
		TSharedPtr<FVideoProducerMediaCapture> NewInput = TSharedPtr<FVideoProducerMediaCapture>(new FVideoProducerMediaCapture());
		NewInput->LateStartActiveViewportCapture();
		return NewInput;
	}

	TSharedPtr<FVideoProducerMediaCapture> FVideoProducerMediaCapture::Create(TObjectPtr<UPixelStreaming2MediaIOCapture> MediaCapture)
	{
		TSharedPtr<FVideoProducerMediaCapture> VideoProducer = TSharedPtr<FVideoProducerMediaCapture>(new FVideoProducerMediaCapture(MediaCapture));
		MediaCapture->SetVideoProducer(VideoProducer);
		return VideoProducer;
	}

	FVideoProducerMediaCapture::FVideoProducerMediaCapture(TObjectPtr<UPixelStreaming2MediaIOCapture> InMediaCapture)
		: MediaCapture(InMediaCapture)
	{
	}

	FVideoProducerMediaCapture::~FVideoProducerMediaCapture()
	{
		// We don't need to remove mediacapture from root and clear delegate if engine is shutting down
		// as UE will already have killed all UObjects by this point.
		if (!IsEngineExitRequested() && MediaCapture)
		{
			MediaCapture->OnStateChangedNative.RemoveAll(this);
			MediaCapture->RemoveFromRoot();
		}
	}

	void FVideoProducerMediaCapture::StartActiveViewportCapture()
	{
		// If we were bound to the OnFrameEnd delegate to ensure a frame was rendered before starting, then we can unset it here.
		if (OnFrameEndDelegateHandle.IsSet())
		{
			FCoreDelegates::OnEndFrame.Remove(OnFrameEndDelegateHandle.GetValue());
			OnFrameEndDelegateHandle.Reset();
		}

		if (MediaCapture)
		{
			MediaCapture->OnStateChangedNative.RemoveAll(this);
			MediaCapture->RemoveFromRoot();
		}

		MediaCapture = NewObject<UPixelStreaming2MediaIOCapture>();
		MediaCapture->AddToRoot(); // prevent GC on this
		UPixelStreaming2MediaIOOutput* MediaOutput = NewObject<UPixelStreaming2MediaIOOutput>();
		// Note the number of texture buffers is how many textures we have in reserve to copy into while we wait for other captures to complete
		// On slower hardware this number needs to be bigger. Testing on AWS T4 GPU's (which are sort of like min-spec for PS) we determined
		// the default number (4) is too low and will cause media capture to regularly overrun (which results in either a skipped frame or a
		// GPU flush depending on the EMediaCaptureOverrunAction option below). After testing, it was found that 8 textures (the max),
		// reduced overruns to infrequent levels on the AWS T4 GPU.
		MediaOutput->NumberOfTextureBuffers = 8;
		MediaCapture->SetMediaOutput(MediaOutput);
		MediaCapture->SetVideoProducer(AsShared());
		MediaCapture->OnStateChangedNative.AddSP(this, &FVideoProducerMediaCapture::OnCaptureActiveViewportStateChanged);

		FMediaCaptureOptions Options;
		Options.bSkipFrameWhenRunningExpensiveTasks = false;
		Options.OverrunAction = EMediaCaptureOverrunAction::Skip;
		Options.ResizeMethod = EMediaCaptureResizeMethod::None;

		// Start capturing the active viewport
		MediaCapture->CaptureActiveSceneViewport(Options);
	}

	void FVideoProducerMediaCapture::LateStartActiveViewportCapture()
	{
		// Bind the OnEndFrame delegate to ensure we only start capture once a frame has been rendered
		OnFrameEndDelegateHandle = FCoreDelegates::OnEndFrame.AddSP(this, &FVideoProducerMediaCapture::StartActiveViewportCapture);
	}

	FString FVideoProducerMediaCapture::ToString()
	{
		return VideoProducerIdentifiers::FVideoProducerMediaCapture;
	}

	void FVideoProducerMediaCapture::OnCaptureActiveViewportStateChanged()
	{
		if (!MediaCapture)
		{
			return;
		}

		switch (MediaCapture->GetState())
		{
			case EMediaCaptureState::Capturing:
				UE_LOGF(LogPixelStreaming2, Log, "Starting media capture for Pixel Streaming.");
				break;
			case EMediaCaptureState::Stopped:
				if (MediaCapture->WasViewportResized())
				{
					UE_LOGF(LogPixelStreaming2, Log, "Pixel Streaming capture was stopped due to resize, going to restart capture.");
					// If it was stopped and viewport resized we assume resize caused the stop, so try a restart of capture here.
					StartActiveViewportCapture();
				}
				else
				{
					UE_LOGF(LogPixelStreaming2, Log, "Stopping media capture for Pixel Streaming.");
				}
				break;
			case EMediaCaptureState::Error:
				UE_LOGF(LogPixelStreaming2, Log, "Pixel Streaming capture hit an error, capturing will stop.");
				break;
			default:
				break;
		}
	}

} // namespace UE::PixelStreaming2
