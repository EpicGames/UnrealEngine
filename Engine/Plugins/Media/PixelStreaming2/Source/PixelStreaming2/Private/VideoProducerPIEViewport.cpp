// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerPIEViewport.h"

#include "Framework/Application/SlateApplication.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/Geometry.h"
#include "Layout/WidgetPath.h"
#include "PixelCaptureInputFrameRHI.h"
#include "Rendering/SlateRenderer.h"
#include "UtilsAsync.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

namespace UE::PixelStreaming2
{
	TSharedPtr<FVideoProducerPIEViewport> FVideoProducerPIEViewport::Create()
	{
		TSharedPtr<FVideoProducerPIEViewport> NewInput = TSharedPtr<FVideoProducerPIEViewport>(new FVideoProducerPIEViewport());
		TWeakPtr<FVideoProducerPIEViewport>	  WeakInput = NewInput;

		UE::PixelStreaming2::DoOnGameThread([WeakInput]() {
			if (TSharedPtr<FVideoProducerPIEViewport> Input = WeakInput.Pin())
			{
				FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
				if(!Renderer)
				{
					return;
				}
 
				Renderer->OnBackBufferReadyToPresent().AddSP(Input.ToSharedRef(), &FVideoProducerPIEViewport::OnBackBufferReadyToPresent);
				FSlateApplication::Get().OnPreTick().AddSP(Input.ToSharedRef(), &FVideoProducerPIEViewport::OnPreTick);
			}
		});

		return NewInput;
	}

	FString FVideoProducerPIEViewport::ToString()
	{
		return VideoProducerIdentifiers::FVideoProducerPIEViewport;
	}

	bool FVideoProducerPIEViewport::ShouldCaptureViewport()
	{
#if WITH_EDITOR
		return GEditor && GEditor->PlayWorld && GEditor->PlayWorld->WorldType == EWorldType::PIE;
#else
		return false;
#endif
	}
	
	void FVideoProducerPIEViewport::OnPreTick(float DeltaTime)
	{
		if (!ShouldCaptureViewport())
		{
			return;
		}
	
		TargetViewport = FSlateApplication::Get().GetGameViewport();
		if (TargetViewport.IsValid())
		{
			TargetWindow = FSlateApplication::Get().FindWidgetWindow(TargetViewport.Pin().ToSharedRef());
			if (TargetWindow.IsValid())
			{
				CalculateCaptureRegion(TargetViewport.Pin().ToSharedRef(), TargetWindow.Pin().ToSharedRef());
			}
		}
	}
} // namespace UE::PixelStreaming2