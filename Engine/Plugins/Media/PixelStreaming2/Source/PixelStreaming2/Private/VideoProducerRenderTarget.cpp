// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerRenderTarget.h"

#include "Engine/TextureRenderTarget2D.h"
#include "PixelCaptureInputFrameRHI.h"
#include "Rendering/SlateRenderer.h"
#include "TextureResource.h"

namespace UE::PixelStreaming2
{

	TSharedPtr<FVideoProducerRenderTarget> FVideoProducerRenderTarget::Create(UTextureRenderTarget2D* Target)
	{
		return TSharedPtr<FVideoProducerRenderTarget>(new FVideoProducerRenderTarget(Target));
	}

	FVideoProducerRenderTarget::~FVideoProducerRenderTarget()
	{
		FCoreDelegates::OnEndFrameRT.Remove(DelegateHandle);
	}

	FVideoProducerRenderTarget::FVideoProducerRenderTarget(UTextureRenderTarget2D* InTarget)
		: Target(InTarget)
		, DelegateHandle(FCoreDelegates::OnEndFrameRT.AddRaw(this, &FVideoProducerRenderTarget::OnEndFrameRenderThread))
	{
	}

	void FVideoProducerRenderTarget::OnEndFrameRenderThread()
	{
		if (!Target)
		{
			return;
		}

		FTextureResource* Resource = Target->GetResource();
		if (!Resource)
		{
			return;
		}

		FTextureRHIRef Texture = Resource->GetTexture2DRHI();
		if (!Texture)
		{
			return;
		}
		
		PushFrame(FPixelCaptureInputFrameRHI(Texture));
	}

	FString FVideoProducerRenderTarget::ToString()
	{
		if (Target)
		{
			FString TargetName = Target->GetName();
			return VideoProducerIdentifiers::FVideoProducerRenderTarget + FString::Printf(TEXT(" (%s)"), *TargetName);
		}
		return VideoProducerIdentifiers::FVideoProducerRenderTarget;
	}

} // namespace UE::PixelStreaming2