// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerViewportBase.h"

#include "EngineModule.h"
#include "Framework/Application/SlateApplication.h"
#include "GlobalShader.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/Geometry.h"
#include "Layout/WidgetPath.h"
#include "MediaShaders.h"
#include "PixelCaptureInputFrameRHI.h"
#include "ScreenPass.h"
#include "SceneView.h"
#include "RenderGraph.h"
#include "RHI.h"

namespace UE::PixelStreaming2
{
	EVideoProducerCapabilities FVideoProducerViewportBase::GetCapabilities()
	{
#if PLATFORM_APPLE
		// On Mac, we must not claim ProducesPreprocessedFrames so that the downstream
		// capturer uses the fenced RHI/MediaCapture path, which creates its own CPUReadback
		// output texture with proper GPU synchronization. The NoCopy path (triggered by
		// ProducesPreprocessedFrames) passes our texture directly to the encoder without
		// synchronization, causing flickering on Metal.
		return EVideoProducerCapabilities::Default;
#else
		return EVideoProducerCapabilities::ProducesPreprocessedFrames;
#endif
	}
	
	bool FVideoProducerViewportBase::ShouldCaptureViewport()
	{
		return false;
	}

	void FVideoProducerViewportBase::CalculateCaptureRegion(TSharedRef<SViewport> Viewport, TSharedRef<SWindow> Window)
	{
		check(IsInGameThread());

		FScopeLock CSLock(&CaptureRectCS);
	
		CaptureRect = FIntRect(0, 0, 0, 0);
	
		FGeometry InnerWindowGeometry = Window->GetWindowGeometryInWindow();
	
		// Find the widget path relative to the window
		FArrangedChildren JustWindow(EVisibility::Visible);
		JustWindow.AddWidget(FArrangedWidget(Window, InnerWindowGeometry));
	
		FWidgetPath WidgetPath(Window, JustWindow);
		if (WidgetPath.ExtendPathTo(FWidgetMatcher(Viewport), EVisibility::Visible))
		{
			FArrangedWidget ArrangedWidget = WidgetPath.FindArrangedWidget(Viewport).Get(FArrangedWidget::GetNullWidget());
		
			FVector2D Position = ArrangedWidget.Geometry.GetAbsolutePosition();
			FVector2D Size = ArrangedWidget.Geometry.GetAbsoluteSize();
		
			CaptureRect = FIntRect(
				Position.X,
				Position.Y,
				Position.X + Size.X,
				Position.Y + Size.Y);
		}
	}

	void FVideoProducerViewportBase::OnBackBufferReadyToPresent(SWindow& Window, ISlateViewportProvider& ViewportProvider)
	{	
		if (!ShouldCaptureViewport())
		{
			return;
		}
		
		if (!TargetWindow.IsValid() || !TargetWindow.HasSameObject(&Window))
		{
			return;
		}
	
		FScopeLock CSLock(&CaptureRectCS);
	
		if (CaptureRect.IsEmpty())
		{
			return;
		}

		FRHITexture* BackBuffer = ViewportProvider.GetBackBufferResource();
	
		// Ensure the capture rect in the CopyInfo is in the bound of InputTexture.
		FIntPoint VirtualPos{
			FMath::Min(FMath::Max(CaptureRect.Min.X, 0), BackBuffer->GetDesc().Extent.X),
			FMath::Min(FMath::Max(CaptureRect.Min.Y, 0), BackBuffer->GetDesc().Extent.Y)
		};
		FIntPoint VirtualSize{
			FMath::Min(CaptureRect.Width(), FMath::Max((BackBuffer->GetDesc().Extent.X - VirtualPos.X), 0)),
			FMath::Min(CaptureRect.Height(), FMath::Max((BackBuffer->GetDesc().Extent.Y - VirtualPos.Y), 0))
		};
	
		if (VirtualSize.X == 0 || VirtualSize.Y == 0)
		{
			return; // discard for copying nothing
		}
	
		TRefCountPtr<IPooledRenderTarget> OutTexture;
		{
			FRDGBuilder GraphBuilder(FRHICommandListImmediate::Get());
			FRDGTextureRef InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(BackBuffer, TEXT("VideoProducerViewportBaseInput")));
			// TODO (william.belcher): The cropped texture having an 8bit pixel format matches the full editor video producer code path but we may need to reconsider this if we want to add 10bit video support
			FRDGTextureRef CroppedTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(VirtualSize, EPixelFormat::PF_B8G8R8A8, FClearValueBinding::None, GetTexCreateFlags()), TEXT("VideoProducerViewportBaseCropped"));

			{
				// Configure our viewports appropriately
				FScreenPassTextureViewport InputViewport(InputTexture, FIntRect(VirtualPos, VirtualPos + VirtualSize));
				FScreenPassTextureViewport OutputViewport(CroppedTexture, FIntRect(FIntPoint(0, 0), VirtualSize));

				// Rectangle area to use from the source texture
				const FIntRect ViewRect(VirtualPos, VirtualPos + VirtualSize);

				// Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
				FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game)).SetTime(FGameTime()));
				FSceneViewInitOptions ViewInitOptions;
				ViewInitOptions.ViewFamily = &ViewFamily;
				ViewInitOptions.SetViewRectangle(ViewRect);
				ViewInitOptions.ViewOrigin = FVector::ZeroVector;
				ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
				ViewInitOptions.ProjectionMatrix = FMatrix::Identity;

				GetRendererModule().CreateAndInitSingleView(GraphBuilder.RHICmdList, &ViewFamily, &ViewInitOptions);
				const FSceneView &View = *ViewFamily.Views[0];
				// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
				int32 ConversionOperation = 0; // None
				FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(ConversionOperation);

				TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
				FModifyAlphaSwizzleRgbaPS::FParameters *PixelShaderParameters = GraphBuilder.AllocParameters<FModifyAlphaSwizzleRgbaPS::FParameters>();
				PixelShaderParameters->InputTexture = InputTexture;
				PixelShaderParameters->InputSampler = TStaticSamplerState<SF_Point>::GetRHI();
				PixelShaderParameters->RenderTargets[0] = FRenderTargetBinding{CroppedTexture, ERenderTargetLoadAction::ELoad};

				TShaderMapRef<FScreenPassVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
				// Add screen pass to convert whatever format the editor produces to BGRA8
				AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("VideoProducerViewportBaseCropped"), View, OutputViewport, InputViewport, VertexShader, PixelShader, PixelShaderParameters);
			}

			GraphBuilder.QueueTextureExtraction(CroppedTexture, &OutTexture);

			GraphBuilder.Execute();
		}

		PushFrame(FPixelCaptureInputFrameRHI(OutTexture->GetRHI()));
	}
} // namespace UE::PixelStreaming2