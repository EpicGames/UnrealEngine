// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoSink.h"

#include "Async/Async.h"
#include "IPixelStreaming2VideoConsumer.h"
#include "PixelStreaming2Trace.h"
#include "RenderTargetPool.h"
#include "TextureResource.h"
#include "Framework/Application/SlateApplication.h"

namespace UE::PixelStreaming2
{

	FVideoSink::~FVideoSink()
	{
		FScopeLock Lock(&VideoConsumersCS);
		for (auto Iter = VideoConsumers.CreateIterator(); Iter; ++Iter)
		{
			TWeakPtrVariant<IPixelStreaming2VideoConsumer> VideoConsumer = *Iter;
			Iter.RemoveCurrent();
			if (TStrongPtrVariant<IPixelStreaming2VideoConsumer> PinnedVideoConsumer = VideoConsumer.Pin(); PinnedVideoConsumer.IsValid())
			{
				PinnedVideoConsumer.Get()->OnVideoConsumerRemoved();
			}
		}
	}

	void FVideoSink::AddVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>& VideoConsumer)
	{
		FScopeLock Lock(&VideoConsumersCS);

		if (!VideoConsumers.Contains(VideoConsumer))
		{
			VideoConsumers.Add(VideoConsumer);
			if (TStrongPtrVariant<IPixelStreaming2VideoConsumer> PinnedVideoConsumer = VideoConsumer.Pin(); PinnedVideoConsumer.IsValid())
			{	
				PinnedVideoConsumer.Get()->OnVideoConsumerAdded();
			}
		}
	}

	void FVideoSink::RemoveVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>& VideoConsumer)
	{
		FScopeLock Lock(&VideoConsumersCS);
		if (VideoConsumers.Contains(VideoConsumer))
		{
			VideoConsumers.Remove(VideoConsumer);

			if (TStrongPtrVariant<IPixelStreaming2VideoConsumer> PinnedVideoConsumer = VideoConsumer.Pin(); PinnedVideoConsumer.IsValid())
			{	
				PinnedVideoConsumer.Get()->OnVideoConsumerRemoved();
			}
		}
	}

	bool FVideoSink::HasVideoConsumers()
	{
		return VideoConsumers.Num() > 0;
	}

	void FVideoSink::OnVideoData(FTextureRHIRef Frame)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FVideoSink::OnData", PixelStreaming2Channel);

		if (!HasVideoConsumers() || bIsMuted || IsEngineExitRequested())
		{
			return;
		}

		// Iterate video consumers and pass this data to their buffers
		FScopeLock Lock(&VideoConsumersCS);
		for (TWeakPtrVariant<IPixelStreaming2VideoConsumer> VideoConsumer : VideoConsumers)
		{
			if (TStrongPtrVariant<IPixelStreaming2VideoConsumer> PinnedVideoConsumer = VideoConsumer.Pin(); PinnedVideoConsumer.IsValid())
			{
				PinnedVideoConsumer.Get()->ConsumeFrame(Frame);
			}
		}
	}

	void FVideoSink::SetMuted(bool bInIsMuted)
	{
		bIsMuted = bInIsMuted;
	}

} // namespace UE::PixelStreaming2