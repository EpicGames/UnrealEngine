// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoSource.h"

namespace UE::PixelStreaming2
{
	void FVideoSource::PushFrame()
	{
	}

	void FVideoSource::ForceKeyFrame()
	{
	}

	void FVideoSource::SetMuted(bool bInIsMuted)
	{
		bIsMuted = bInIsMuted;
	}
} // namespace UE::PixelStreaming2
