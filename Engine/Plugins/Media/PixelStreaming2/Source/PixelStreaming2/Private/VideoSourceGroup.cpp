// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoSourceGroup.h"

#include "Logging.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2Trace.h"
#include "VideoSource.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "UnrealClient.h"

namespace UE::PixelStreaming2
{
	class FFrameRunnable : public FRunnable, public FSingleThreadRunnable
	{
	public:
		FFrameRunnable(TWeakPtr<FVideoSourceGroup> VideoSourceGroup, bool bEnabled)
			: bEnabled(bEnabled)
			, OuterVideoSourceGroup(VideoSourceGroup)
		{
		}

		virtual ~FFrameRunnable() = default;

		void SetEnabled(bool bInEnabled)
		{
			bEnabled = bInEnabled;

			if (bEnabled)
			{
				// The thread has enabled so wake it up
				FrameEvent->Trigger();
			}
		}

		virtual bool Init() override
		{
			return true;
		}

		virtual uint32 Run() override
		{
			bIsRunning = true;

			while (bIsRunning)
			{
				if (!bEnabled)
				{
					// Sleep the thread indefinitely because it is not enabled
					FrameEvent->Wait();
				}

				// Need to check bIsRunning in order to not push a frame when running is disabled and FrameEvent->Wait() has returned.
				if (TSharedPtr<FVideoSourceGroup> VideoSourceGroup = OuterVideoSourceGroup.Pin(); bIsRunning && VideoSourceGroup)
				{
					const double TimeSinceLastSubmitMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - LastSubmitCycles);

					// Decrease this value to make expected frame delivery more precise, however may result in more old frames being sent
					const double PrecisionFactor = 0.1;
					const double WaitFactor = UPixelStreaming2PluginSettings::CVarDecoupleWaitFactor.GetValueOnAnyThread();

					// In "auto" mode vary this value based on historical average
					const double TargetSubmitMs = 1000.0 / VideoSourceGroup->GetFPS();
					const double TargetSubmitMsWithPadding = TargetSubmitMs * WaitFactor;
					const double MinSubmitMs = TargetSubmitMs * (1.f / WaitFactor);
					const double CloseEnoughMs = TargetSubmitMs * PrecisionFactor;
					const bool	 bFrameOverdue = TimeSinceLastSubmitMs >= TargetSubmitMsWithPadding;

					// Check frame arrived in time
					if (!bFrameOverdue)
					{
						// Frame arrived in a timely fashion, but is it too soon to maintain our target rate? If so, sleep.
						double WaitTimeRemainingMs = TargetSubmitMsWithPadding - TimeSinceLastSubmitMs;
						if (WaitTimeRemainingMs > CloseEnoughMs)
						{
							bool bGotNewFrame = FrameEvent->Wait(WaitTimeRemainingMs);
							if (!bGotNewFrame)
							{
								UE_LOGF(LogPixelStreaming2, VeryVerbose, "Old frame submitted");
							}
						}
					}

					// Push frame immediately
					PushFrame(VideoSourceGroup);
				}
			}

			return 0;
		}

		virtual void Stop() override
		{
			bIsRunning = false;
			// Wake the thread in case it was sleeping. This will cause it to then exit the ::Run loop
			FrameEvent->Trigger();
		}

		virtual void Exit() override
		{
			Stop();
		}

		virtual FSingleThreadRunnable* GetSingleThreadInterface() override
		{
			bIsRunning = true;
			return this;
		}

		/**
		 * Note this function is required as part of `FSingleThreadRunnable` and only gets called when engine is run in single-threaded mode,
		 * so the logic is much less complex as this is not a case we particularly optimize for, a simple tick on an interval will be acceptable.
		 */
		virtual void Tick() override
		{
			if (TSharedPtr<FVideoSourceGroup> VideoSourceGroup = OuterVideoSourceGroup.Pin())
			{
				const uint64 NowCycles = FPlatformTime::Cycles64();
				const double DeltaMs = FPlatformTime::ToMilliseconds64(NowCycles - LastSubmitCycles);
				const double TargetSubmitMs = 1000.0 / VideoSourceGroup->GetFPS();
				if (DeltaMs >= TargetSubmitMs)
				{
					PushFrame(VideoSourceGroup);
				}
			}
		}

		void PushFrame(TSharedPtr<FVideoSourceGroup> VideoSourceGroup)
		{
			VideoSourceGroup->PushFrame();
			LastSubmitCycles = FPlatformTime::Cycles64();
		}

		std::atomic<bool>			bEnabled = false;
		std::atomic<bool>			bIsRunning = false;
		TWeakPtr<FVideoSourceGroup> OuterVideoSourceGroup = nullptr;
		uint64						LastSubmitCycles = 0;

		// Use this event to signal when we should wake and also how long we should sleep for between transmitting a frame.
		FEventRef FrameEvent;
	};

	TSharedPtr<FVideoSourceGroup> FVideoSourceGroup::Create(TSharedPtr<FVideoCapturer> InVideoCapturer)
	{
		TSharedPtr<FVideoSourceGroup> VideoSourceGroup = TSharedPtr<FVideoSourceGroup>(new FVideoSourceGroup());

		VideoSourceGroup->FrameDelegateHandle = InVideoCapturer->OnFrameCaptured.AddSP(VideoSourceGroup.ToSharedRef(), &FVideoSourceGroup::OnFrameCaptured);

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			VideoSourceGroup->FpsDelegateHandle = Delegates->OnWebRTCFpsChanged.AddSP(VideoSourceGroup.ToSharedRef(), &FVideoSourceGroup::OnWebRtcFpsChanged);
			VideoSourceGroup->DecoupleDelegateHandle = Delegates->OnDecoupleFramerateChanged.AddSP(VideoSourceGroup.ToSharedRef(), &FVideoSourceGroup::OnDecoupleFramerateChanged);
		}

		return VideoSourceGroup;
	}

	FVideoSourceGroup::FVideoSourceGroup()
		: bDecoupleFramerate(UPixelStreaming2PluginSettings::CVarDecoupleFramerate.GetValueOnAnyThread())
		, FramesPerSecond(UPixelStreaming2PluginSettings::CVarWebRTCFps.GetValueOnAnyThread())
	{
	}

	FVideoSourceGroup::~FVideoSourceGroup()
	{
		Stop();
	}

	void FVideoSourceGroup::SetFPS(int32 InFramesPerSecond)
	{
		bFPSOverride = true;

		FramesPerSecond = InFramesPerSecond;
	}

	int32 FVideoSourceGroup::GetFPS()
	{
		return FramesPerSecond;
	}

	void FVideoSourceGroup::OnWebRtcFpsChanged(IConsoleVariable* Var)
	{
		// User has manually overriden the stream fps so don't respect the cvar change
		if (bFPSOverride)
		{
			return;
		}

		FramesPerSecond = Var->GetInt();
	}

	void FVideoSourceGroup::SetDecoupleFramerate(bool bDecouple)
	{
		bDecoupleOverride = true;

		bDecoupleFramerate = bDecouple;

		if (FrameRunnable)
		{
			FrameRunnable->SetEnabled(bDecoupleFramerate);
		}
	}

	void FVideoSourceGroup::OnDecoupleFramerateChanged(IConsoleVariable* Var)
	{
		// User has manually overriden the decouple bool so don't respect the cvar change
		if (bDecoupleOverride)
		{
			return;
		}

		bDecoupleFramerate = Var->GetBool();

		if (FrameRunnable)
		{
			FrameRunnable->SetEnabled(bDecoupleFramerate);
		}
	}

	void FVideoSourceGroup::AddVideoSource(TSharedPtr<FVideoSource> VideoSource)
	{
		{
			FScopeLock Lock(&CriticalSection);
			VideoSources.Add(VideoSource);
		}
	}

	void FVideoSourceGroup::RemoveVideoSource(const FVideoSource* ToRemove)
	{
		{
			FScopeLock Lock(&CriticalSection);
			VideoSources.RemoveAll([ToRemove](const TSharedPtr<FVideoSource>& Target) {
				return Target.Get() == ToRemove;
			});
		}
	}

	void FVideoSourceGroup::RemoveAllVideoSources()
	{
		{
			FScopeLock Lock(&CriticalSection);
			VideoSources.Empty();
		}
	}

	void FVideoSourceGroup::Start()
	{
		bRunning = true;
		StartThread();
	}

	void FVideoSourceGroup::Stop()
	{
		bRunning = false;
		StopThread();
	}

	void FVideoSourceGroup::PushFrame()
	{
		// The FrameRunnable may push a frame post engine exit because it runs in its own thread
		if (IsEngineExitRequested())
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming2 Video Source Group Tick", PixelStreaming2Channel);
		FScopeLock Lock(&CriticalSection);

		// for each player session, push a frame
		for (auto& VideoSource : VideoSources)
		{
			if (VideoSource)
			{
				VideoSource->PushFrame();
			}
		}
	}

	void FVideoSourceGroup::OnFrameCaptured()
	{
		if (!bDecoupleFramerate)
		{
			// Source FPS and engine FPS are coupled. Manually push a frame
			PushFrame();
		}
	}

	void FVideoSourceGroup::StartThread()
	{
		FScopeLock Lock(&RunnableCriticalSection);
		if (!FrameRunnable)
		{
			FrameRunnable = MakeShared<FFrameRunnable>(AsWeak(), bDecoupleFramerate);
		}

		if (!FrameThread)
		{
			FrameThread = TSharedPtr<FRunnableThread>(FRunnableThread::Create(FrameRunnable.Get(), TEXT("FVideoSourceGroup Thread"), 0, TPri_TimeCritical));
		}
	}

	void FVideoSourceGroup::StopThread()
	{
		FScopeLock Lock(&RunnableCriticalSection);
		if (FrameRunnable)
		{
			FrameRunnable->Stop();
		}
		
		if (FrameThread)
		{
			FrameThread->Kill(true);
			FrameThread.Reset();
		}

		// Need to reset FrameRunnable after FrameThread as that will try to access FrameRunnable when killing it.
		if (FrameRunnable)
		{
			FrameRunnable.Reset();
		}
	}

	void FVideoSourceGroup::ForceKeyFrame()
	{
		TArray<TSharedPtr<FVideoSource>> VideoSourcesCopy;
		{
			// Grab a copy of the SharedPtr inside the lock to make sure it does not change elsewhere
			// while allowing the functions to be called on the video source outside the lock.
			FScopeLock Lock(&CriticalSection);
			VideoSourcesCopy = VideoSources;
		}

		for (auto& VideoSource : VideoSourcesCopy)
		{
			if (VideoSource)
			{
				VideoSource->ForceKeyFrame();
			}
		}
	}
} // namespace UE::PixelStreaming2
