// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "IPixelStreaming2AudioProducer.h"
#include "IPixelStreaming2Streamer.h"
#include "IPixelStreaming2VideoProducer.h"

/**
 * The IPixelStreaming2Module interface manages the core functionality of the Pixel Streaming system.
 * This class provides access to streamers, video and audio producers, and handles starting and stopping the streaming process.
 * It also allows interaction with the signalling server and manages the lifecycle of streamers within the Pixel Streaming system.
 */
class IPixelStreaming2Module : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware calling this during the shutdown phase, though. Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreaming2Module& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreaming2Module>("PixelStreaming2");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PixelStreaming2");
	}

	/**
	 * Event fired when internal streamer is initialized and the methods on this module are ready for use.
	 */
	DECLARE_EVENT_OneParam(IPixelStreaming2Module, FReadyEvent, IPixelStreaming2Module&);

	/**
	 * A getter for the OnReady event. Intent is for users to call IPixelStreaming2Module::Get().OnReady().AddXXX.
	 * @return The bindable OnReady event.
	 */
	virtual FReadyEvent& OnReady() = 0;

	/**
	 * Is the PixelStreaming2 module actually ready to use? Is the streamer created.
	 * @return True if Pixel Streaming module methods are ready for use.
	 */
	virtual bool IsReady() = 0;

	/**
	 * Starts streaming on all streamers.
	 * @return False if the module was not able to start streaming.
	 */
	virtual bool StartStreaming() = 0;

	/**
	 * Stops all streamers from streaming.
	 */
	virtual void StopStreaming() = 0;

	/**
	 * Creates a new streamer.
	 * @param StreamerId - The ID of the Streamer to be created.
	 * @return SharedPtr to the streamer.
	 */
	virtual TSharedPtr<IPixelStreaming2Streamer> CreateStreamer(const FString& StreamerId, TOptional<FString> Type = TOptional<FString>()) = 0;

	/**
	 * @brief Creates a new video producer.
	 * @return An video producer interface for you to push video frames in to.
	 */
	UE_DEPRECATED(5.7, "CreateVideoProducer has been deprecated. You can create a class that inherits from IPixelStreaming2VideoProducer and call PushFrame.")
	virtual TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducer() = 0;

	/**
	 * @brief Creates a new audio producer. Any audio you push in with `PushAudio` will be mixed with other audio sources before being streamed.
	 * @note Users are responsible for the lifetime of this object.
	 * @return An audio producer interface for you to push audio in to.
	 */
	UE_DEPRECATED(5.7, "CreateAudioProducer has been deprecated. You can create a class that inherits from IPixelStreaming2AudioProducer and call PushAudio.")
	virtual TSharedPtr<IPixelStreaming2AudioProducer> CreateAudioProducer() = 0;

	/**
	 * Returns a TArray containing the keys to the currently held streamers.
	 * @return TArray containing the keys to the currently held streamers.
	 */
	virtual TArray<FString> GetStreamerIds() = 0;

	/**
	 * Find a streamer by an ID.
	 * @param StreamerId	-	The ID of the streamer to be found.
	 * @return A pointer to the interface for a streamer. nullptr if the streamer isn't found.
	 */
	virtual TSharedPtr<IPixelStreaming2Streamer> FindStreamer(const FString& StreamerId) = 0;

	/**
	 * Remove a streamer by an ID
	 * @param StreamerId	-	The ID of the streamer to be removed.
	 * @return The removed streamer. nullptr if the streamer wasn't found.
	 */
	virtual TSharedPtr<IPixelStreaming2Streamer> DeleteStreamer(const FString& StreamerId) = 0;

	/**
	 * Remove a streamer by its pointer
	 * @param ToBeDeleted The streamer to remove from the internal management.
	 */
	virtual void DeleteStreamer(TSharedPtr<IPixelStreaming2Streamer> ToBeDeleted) = 0;

	/**
	 * Get the Default Streamer ID.
	 * @return FString The default streamer ID.
	 */
	virtual FString GetDefaultStreamerID() = 0;

	/**
	 * Get the Default Connection URL ("ws://127.0.0.1:8888").
	 * @return FString The default connection url ("ws://127.0.0.1:8888").
	 */
	virtual FString GetDefaultConnectionURL() = 0;

	/**
	 * Get the Default Signaling URL ("ws://127.0.0.1:8888").
	 * @return FString The default signaling url ("ws://127.0.0.1:8888").
	 */
	UE_DEPRECATED(5.6, "GetDefaultSignallingURL has been deprecated. Please use GetDefaultConnectionURL instead.")
	virtual FString GetDefaultSignallingURL() { return GetDefaultConnectionURL(); }

	/**
	 * @brief A method for iterating through all of the streamers on the module.
	 *
	 * @param Func The lambda to execute with each streamer.
	 */
	virtual void ForEachStreamer(const TFunction<void(TSharedPtr<IPixelStreaming2Streamer>)>& Func) = 0;
};
