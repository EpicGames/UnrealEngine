// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "UObject/Object.h"

#include "PixelStreaming2Delegates.generated.h"

#define UE_API PIXELSTREAMING2_API

/**
 * Pixel Streaming Delegates that can be invoked when pixel streaming events take place.
 * Includes blueprint and native c++ delegates.
 */
UCLASS(MinimalAPI)
class UPixelStreaming2Delegates : public UObject
{
	GENERATED_BODY()

public:
	/** Blueprint delegate type for when a connection to the signalling server was made. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FConnectedToSignallingServer, FString, StreamerId);

	/** Invoked when a connection to the signalling server was made. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FConnectedToSignallingServer OnConnectedToSignallingServer;
	
	/** Delegate type for when a connection to the signalling server was made. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FConnectedToSignallingServerNative, FString /* StreamerId */);

	/** Invoked when a connection to the signalling server was made. */
	FConnectedToSignallingServerNative OnConnectedToSignallingServerNative;

	/** Blueprint delegate type for when a connection to the signalling server was lost. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FDisconnectedFromSignallingServer, FString, StreamerId);

	/** Invoked when a connection to the signalling server was lost. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FDisconnectedFromSignallingServer OnDisconnectedFromSignallingServer;
	
	/** Delegate type for when a connection to the signalling server was lost. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FDisconnectedFromSignallingServerNative, FString /* StreamerId */);

	/** Invoked when a connection to the signalling server was lost. */
	FDisconnectedFromSignallingServerNative OnDisconnectedFromSignallingServerNative;

	/** Blueprint delegate type for when a new connection has been made to the session. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FNewConnection, FString, StreamerId, FString, PlayerId);

	/** Invoked when a new connection has been made to the session. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FNewConnection OnNewConnection;

	/** Delegate type for when a new connection has been made to the session. */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FNewConnectionNative, FString /* Streamer id*/, FString /* Peer id */);

	/** Invoked when a new connection has been made to the session. */
	FNewConnectionNative OnNewConnectionNative;

	/** Blueprint delegate type for when a connection to a player was lost. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FClosedConnection, FString, StreamerId, FString, PlayerId);

	/** Invoked when a connection to a player was lost. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FClosedConnection OnClosedConnection;

	/** Delegate type for when a connection to a player was lost. */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FClosedConnectionNative, FString /* Streamer id */, FString /* Peer id */);

	/** Invoked when a connection to a player was lost. */
	FClosedConnectionNative OnClosedConnectionNative;

	/** Blueprint delegate type for when all connections have closed and nobody is viewing or interacting with the app. This is an opportunity to reset the app. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAllConnectionsClosed, FString, StreamerId);

	/** Invoked when all connections have closed and nobody is viewing or interacting with the app. This is an opportunity to reset the app. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FAllConnectionsClosed OnAllConnectionsClosed;

	/** Delegate type for when all connections have closed and nobody is viewing or interacting with the app. This is an opportunity to reset the app. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FAllConnectionsClosedNative, FString);

	/** Invoked when all connections have closed and nobody is viewing or interacting with the app. This is an opportunity to reset the app. */
	FAllConnectionsClosedNative OnAllConnectionsClosedNative;

	/** Blueprint delegate type for when a new data track has been opened. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FDataTrackOpen, FString, StreamerId, FString, PlayerId);

	/** Invoked when a new data track has been opened. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FDataTrackOpen OnDataTrackOpen;
	
	/** Delegate type for when a new data track has been opened. */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FDataTrackOpenNative, FString /* Streamer id */, FString /* Peer id */);

	/** Invoked when a new data track has been opened. */
	FDataTrackOpenNative OnDataTrackOpenNative;

	/** Blueprint delegate type for when an existing data track has been closed. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FDataTrackClosed, FString, StreamerId, FString, PlayerId);

	/** Invoked when an existing data track has been closed. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FDataTrackClosed OnDataTrackClosed;

	/** Delegate type for when an existing data track has been closed. */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FDataTrackClosedNative, FString /* Streamer id */, FString /* Peer id */);

	/** Invoked when an existing data track has been closed. */
	FDataTrackClosedNative OnDataTrackClosedNative;

	/** Delegate type for when a new video track has been opened. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FVideoTrackOpenNative, FString /* Streamer id */, FString /* Peer id */, bool /* bIsRemote */);

	/** Invoked when a new video track has been opened. */
	FVideoTrackOpenNative OnVideoTrackOpenNative;

	/** Delegate type for when an existing video track has been closed. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FVideoTrackClosedNative, FString /* Streamer id */, FString /* Peer id */, bool /* bIsRemote */);

	/** Invoked when an existing video track has been closed. */
	FVideoTrackClosedNative OnVideoTrackClosedNative;

	/** Delegate type for when a new audio track has been opened. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FAudioTrackOpenNative, FString /* Streamer id */, FString /* Peer id */, bool /* bIsRemote */);

	/** Invoked when a new audio track has been opened. */
	FAudioTrackOpenNative OnAudioTrackOpenNative;

	/** Delegate type for when an existing audio track has been closed. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FAudioTrackClosedNative, FString /* Streamer id */, FString /* Peer id */, bool /* bIsRemote */);

	/** Invoked when an existing audio track has been closed. */
	FAudioTrackClosedNative OnAudioTrackClosedNative;

	/** Blueprint delegate type for when a pixel streaming stat has changed. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FStatsChanged, FString, PlayerId, FName, StatName, float, StatValue);

	/** Invoked when a pixel streaming stat has changed. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FStatsChanged OnStatChanged;

	/** Delegate type for when a pixel streaming stat has changed. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FStatChangedNative, FString /* Peer id */, FName /* Stat name */, float);

	/** Invoked when a pixel streaming stat has changed. */
	FStatChangedNative OnStatChangedNative;

	/** Blueprint delegate type for when the GPU ran out of available hardware encoders and fell back to software encoders. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FFallbackToSoftwareEncoding);

	/** Invoked when the GPU ran out of available hardware encoders and fell back to software encoders. */
	UPROPERTY(BlueprintAssignable, Category = "Pixel Streaming Delegates")
	FFallbackToSoftwareEncoding OnFallbackToSoftwareEncoding;

	/** Delegate type for when the GPU ran out of available hardware encoders and fell back to software encoders. */
	DECLARE_TS_MULTICAST_DELEGATE(FFallbackToSoftwareEncodingNative);

	/** Invoked when the GPU ran out of available hardware encoders and fell back to software encoders. */
	FFallbackToSoftwareEncodingNative OnFallbackToSoftwareEncodingNative;

	/**
	 * @param Get the UPixelStreaming2Delegates singleton.
	 * @return UPixelStreaming2Delegates pointer.
	 */
	static UE_API UPixelStreaming2Delegates* Get();

	UE_API virtual ~UPixelStreaming2Delegates();

private:
	// The singleton object.
	static UE_API UPixelStreaming2Delegates* Singleton;
};

#undef UE_API
