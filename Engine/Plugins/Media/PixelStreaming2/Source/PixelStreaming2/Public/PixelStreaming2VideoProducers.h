// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IPixelStreaming2VideoProducer.h"
#include "TextureResource.h"

#define UE_API PIXELSTREAMING2_API

namespace UE::PixelStreaming2
{
    UE_API TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducerBackBuffer();
    UE_API TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducerRenderTarget(UTextureRenderTarget2D* Target);
    UE_API TSharedPtr<IPixelStreaming2VideoProducer> CreateVideoProducerPIEViewport();
}

#undef UE_API