// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;

namespace UnrealBuildTool.Rules
{
    public class PixelStreaming2Core : ModuleRules
    {
        public PixelStreaming2Core(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(new string[] { 
                "CoreUObject",
                "PixelCapture",
                "RHI",
                "WebRTC"
            });

            PrivateDependencyModuleNames.AddRange(new string[]
            {
                "Core"
            });
        }
    }
}
