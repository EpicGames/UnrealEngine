// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::PixelStreaming2
{
	class FPixelStreaming2CoreModule : public IModuleInterface
	{
	public:
		virtual ~FPixelStreaming2CoreModule() = default;
	};
} // namespace UE::PixelStreaming2

IMPLEMENT_MODULE(UE::PixelStreaming2::FPixelStreaming2CoreModule, PixelStreaming2Core)