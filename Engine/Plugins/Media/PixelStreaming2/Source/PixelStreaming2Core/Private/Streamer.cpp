// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streamer.h"

#include "Features/IModularFeatures.h"
#include "Logging.h"

static FName ModularFeatureName = TEXT("PixelStreaming2 Streamer");

namespace UE::PixelStreaming2
{
	class FDummyStreamer : public IPixelStreaming2Streamer
	{
	public:
		FDummyStreamer() = default;
		virtual ~FDummyStreamer() = default;

		virtual void										 Initialize() override {}
		virtual void										 SetStreamFPS(int32 InFramesPerSecond) override {}
		virtual int32										 GetStreamFPS() override { return 0; }
		virtual void										 SetCoupleFramerate(bool bCouple) override {}
		virtual void										 SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> Input) override {}
		virtual TWeakPtr<IPixelStreaming2VideoProducer>		 GetVideoProducer() override { return nullptr; }
		virtual void AddAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) override {}
		virtual void RemoveAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) override {}
		virtual TArray<TWeakPtr<IPixelStreaming2AudioProducer>> GetAudioProducers() override { return {}; }
		virtual void										 SetConnectionURL(const FString& InSignallingServerURL) override {}
		virtual FString										 GetConnectionURL() override { return FString(); }
		virtual FString										 GetId() override { return TEXT("DummyStreamer"); }
		virtual bool										 IsConnected() override { return false; }
		virtual void										 StartStreaming() override {}
		virtual void										 StopStreaming() override {}
		virtual bool										 IsStreaming() const { return false; }
		virtual FPreConnectionEvent&						 OnPreConnection() override { return StreamingPreConnectionEvent; }
		virtual FStreamingStartedEvent&						 OnStreamingStarted() override { return StreamingStartedEvent; }
		virtual FStreamingStoppedEvent&						 OnStreamingStopped() override { return StreamingStoppedEvent; }
		virtual void										 ForceKeyFrame() override {}
		virtual void										 FreezeStream(UTexture2D* Texture) override {}
		virtual void										 UnfreezeStream() override {}
		virtual void										 SendAllPlayersMessage(FString MessageType, const FString& Descriptor) override {}
		virtual void										 SendPlayerMessage(FString PlayerId, FString MessageType, const FString& Descriptor) override {}
		virtual void										 SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension) override {}
		virtual void										 KickPlayer(FString PlayerId) override {}
		virtual TArray<FString>								 GetConnectedPlayers() override { return {}; }
		virtual TWeakPtr<class IPixelStreaming2InputHandler> GetInputHandler() override { return nullptr; }
		virtual void SetInputHandler(TSharedPtr<class IPixelStreaming2InputHandler> Handler) override {}
		virtual TWeakPtr<IPixelStreaming2AudioSink>			 GetPeerAudioSink(FString PlayerId) override { return nullptr; }
		virtual TWeakPtr<IPixelStreaming2AudioSink>			 GetUnlistenedAudioSink() override { return nullptr; }
		virtual TWeakPtr<IPixelStreaming2VideoSink>			 GetPeerVideoSink(FString PlayerId) override { return nullptr; }
		virtual TWeakPtr<IPixelStreaming2VideoSink>			 GetUnwatchedVideoSink() override { return nullptr; }
		virtual void										 SetConfigOption(const FName& OptionName, const FString& Value) override {}
		virtual bool										 GetConfigOption(const FName& OptionName, FString& OutValue) override { return false; }
		virtual void										 PlayerRequestsBitrate(FString PlayerId, int MinBitrate, int MaxBitrate) override {}
		virtual void										 RefreshStreamBitrate() override {}

	private:
		FPreConnectionEvent	   StreamingPreConnectionEvent;
		FStreamingStartedEvent StreamingStartedEvent;
		FStreamingStoppedEvent StreamingStoppedEvent;
	};

	FString FDummyStreamerFactory::GetStreamType()
	{
		return FString();
	}

	TSharedPtr<IPixelStreaming2Streamer> FDummyStreamerFactory::CreateNewStreamer(const FString& StreamerId)
	{
		return TSharedPtr<FDummyStreamer>(new FDummyStreamer());
	}
} // namespace UE::PixelStreaming2

IPixelStreaming2StreamerFactory::IPixelStreaming2StreamerFactory()
{
	IPixelStreaming2StreamerFactory::RegisterStreamerFactory(this);
}

IPixelStreaming2StreamerFactory::~IPixelStreaming2StreamerFactory()
{
	IPixelStreaming2StreamerFactory::UnregisterStreamerFactory(this);
}

void IPixelStreaming2StreamerFactory::RegisterStreamerFactory(IPixelStreaming2StreamerFactory* InFactory)
{
	IModularFeatures::Get().RegisterModularFeature(ModularFeatureName, InFactory);
}

void IPixelStreaming2StreamerFactory::UnregisterStreamerFactory(IPixelStreaming2StreamerFactory* InFactory)
{
	IModularFeatures::Get().UnregisterModularFeature(ModularFeatureName, InFactory);
}

IPixelStreaming2StreamerFactory* IPixelStreaming2StreamerFactory::Get(const FString& InType)
{
	if (InType == FString())
	{
		return nullptr;
	}

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IPixelStreaming2StreamerFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IPixelStreaming2StreamerFactory>(ModularFeatureName);
	IModularFeatures::Get().UnlockModularFeatureList();

	for (IPixelStreaming2StreamerFactory* Factory : Factories)
	{
		if (Factory && InType == Factory->GetStreamType())
		{
			return Factory;
		}
	}

	UE_LOGFMT(LogPixelStreaming2Core, Warning, "No streamer factory implementation for {0} found. Streamers set to this type will not do anything.", *InType);
	static UE::PixelStreaming2::FDummyStreamerFactory DummyFactory = UE::PixelStreaming2::FDummyStreamerFactory();
	return &DummyFactory;
}

TArray<FString> IPixelStreaming2StreamerFactory::GetAvailableFactoryTypes()
{
	TArray<FString> StreamProtocols;

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IPixelStreaming2StreamerFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IPixelStreaming2StreamerFactory>(ModularFeatureName);
	IModularFeatures::Get().UnlockModularFeatureList();

	for (IPixelStreaming2StreamerFactory* Factory : Factories)
	{
		StreamProtocols.AddUnique(Factory->GetStreamType());
	}

	return StreamProtocols;
}