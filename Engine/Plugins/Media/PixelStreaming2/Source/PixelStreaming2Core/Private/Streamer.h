// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPixelStreaming2Streamer.h"

namespace UE::PixelStreaming2
{
	class FDummyStreamerFactory : public IPixelStreaming2StreamerFactory
	{
	public:
		virtual FString								 GetStreamType() override;
		virtual TSharedPtr<IPixelStreaming2Streamer> CreateNewStreamer(const FString& StreamerId) override;
	};
} // namespace UE::PixelStreaming2