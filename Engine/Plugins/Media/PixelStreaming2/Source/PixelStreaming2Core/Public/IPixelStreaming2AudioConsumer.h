// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IPixelStreaming2AudioConsumer.generated.h"

UINTERFACE(MinimalAPI)
class UPixelStreaming2AudioConsumer : public UInterface
{
	GENERATED_BODY()
};

/**
 * An "Audio Consumer" is an object that is responsible for outputting the audio received from a peer. For example, by
 * passing the audio into a UE submix.
 */
class IPixelStreaming2AudioConsumer
{
	GENERATED_BODY()

public:
	/**
	 * @brief Consume raw audio data.
	 * @param AudioData Pointer to the audio data.
	 * @param SampleRate Audio sample rate in samples per second.
	 * @param NumChannels Number of audio channels. For example 2 for stero audio.
	 * @param NumFrames Number of Audio frames in a single channel.
	 */
	virtual void ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames) = 0;

	/**
	 * @brief Called when a audio consumer is added.
	 */
	virtual void OnAudioConsumerAdded() = 0;

	UE_DEPRECATED(5.7, "OnConsumerAdded has been deprecated. Please use OnAudioConsumerAdded instead.")
	virtual void OnConsumerAdded() {}

	/**
	 * @brief Called when a audio consumer is removed.
	 */
	virtual void OnAudioConsumerRemoved() = 0;
	
	UE_DEPRECATED(5.7, "OnConsumerRemoved has been deprecated. Please use OnAudioConsumerRemoved instead.")
	virtual void OnConsumerRemoved() {}
};
