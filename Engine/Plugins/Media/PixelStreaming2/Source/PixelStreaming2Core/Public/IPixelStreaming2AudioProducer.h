// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IPixelStreaming2AudioProducer.generated.h"

UINTERFACE(MinimalAPI)
class UPixelStreaming2AudioProducer : public UInterface
{
	GENERATED_BODY()
};

/**
 * A "Audio Producer" is an object that you use to push audio frames into the Pixel Streaming system.
 *
 * Example usage:
 *
 * (1) Each new frame you want to push you call `MyAudioProducer->PushFrame(Buffer, NumSamples, NumChannels, SampleRate)`
 *
 * Note: Any audio you push in with `PushAudio` will be mixed with other audio producers before being streamed
 */
class IPixelStreaming2AudioProducer
{
	GENERATED_BODY()

public:
	/**
	 * @brief Pushes audio into the PS2 audio pipeline which will mix with other audio producers before broadcasting.
	 * @param InBuffer Pointer to the audio data.
	 * @param NumSamples Number of Audio frames in a single channel.
	 * @param NumChannels Number of audio channels. For example 2 for stero audio.
	 * @param SampleRate Audio sample rate in samples per second.
	 */
	virtual void PushAudio(const float* InBuffer, int32 NumSamples, int32 NumChannels, int32 SampleRate) final
	{
		OnAudioPushed.Broadcast(InBuffer, NumSamples, NumChannels, SampleRate);
	}

	/**
	 * Event triggered when a frame is pushed to the audio producer. You can use this to hook into the frame stream.
	 * Currently, streamers will listen to this event to know when a new frame is available for encoding.
	 */
	DECLARE_EVENT_FourParams(IPixelStreaming2AudioProducer, FOnAudioPushed, const float* /*InBuffer*/, int32 /*NumSamples*/, int32 /*NumChannels*/, int32 /*SampleRate*/);
	FOnAudioPushed OnAudioPushed;
};
