// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2AudioConsumer.h"
#include "Templates/PointerVariants.h"
#include "UObject/Interface.h"

#include "IPixelStreaming2AudioSink.generated.h"

UINTERFACE(MinimalAPI)
class UPixelStreaming2AudioSink : public UInterface
{
	GENERATED_BODY()
};

/**
 * An "Audio Sink" is an object that receives audio from a singular peer. From here, you can add multiple consumers to
 * output the received audio.
 */
class IPixelStreaming2AudioSink
{
	GENERATED_BODY()

public:
	/**
	 * @brief Add an audio consumer to the sink.
	 * @param AudioConsumer The Audio consumer to add to the sink.
	 */
	UE_DEPRECATED(5.6, "AddAudioConsumer(IPixelStreaming2AudioConsumer*) has been deprecated. Please use AddAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>&) instead.")
	virtual void AddAudioConsumer(IPixelStreaming2AudioConsumer* AudioConsumer) {}

	virtual void AddAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>& AudioConsumer) = 0;

	/**
	 * @brief Remove an audio consumer to remove from the sink.
	 * @param AudioConsumer The Audio consumer to remove from the sink.
	 */
	UE_DEPRECATED(5.6, "RemoveAudioConsumer(IPixelStreaming2AudioConsumer*) has been deprecated. Please use RemoveAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>&) instead.")
	virtual void RemoveAudioConsumer(IPixelStreaming2AudioConsumer* AudioConsumer) {}

	virtual void RemoveAudioConsumer(const TWeakPtrVariant<IPixelStreaming2AudioConsumer>& AudioConsumer) = 0;
};
