// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "IPixelStreaming2AudioProducer.h"
#include "IPixelStreaming2AudioSink.h"
#include "IPixelStreaming2VideoProducer.h"
#include "IPixelStreaming2VideoSink.h"
#include "Templates/SharedPointer.h"

#define UE_API PIXELSTREAMING2CORE_API

class UTexture2D;
namespace UE::PixelStreaming2
{
	class FPixelStreaming2Module;
} // namespace UE::PixelStreaming2

/**
 * The IPixelStreaming2Streamer interface provides functionality for managing a Pixel Streaming session.
 * This class manages the core operations of streaming, such as setting the framerate, handling video input,
 * and interacting with the signalling server. It also allows for control over connected players and sending data to them.
 */
class IPixelStreaming2Streamer
{
public:
	virtual ~IPixelStreaming2Streamer() = default;

	/**
	 * Called just after streamer creation. Use this method to initialize any methods that need
	 * a shared pointer to the streamer by using AsShared() inside the method.
	 */
	virtual void Initialize() = 0;

	/**
	 * @brief Set the Stream FPS.
	 * @param InFramesPerSecond The number of frames per second the streamer will stream at.
	 */
	virtual void SetStreamFPS(int32 InFramesPerSecond) = 0;

	/**
	 * @brief Get the Stream FPS.
	 * @return - The number of frames per second the streamer will stream at.
	 */
	virtual int32 GetStreamFPS() = 0;

	/**
	 * Setting this to true will cause the streamer to ignore the FPS value and instead push out frames
	 * as they are submitted from the video input. If the encode process takes longer than the time
	 * between frames, frames will be dropped.
	 * @param bCouple true to couple the streamer framerate to that of the video input.
	 */
	virtual void SetCoupleFramerate(bool bCouple) = 0;

	/**
	 * @brief Set the Video Input object.
	 * @param Input The IPixelStreaming2VideoProducer that this streamer will stream.
	 */
	virtual void SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> Input) = 0;

	/**
	 * @brief Get the Video Input object.
	 * @return The IPixelStreaming2VideoProducer that this streamer will stream.
	 */
	virtual TWeakPtr<IPixelStreaming2VideoProducer> GetVideoProducer() = 0;

	/**
	 * @brief Add an audio producer to the streamer.
	 * @param AudioProducer An IPixelStreaming2AudioProducer that this streamer will mix with engine audio before streaming.
	 */
	virtual void AddAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) = 0;

	/**
	 * @brief Remove an audio producer from the streamer.
	 * @param AudioProducer The IPixelStreaming2AudioProducer to remove.
	 */
	virtual void RemoveAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) = 0;

	/**
	 * @brief Get the audio producers currently added to this streamer.
	 * @return An array of weak pointers to the IPixelStreaming2AudioProducers currently added to this streamer.
	 */
	virtual TArray<TWeakPtr<IPixelStreaming2AudioProducer>> GetAudioProducers() = 0;

	/**
	 * @brief Set the URL this streamer will connect to.
	 * @param InConnectionURL.
	 */
	virtual void SetConnectionURL(const FString& InConnectionURL) = 0;

	/**
	 * @brief Set the Signalling Server URL.
	 * @param InSignallingServerURL.
	 */
	UE_DEPRECATED(5.6, "SetSignallingServerURL has been deprecated. Please use SetConnectionURL instead.")
	virtual void SetSignallingServerURL(const FString& InSignallingServerURL) { SetConnectionURL(InSignallingServerURL); }

	/**
	 * @brief Get the URL this streamer will connect to.
	 * @return The connection URL.
	 */
	virtual FString GetConnectionURL() = 0;

	/**
	 * @brief Get the Signalling Server URL.
	 * @return The Signalling Server URL.
	 */
	UE_DEPRECATED(5.6, "GetSignallingServerURL has been deprecated. Please use GetConnectionURL instead.")
	virtual FString GetSignallingServerURL() { return GetConnectionURL(); }

	/**
	 * @brief Get this streamer's ID.
	 * @return The streamer's ID.
	 */
	virtual FString GetId() = 0;

	/**
	 * @brief Check if this streamer is currently connected to the connection URL.
	 * @return True if the streamer is connected.
	 */
	virtual bool IsConnected() = 0;

	/**
	 * @brief Check if this streamer is currently connected to the signalling mechanism (e.g. websocket for signalling server).
	 * @return True if the streamer is connected to the signalling mechanism.
	 */
	UE_DEPRECATED(5.6, "IsSignallingConnected has been deprecated. Please use IsConnected instead.")
	virtual bool IsSignallingConnected() { return IsConnected(); }

	/**
	 * @brief Start streaming this streamer
	 */
	virtual void StartStreaming() = 0;

	/**
	 * @brief Stop this streamer from streaming
	 */
	virtual void StopStreaming() = 0;

	/**
	 * @brief Get the current state of this streamer
	 * @return True if streaming, false otherwise.
	 */
	virtual bool IsStreaming() const = 0;

	/**
	 * @brief Event fired just before the streamer begins connecting to signalling.
	 */
	DECLARE_EVENT_OneParam(IPixelStreaming2Streamer, FPreConnectionEvent, IPixelStreaming2Streamer*);

	/**
	 * @brief A getter for the OnPreConnection event. Intent is for users to call IPixelStreaming2Module::Get().FindStreamer(ID)->OnPreConnection().AddXXX.
	 * @return - The bindable OnPreConnection event.
	 */
	virtual FPreConnectionEvent& OnPreConnection() = 0;

	/**
	 * @brief Event fired when the streamer has connected to a signalling server and is ready for peers.
	 */
	DECLARE_EVENT_OneParam(IPixelStreaming2Streamer, FStreamingStartedEvent, IPixelStreaming2Streamer*);

	/**
	 * @brief A getter for the OnStreamingStarted event. Intent is for users to call IPixelStreaming2Module::Get().FindStreamer(ID)->OnStreamingStarted().AddXXX.
	 * @return - The bindable OnStreamingStarted event.
	 */
	virtual FStreamingStartedEvent& OnStreamingStarted() = 0;

	/**
	 * @brief Event fired when the streamer has disconnected from a signalling server and has stopped streaming.
	 */
	DECLARE_EVENT_OneParam(IPixelStreaming2Streamer, FStreamingStoppedEvent, IPixelStreaming2Streamer*);

	/**
	 * @brief A getter for the OnStreamingStopped event. Intent is for users to call IPixelStreaming2Module::Get().FindStreamer(ID)->OnStreamingStopped().AddXXX.
	 * @return - The bindable OnStreamingStopped event.
	 */
	virtual FStreamingStoppedEvent& OnStreamingStopped() = 0;

	/**
	 * @brief Force a key frame to be sent.
	 */
	virtual void ForceKeyFrame() = 0;

	/**
	 * @brief Freeze Pixel Streaming.
	 * @param Texture 		- The freeze frame to display. If null then the back buffer is captured.
	 */
	virtual void FreezeStream(UTexture2D* Texture) = 0;

	/**
	 * @brief Unfreeze Pixel Streaming.
	 */
	virtual void UnfreezeStream() = 0;

	/**
	 * @brief Send all players connected to this streamer a message.
	 * @param MessageType The message type to be sent to the player.
	 * @param Descriptor The contents of the message.
	 */
	virtual void SendAllPlayersMessage(FString MessageType, const FString& Descriptor) = 0;

	/**
	 * @brief Send all players connected to this streamer a message.
	 * @param MessageType The message type to be sent to the player.
	 * @param Descriptor The contents of the message.
	 */
	virtual void SendPlayerMessage(FString PlayerId, FString MessageType, const FString& Descriptor) = 0;

	/**
	 * @brief Send a file to the browser where we are sending video.
	 * @param ByteData	 	- The raw byte data of the file.
	 * @param MimeType	 	- The files Mime type. Used for reconstruction on the front end.
	 * @param FileExtension - The files extension. Used for reconstruction on the front end.
	 */
	virtual void SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension) = 0;

	/**
	 * @brief Kick a player by player id.
	 * @param PlayerId		- The ID of the player to kick
	 */
	virtual void KickPlayer(FString PlayerId) = 0;

	/**
	 * @brief Get the ids of the connected players.
	 * @return TArray<FString> The ids of the connected players.
	 */
	virtual TArray<FString> GetConnectedPlayers() = 0;

	/**
	 * @brief Get the streamer's input handler
	 * @return The streamer's input handler
	 */
	virtual TWeakPtr<class IPixelStreaming2InputHandler> GetInputHandler() = 0;

	/**
	 * @brief Set the streamer's input handler
	 * @param Handler The streamer's input handler
	 */
	virtual void SetInputHandler(TSharedPtr<class IPixelStreaming2InputHandler> Handler) = 0;

	/**
	 * @brief Get the audio sink associated with a specific peer/player.
	 * @param PlayerId The player id of the audio sink to retrieve.
	 * @return The PeerAudioSink.
	 */
	virtual TWeakPtr<IPixelStreaming2AudioSink> GetPeerAudioSink(FString PlayerId) = 0;

	/**
	 * @brief Get an audio sink that has no peers/players listening to it.
	 * @return The unlistened PeerAudioSink.
	 */
	virtual TWeakPtr<IPixelStreaming2AudioSink> GetUnlistenedAudioSink() = 0;

	/**
	 * @brief Get the video sink associated with a specific peer/player.
	 * @return The PeerVideoSink.
	 */
	virtual TWeakPtr<IPixelStreaming2VideoSink> GetPeerVideoSink(FString PlayerId) = 0;

	/**
	 * @brief Get a video sink that has no peers/players watching it.
	 * @return The unwatched PeerVideoSink.
	 */
	virtual TWeakPtr<IPixelStreaming2VideoSink> GetUnwatchedVideoSink() = 0;

	/**
	 * @brief Allows sending arbitrary configuration options during initial connection.
	 * @param OptionName The name of the option to set.
	 * @param Value Setting a value to an empty string clears it from the mapping and prevents it being sent.
	 */
	virtual void SetConfigOption(const FName& OptionName, const FString& Value) = 0;

	/**
	 * @brief Get the configuration value for a specific option.
	 * @param OptionName The name of the config option to get.
	 * @param Value The value of the config option retrieved.
	 */
	virtual bool GetConfigOption(const FName& OptionName, FString& OutValue) = 0;

	/**
	 * @brief Set the minimum and maximum bitrate for the streamer.
	 * @param PlayerId Currently unused. For setting the bitrate for a individual player id.
	 * @param MinBitrate minimum bitrate for the streamer.
	 * @param MaxBitrate maximum bitrate for the streamer.
	 */
	virtual void PlayerRequestsBitrate(FString PlayerId, int MinBitrate, int MaxBitrate) = 0;

	/**
	 * @brief Refresh connection with minimum and maximum bitrate.
	 */
	virtual void RefreshStreamBitrate() = 0;
};

/**
 * The IPixelStreaming2StreamerFactory interface provides functionality for creating custom streamers based on an identifier.
 *
 */
class IPixelStreaming2StreamerFactory : public IModularFeature
{
public:
	UE_API IPixelStreaming2StreamerFactory();
	UE_API virtual ~IPixelStreaming2StreamerFactory();

	virtual FString GetStreamType() = 0;

	static UE_API void RegisterStreamerFactory(IPixelStreaming2StreamerFactory* InFactory);

	static UE_API void UnregisterStreamerFactory(IPixelStreaming2StreamerFactory* InFactory);

	static UE_API IPixelStreaming2StreamerFactory* Get(const FString& InType);

	static UE_API TArray<FString> GetAvailableFactoryTypes();

private:
	// Only the module should call CreateNewStreamer on the factories.
	friend UE::PixelStreaming2::FPixelStreaming2Module;
	virtual TSharedPtr<IPixelStreaming2Streamer> CreateNewStreamer(const FString& StreamerId) = 0;
};

#undef UE_API
