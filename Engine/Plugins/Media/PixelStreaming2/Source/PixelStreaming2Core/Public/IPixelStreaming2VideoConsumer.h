// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "RHIFwd.h"

#include "IPixelStreaming2VideoConsumer.generated.h"

UINTERFACE(MinimalAPI)
class UPixelStreaming2VideoConsumer : public UInterface
{
	GENERATED_BODY()
};

/**
 * An "Video Consumer" is an object that is responsible for outputting the video received from a peer. For example, by
 * rendering to a render target.
 */
class IPixelStreaming2VideoConsumer
{
	GENERATED_BODY()

public:
	/**
	 * @brief Consume a texture as a video frame.
	 * @param Frame The Frame to consume.
	 */
	virtual void ConsumeFrame(FTextureRHIRef Frame) = 0;

	/**
	 * @brief Called when a video consumer is added.
	 */
	virtual void OnVideoConsumerAdded() = 0;

	UE_DEPRECATED(5.7, "OnConsumerAdded has been deprecated. Please use OnVideoConsumerAdded instead.")
	virtual void OnConsumerAdded() {}

	/**
	 * @brief Called when a video consumer is removed.
	 */
	virtual void OnVideoConsumerRemoved() = 0;

	UE_DEPRECATED(5.7, "OnConsumerRemoved has been deprecated. Please use OnVideoConsumerRemoved instead.")
	virtual void OnConsumerRemoved() {}
};
