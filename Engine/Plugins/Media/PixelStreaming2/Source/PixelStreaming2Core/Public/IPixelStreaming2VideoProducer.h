// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelCaptureInputFrame.h"
#include "UObject/Interface.h"

#include "IPixelStreaming2VideoProducer.generated.h"

UINTERFACE(MinimalAPI)
class UPixelStreaming2VideoProducer : public UInterface
{
	GENERATED_BODY()
};

enum class EVideoProducerCapabilities : uint8
{
	Default = 0,
	ProducesPreprocessedFrames = 1 << 0, // Frames pushed into this producer are preprocessed (e.g. resized, format converted) and ready for encoding.
};
ENUM_CLASS_FLAGS(EVideoProducerCapabilities);

/**
 * A "Video Producer" is an object that you use to push video frames into the Pixel Streaming system.
 *
 * Example usage:
 *
 * (1) Each new frame you want to push you call `MyVideoProducer->PushFrame(MyFrame)`
 *
 * Note: Your frame is likely a `FPixelCaptureInputFrameRHI` if the frame is a GPU texture or
 * a `FPixelCaptureInputFrameI420` or `FPixelCaptureInputFrameNV12` if your frame is from the CPU.
 */
class IPixelStreaming2VideoProducer
{
	GENERATED_BODY()

public:
	/**
	 * Pushes a raw video frame into the Pixel Streaming system.
	 * @param InputFrame The raw input frame, which may be a GPU texture or a CPU texture based on the underlying type of frame pushed.
	 */
	virtual void PushFrame(const IPixelCaptureInputFrame& InputFrame) final
	{
		OnFramePushed.Broadcast(InputFrame);
	}

	/**
	 * Event triggered when a frame is pushed to the video producer. You can use this to hook into the frame stream.
	 * Currently, streamers will listen to this event to know when a new frame is available for encoding.
	 */
	DECLARE_EVENT_OneParam(IPixelStreaming2VideoProducer, FOnFramePushed, const IPixelCaptureInputFrame&);
	FOnFramePushed OnFramePushed;

	/**
	 * Returns the capabilities of the video producer.
	 * @return The capabilities of the video producer.
	 */
	virtual EVideoProducerCapabilities GetCapabilities() = 0;

    /**
	 * A human readable identifier used when displaying what the streamer is streaming in the toolbar
	 * @return A string containing the display name.
	 */
	virtual FString ToString() = 0;
};
