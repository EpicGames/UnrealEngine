// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2VideoConsumer.h"
#include "Templates/PointerVariants.h"
#include "UObject/Interface.h"

#include "IPixelStreaming2VideoSink.generated.h"

UINTERFACE(MinimalAPI)
class UPixelStreaming2VideoSink : public UInterface
{
	GENERATED_BODY()
};

class IPixelStreaming2VideoConsumer;

/**
 * Interface for a sink that collects video coming in from the browser and passes into into UE.
 */
class IPixelStreaming2VideoSink
{
	GENERATED_BODY()

public:
	/**
	 * @brief Add a video consumer to the sink.
	 * @param VideoConsumer The video consumer to add to the sink.
	 */
	UE_DEPRECATED(5.6, "AddVideoConsumer(IPixelStreaming2VideoConsumer*) has been deprecated. Please use AddVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>&) instead.")
	virtual void AddVideoConsumer(IPixelStreaming2VideoConsumer* VideoConsumer) {}

	/**
	 * @brief Add a video consumer to the sink.
	 * @param VideoConsumer The video consumer to add to the sink.
	 */
	virtual void AddVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>& VideoConsumer) = 0;

	/**
	 * @brief Remove a video consumer to remove from the sink.
	 * @param VideoConsumer The video consumer to remove from the sink.
	 */
	UE_DEPRECATED(5.6, "RemoveVideoConsumer(IPixelStreaming2VideoConsumer*) has been deprecated. Please use RemoveVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>&) instead.")
	virtual void RemoveVideoConsumer(IPixelStreaming2VideoConsumer* VideoConsumer) {}

	/**
	 * @brief Remove a video consumer to remove from the sink.
	 * @param VideoConsumer The video consumer to remove from the sink.
	 */
	virtual void RemoveVideoConsumer(const TWeakPtrVariant<IPixelStreaming2VideoConsumer>& VideoConsumer) = 0;
};
