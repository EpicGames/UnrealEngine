// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/SlateStyle.h"

#define UE_API PIXELSTREAMING2EDITOR_API

namespace UE::EditorPixelStreaming2
{
    class FPixelStreaming2Style
    {
    public:
        static UE_API void Initialize();
        static UE_API void Shutdown();
        /** reloads textures used by slate renderer */
        static UE_API void ReloadTextures();
        /** @return The Slate style set for the Shooter game */
        static UE_API FSlateStyleSet& Get();
        static UE_API FName GetStyleSetName();
   
    private:
        static UE_API TSharedRef< class FSlateStyleSet > Create();
   
    private:
        static UE_API TSharedPtr< class FSlateStyleSet > StyleInstance;
    };
}

#undef UE_API
