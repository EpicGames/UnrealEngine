// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoProducer.h"

#include "Widgets/SWindow.h"
#include "RendererInterface.h"
#include "Slate/SlateViewportProvider.h"
#include "Delegates/IDelegateInstance.h"

#define UE_API PIXELSTREAMING2EDITOR_API

namespace UE::EditorPixelStreaming2
{
	using namespace UE::PixelStreaming2;

	/**
	 * Use this if you want to send the full UE editor as video input.
	 */
	class FVideoProducerBackBufferComposited : public FVideoProducerBase
	{
	public:
		static UE_API TSharedPtr<FVideoProducerBackBufferComposited> Create();
		UE_API virtual ~FVideoProducerBackBufferComposited();

		UE_API virtual FString ToString() override;

#if PLATFORM_APPLE
		virtual EVideoProducerCapabilities GetCapabilities() { return EVideoProducerCapabilities::Default; }
#else
		virtual EVideoProducerCapabilities GetCapabilities() { return EVideoProducerCapabilities::ProducesPreprocessedFrames; }
#endif

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnFrameSizeChanged, TWeakPtr<FIntRect>);
		FOnFrameSizeChanged OnFrameSizeChanged;

	private:
		// Our class to keep window and texture information. Use of this struct prevents SWindow properties being updated composition as well
		// as preventing deletion of our staging textures pre composition
		class FTexturedWindow
		{
		public:
			FTexturedWindow(FVector2D InPositionInScreen, FVector2D InSizeInScreen, float InOpacity, EWindowType InType, SWindow* InOwningWindow)
				: PositionInScreen(InPositionInScreen), SizeInScreen(InSizeInScreen), Opacity(InOpacity), Type(InType), Texture(nullptr), OwningWindow(InOwningWindow)
			{
			}

			FVector2D						   GetPositionInScreen() { return PositionInScreen; }
			FVector2D						   GetSizeInScreen() { return SizeInScreen; }
			float							   GetOpacity() { return Opacity; }
			EWindowType						   GetType() { return Type; }
			SWindow*						   GetOwningWindow() { return OwningWindow; }
			TRefCountPtr<IPooledRenderTarget>& GetTexture() { return Texture; }
			void							   SetTexture(TRefCountPtr<IPooledRenderTarget> InTexture) { Texture = InTexture; }

		private:
			FVector2D						  PositionInScreen;
			FVector2D						  SizeInScreen;
			float							  Opacity;
			EWindowType						  Type;
			TRefCountPtr<IPooledRenderTarget> Texture;
			SWindow*						  OwningWindow;
		};

	private:
		UE_API FVideoProducerBackBufferComposited();
		UE_API void CompositeWindows(TSharedPtr<FVideoProducerUserData> UserData);

		UE_API void OnBackBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);
		UE_API void OnPreTick(float DeltaTime);

		FDelegateHandle OnBackBufferReadyToPresentHandle;
		FDelegateHandle OnPreTickHandle;

		TArray<FTexturedWindow> TopLevelWindows;

		FCriticalSection TopLevelWindowsCriticalSection;

		TSharedPtr<FIntRect> SharedFrameRect;

	private:
		// Util functions for 2D vectors
		template <class T>
		T VectorMax(const T A, const T B)
		{
			// Returns the component-wise maximum of two vectors
			return T(FMath::Max(A.X, B.X), FMath::Max(A.Y, B.Y));
		}

		template <class T>
		T VectorMin(const T A, const T B)
		{
			// Returns the component-wise minimum of two vectors
			return T(FMath::Min(A.X, B.X), FMath::Min(A.Y, B.Y));
		}
	};

} // namespace UE::EditorPixelStreaming2

#undef UE_API
