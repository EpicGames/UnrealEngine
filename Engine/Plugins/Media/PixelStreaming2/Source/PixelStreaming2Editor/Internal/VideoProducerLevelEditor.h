// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2Streamer.h"
#include "VideoProducerViewportBase.h"
#include "Widgets/SWindow.h"

#define UE_API PIXELSTREAMING2EDITOR_API

class FViewport;

namespace UE::EditorPixelStreaming2
{
	using namespace UE::PixelStreaming2;

	/**
	 * Use this if you want to send the UE primary scene viewport as video input - will only work in editor.
	 */
	class FVideoProducerLevelEditor : public FVideoProducerViewportBase
	{
	public:
		static UE_API TSharedPtr<FVideoProducerLevelEditor> Create();
		virtual ~FVideoProducerLevelEditor() = default;

		UE_API virtual FString ToString() override;

	protected:
		UE_API virtual bool ShouldCaptureViewport() override;

	private:
		FVideoProducerLevelEditor() = default;

		UE_API void OnPreTick(float DeltaTime);
	};

} // namespace UE::EditorPixelStreaming2

#undef UE_API
