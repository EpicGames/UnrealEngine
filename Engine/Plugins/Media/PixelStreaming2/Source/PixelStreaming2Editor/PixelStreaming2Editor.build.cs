// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class PixelStreaming2Editor : ModuleRules
	{
		public PixelStreaming2Editor(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"Engine",
				"Projects",
				"RenderCore",
				"Renderer",
				"RHI",
				"PixelStreaming2RTC",
				"PixelStreaming2Settings",
				"Slate",
				"EngineSettings",
				"InputCore",
				"Json",
				"PixelCapture",
				"PixelStreaming2Servers",
				"HTTP",
				"Sockets",
				"ApplicationCore",
				"PixelStreaming2Input",
				"AVCodecsCore",
				"PixelStreaming2",
				"PixelStreaming2Core"
			});

			PublicDependencyModuleNames.AddRange(new string[] 
			{
				"SlateCore"
			});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
				{
					"UnrealEd",
					"ToolMenus",
					"EditorStyle",
					"DesktopPlatform",
					"LevelEditor",
					"MainFrame"
				});
			}

			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
			}
		}
	}
}
