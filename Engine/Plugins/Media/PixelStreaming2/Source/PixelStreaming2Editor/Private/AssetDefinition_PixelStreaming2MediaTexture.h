// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "Blueprints/PixelStreaming2MediaTexture.h"
#include "AssetDefinition_PixelStreaming2MediaTexture.generated.h"

UCLASS()
class UAssetDefinition_PixelStreaming2MediaTexture : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_PixelStreaming2MediaTexture", "Pixel Streaming 2 Media Texture"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPixelStreaming2MediaTexture::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor(192, 64, 64); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "AssetDefinition_PixelStreaming2MediaTextureSubMenu", "Other"), ECategoryMenuType::Section)
		};
		return Categories;
	}
	// UAssetDefinition End
};
