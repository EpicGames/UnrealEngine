// Copyright Epic Games, Inc. All Rights Reserved.


#include "AssetDefinition_VideoProducer.h"

#include "Blueprints/PixelStreaming2VideoProducer.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_VideoProducer"

FText UAssetDefinition_VideoProducer::GetAssetDisplayName() const
{
	return LOCTEXT("VideoProducer_AssetName", "Video Input");
}

TSoftClassPtr<UObject> UAssetDefinition_VideoProducer::GetAssetClass() const
{
	return UPixelStreaming2VideoProducerBase::StaticClass();
}

FLinearColor UAssetDefinition_VideoProducer::GetAssetColor() const
{
	return FColor(192,64,64);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_VideoProducer::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Misc, LOCTEXT("VideoProduce_CategorySection", "Pixel Streaming 2"), ECategoryMenuType::Section) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
