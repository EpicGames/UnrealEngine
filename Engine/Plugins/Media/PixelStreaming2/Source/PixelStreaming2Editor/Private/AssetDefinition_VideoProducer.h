// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_VideoProducer.generated.h"

UCLASS()
class UAssetDefinition_VideoProducer : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual void BuildFilters(TArray<FAssetFilterData>& OutFilters) const override { }
	// UAssetDefinition End
};
