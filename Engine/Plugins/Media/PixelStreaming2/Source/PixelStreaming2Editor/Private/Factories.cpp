// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories.h"

#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "Blueprints/PixelStreaming2MediaTexture.h"
#include "Blueprints/PixelStreaming2VideoProducer.h"
#include "IAssetTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Factories)

#define LOCTEXT_NAMESPACE "PixelStreaming2"

/**
 * ---------- UPixelStreaming2MediaTextureFactory -------------------
 */
UPixelStreaming2MediaTextureFactory::UPixelStreaming2MediaTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;

	SupportedClass = UPixelStreaming2MediaTexture::StaticClass();
}

FText UPixelStreaming2MediaTextureFactory::GetDisplayName() const
{
	return LOCTEXT("MediaTextureFactoryDisplayName", "Pixel Streaming 2 Media Texture");
}

UObject* UPixelStreaming2MediaTextureFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreaming2MediaTexture* Resource = NewObject<UPixelStreaming2MediaTexture>(InParent, InName, Flags | RF_Transactional))
	{
		Resource->UpdateResource();
		return Resource;
	}

	return nullptr;
}

/**
 * ---------- UPixelStreaming2VideoProducerBackBufferFactory -------------------
 */
UPixelStreaming2VideoProducerBackBufferFactory::UPixelStreaming2VideoProducerBackBufferFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPixelStreaming2VideoProducerBackBuffer::StaticClass();
}

FText UPixelStreaming2VideoProducerBackBufferFactory::GetDisplayName() const
{
	return LOCTEXT("VideoProducerBackBufferDisplayName", "Back Buffer Video Input");
}

uint32 UPixelStreaming2VideoProducerBackBufferFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

UObject* UPixelStreaming2VideoProducerBackBufferFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreaming2VideoProducerBackBuffer* Resource = NewObject<UPixelStreaming2VideoProducerBackBuffer>(InParent, InName, Flags | RF_Transactional))
	{
		return Resource;
	}

	return nullptr;
}

/**
 * ---------- UPixelStreaming2VideoProducerMediaCaptureFactory -------------------
 */
UPixelStreaming2VideoProducerMediaCaptureFactory::UPixelStreaming2VideoProducerMediaCaptureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPixelStreaming2VideoProducerMediaCapture::StaticClass();
}

FText UPixelStreaming2VideoProducerMediaCaptureFactory::GetDisplayName() const
{
	return LOCTEXT("VideoProducerMediaCaptureDisplayName", "Media Capture Video Input");
}

uint32 UPixelStreaming2VideoProducerMediaCaptureFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

UObject* UPixelStreaming2VideoProducerMediaCaptureFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreaming2VideoProducerMediaCapture* Resource = NewObject<UPixelStreaming2VideoProducerMediaCapture>(InParent, InName, Flags | RF_Transactional))
	{
		return Resource;
	}

	return nullptr;
}

/**
 * ---------- UPixelStreaming2VideoProducerRenderTargetFactory -------------------
 */
UPixelStreaming2VideoProducerRenderTargetFactory::UPixelStreaming2VideoProducerRenderTargetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;

	SupportedClass = UPixelStreaming2VideoProducerRenderTarget::StaticClass();
}

FText UPixelStreaming2VideoProducerRenderTargetFactory::GetDisplayName() const
{
	return LOCTEXT("VideoProducerRenderTargetDisplayName", "Render Target Video Input");
}

uint32 UPixelStreaming2VideoProducerRenderTargetFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

UObject* UPixelStreaming2VideoProducerRenderTargetFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (UPixelStreaming2VideoProducerRenderTarget* Resource = NewObject<UPixelStreaming2VideoProducerRenderTarget>(InParent, InName, Flags | RF_Transactional))
	{
		return Resource;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
