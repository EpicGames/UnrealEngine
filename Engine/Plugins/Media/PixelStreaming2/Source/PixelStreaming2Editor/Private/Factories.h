// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "Factories.generated.h"

/**
 * ---------- UPixelStreaming2MediaTextureFactory -------------------
 */
UCLASS()
class UPixelStreaming2MediaTextureFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual FText	 GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

/**
 * ---------- UPixelStreaming2VideoProducerBackBufferFactory -------------------
 */
UCLASS()
class UPixelStreaming2VideoProducerBackBufferFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual FText	 GetDisplayName() const override;
	virtual uint32	 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

/**
 * ---------- UPixelStreaming2VideoProducerMediaCaptureFactory -------------------
 */
UCLASS()
class UPixelStreaming2VideoProducerMediaCaptureFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual FText	 GetDisplayName() const override;
	virtual uint32	 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

/**
 * ---------- UPixelStreaming2VideoProducerRenderTargetFactory -------------------
 */
UCLASS()
class UPixelStreaming2VideoProducerRenderTargetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual FText	 GetDisplayName() const override;
	virtual uint32	 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};