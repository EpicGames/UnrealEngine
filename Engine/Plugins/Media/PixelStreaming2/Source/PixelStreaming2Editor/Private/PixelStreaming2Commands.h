// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

/**
 * These are the commands used in the toolbar visible in the editor
 *
 */
namespace UE::EditorPixelStreaming2
{
	class FPixelStreaming2Commands : public TCommands<FPixelStreaming2Commands>
	{
	public:
		FPixelStreaming2Commands()
			: TCommands<FPixelStreaming2Commands>(TEXT("PixelStreaming2"), NSLOCTEXT("Contexts", "PixelStreaming2", "PixelStreaming2 Plugin"), NAME_None, FName(TEXT("PixelStreaming2Style")))
		{
		}

		virtual void RegisterCommands() override;

		TSharedPtr<FUICommandInfo> ExternalSignalling;
		TSharedPtr<FUICommandInfo> ServeHttps;
		TSharedPtr<FUICommandInfo> VP8;
		TSharedPtr<FUICommandInfo> VP9;
		TSharedPtr<FUICommandInfo> H264;
		TSharedPtr<FUICommandInfo> AV1;
		TSharedPtr<FUICommandInfo> StartSignalling;
		TSharedPtr<FUICommandInfo> StopSignalling;
		TSharedPtr<FUICommandInfo> StreamLevelEditor;
		TSharedPtr<FUICommandInfo> StreamEditor;
	};
} // namespace UE::EditorPixelStreaming2