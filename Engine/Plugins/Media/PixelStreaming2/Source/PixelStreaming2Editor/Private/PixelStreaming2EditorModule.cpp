// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2EditorModule.h"

#include "PixelStreaming2Style.h"
#include "PixelStreaming2Toolbar.h"
#include "Engine/GameViewportClient.h"
#include "UtilsCore.h"
#include "Editor/EditorPerformanceSettings.h"
#include "VideoProducerBackBufferComposited.h"
#include "VideoProducerPIEViewport.h"
#include "VideoProducerLevelEditor.h"
#include "PixelStreaming2PluginSettings.h"
#include "UnrealEngine.h"
#include "Engine/Engine.h"
#include "Interfaces/IMainFrameModule.h"
#include "PixelStreaming2Utils.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "LevelEditorViewport.h"
#include "Slate/SceneViewport.h"
#include "SLevelViewport.h"
#include "IPixelStreaming2InputHandler.h"
#include "PixelStreaming2Delegates.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Logging.h"
#include "IPixelStreaming2Module.h"
#include "UtilsAsync.h"
#include "LevelEditor.h"

#define LOCTEXT_NAMESPACE "PixelStreaming2EditorModule"

#define IMAGE_BRUSH_SVG(Style, RelativePath, ...) FSlateVectorImageBrush(Style.RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

namespace
{
	void ResizeEditor(int Width, int Height)
	{
		TSharedPtr<SWindow> ParentWindow = IMainFrameModule::Get().GetParentWindow();
		ParentWindow->Resize(FVector2D(Width, Height));
		FSlateApplication::Get().OnSizeChanged(ParentWindow->GetNativeWindow().ToSharedRef(), Width, Height);
		// Triggers the NullApplication to rebuild its DisplayMetrics with the new resolution and inform slate
		// about the updated virtual desktop size
		FSystemResolution::RequestResolutionChange(Width, Height, GSystemResolution.WindowMode);
		IConsoleManager::Get().CallAllConsoleVariableSinks();
	}
	void ResizeViewport(int Width, int Height)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedPtr<SLevelViewport>  Viewport = LevelEditorModule.GetFirstActiveLevelViewport();
		const FLevelEditorViewportClient* ViewportClient = Viewport ? &Viewport->GetLevelViewportClient() : nullptr;
		const TSharedPtr<SEditorViewport> ViewportWidget = ViewportClient ? ViewportClient->GetEditorViewportWidget() : nullptr;
		const TSharedPtr<FSceneViewport>  SceneViewport = ViewportClient ? ViewportWidget->GetSceneViewport() : nullptr;
		if (SceneViewport)
		{
			SceneViewport->SetFixedViewportSize(Width, Height);
		}
	}

}	// namespace

namespace UE::EditorPixelStreaming2
{
	/**
	 * IModuleInterface implementation
	 */
	void FPixelStreaming2EditorModule::StartupModule()
	{
		if (!UE::PixelStreaming2::IsStreamingSupported())
		{
			return;
		}


		// Initialize the editor toolbar
		FPixelStreaming2Style::Initialize();
		FSlateStyleSet& StyleInstance = UE::EditorPixelStreaming2::FPixelStreaming2Style::Get();

		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		StyleInstance.Set("ClassThumbnail.PixelStreaming2VideoProducerBackBuffer", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming2_64", Icon64x64));
		StyleInstance.Set("ClassIcon.PixelStreaming2VideoProducerBackBuffer", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming2_20", Icon20x20));
		StyleInstance.Set("ClassThumbnail.PixelStreaming2VideoProducerRenderTarget", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming2_64", Icon64x64));
		StyleInstance.Set("ClassIcon.PixelStreaming2VideoProducerRenderTarget", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming2_20", Icon20x20));
		StyleInstance.Set("ClassThumbnail.PixelStreaming2VideoProducerMediaCapture", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming2_64", Icon64x64));
		StyleInstance.Set("ClassIcon.PixelStreaming2VideoProducerMediaCapture", new IMAGE_BRUSH_SVG(StyleInstance, "PixelStreaming2_20", Icon20x20));

		FPixelStreaming2Style::ReloadTextures();

		Toolbar = MakeShared<FPixelStreaming2Toolbar>();

		if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
		{
			Delegates->OnFallbackToSoftwareEncodingNative.AddLambda([]() {
				// Creates a new notification info, we pass in our text as the parameter.
				FNotificationInfo Info(LOCTEXT("PixelStreaming2EditorModule_Notification", "Pixel Streaming: Unable to create hardware encoder, falling back to VP8 software encoding."));
				// Set a default expire duration
				Info.ExpireDuration = 5.0f;
				// And call Add Notification
				FSlateNotificationManager::Get().AddNotification(Info);
			});
		}

		// We register console commands for "Stat xxx" here as the autocomplete logic doesn't execute in the editor
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Stat PixelStreaming2"),
			TEXT("Stats for the Pixel Streaming plugin and its peers."),
			FConsoleCommandDelegate::CreateLambda([]() {
				for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
				{
					UWorld*				 World = WorldContext.World();
					UGameViewportClient* ViewportClient = World->GetGameViewport();
					GEngine->SetEngineStat(World, ViewportClient, TEXT("PixelStreaming2"), true);
				}
			}),
			ECVF_Default);

		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("Stat PixelStreaming2Graphs"),
			TEXT("Draws stats graphs for the Pixel Streaming plugin."),
			FConsoleCommandDelegate::CreateLambda([]() {
				for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
				{
					UWorld*				 World = WorldContext.World();
					UGameViewportClient* ViewportClient = World->GetGameViewport();
					GEngine->SetEngineStat(World, ViewportClient, TEXT("PixelStreaming2"), true);
				}
			}),
			ECVF_Default);

		IPixelStreaming2RTCModule& Module = IPixelStreaming2RTCModule::Get();
		Module.OnReady().AddRaw(this, &FPixelStreaming2EditorModule::InitEditorStreaming);
	}

	void FPixelStreaming2EditorModule::ShutdownModule()
	{
		if (!UE::PixelStreaming2::IsStreamingSupported())
		{
			return;
		}

		StopStreaming();
		StopSignalling(true);
	}

	void FPixelStreaming2EditorModule::InitEditorStreaming(IPixelStreaming2RTCModule& Module)
	{
		if (!FParse::Value(FCommandLine::Get(), TEXT("PixelStreamingID="), EditorStreamerID))
		{
			EditorStreamerID = "Editor";
		}

		EditorStreamer = IPixelStreaming2Module::Get().CreateStreamer(EditorStreamerID);

		// Bind to start/stop streaming so we disable/restore relevant editor settings
		EditorStreamer->OnStreamingStarted().AddLambda([this](IPixelStreaming2Streamer* Streamer) {
			DisableCPUThrottlingSetting();
		});
		EditorStreamer->OnStreamingStopped().AddLambda([this](IPixelStreaming2Streamer* Streamer) {
			if (!IsEngineExitRequested())
			{
				RestoreCPUThrottlingSetting();
			}
		});

		/**
		 * Called before the engine exits. Separate from OnPreExit as OnEnginePreExit occurs before shutting down any core modules.
		 */
		FCoreDelegates::OnEnginePreExit.AddLambda([this]() {
			RestoreCPUThrottlingSetting(true);
		});

		// Give the editor streamer the default url if the user hasn't specified one when launching the editor
		if (EditorStreamer->GetConnectionURL().IsEmpty())
		{
			// No URL was passed on the command line, initialize defaults
			StreamerPort = 8888;
			SignallingDomain = TEXT("ws://127.0.0.1");

			EditorStreamer->SetConnectionURL(FString::Printf(TEXT("%s:%d"), *SignallingDomain, StreamerPort));
		}
		else
		{
			FString			  SpecifiedSignallingURL = EditorStreamer->GetConnectionURL();
			TOptional<uint16> ExtractedStreamerPort = FGenericPlatformHttp::GetUrlPort(SpecifiedSignallingURL);
			StreamerPort = (int32)ExtractedStreamerPort.Get(8888);

			FString ExtractedSignallingDomain = FGenericPlatformHttp::GetUrlDomain(SpecifiedSignallingURL);
			if (FGenericPlatformHttp::IsSecureProtocol(SpecifiedSignallingURL).Get(false))
			{
				SignallingDomain = FString::Printf(TEXT("wss://%s"), *ExtractedSignallingDomain);
			}
			else
			{
				SignallingDomain = FString::Printf(TEXT("ws://%s"), *ExtractedSignallingDomain);
			}
		}

		EditorStreamer->SetConfigOption(FName(*FString(TEXT("DefaultToHover"))), TEXT("true"));

		IMainFrameModule::Get().OnMainFrameCreationFinished().AddLambda([&](TSharedPtr<SWindow> RootWindow, bool bIsRunningStartupDialog) {
			MaybeResizeEditor(RootWindow);

			if (UPixelStreaming2PluginSettings::CVarEditorStartOnLaunch.GetValueOnAnyThread())
			{
				EPixelStreaming2EditorStreamTypes Source = UE::PixelStreaming2::GetEnumFromCVar<EPixelStreaming2EditorStreamTypes>(UPixelStreaming2PluginSettings::CVarEditorSource);
				StartStreaming(Source);
			}
		});

#if WITH_EDITOR
		FEditorDelegates::PostPIEStarted.AddRaw(this, &FPixelStreaming2EditorModule::OnBeginPIE);
		FEditorDelegates::EndPIE.AddRaw(this, &FPixelStreaming2EditorModule::OnEndPIE);
#endif
	}

	void FPixelStreaming2EditorModule::StartStreaming(EPixelStreaming2EditorStreamTypes InStreamType)
	{
		// Activate our editor streamer
		if (!EditorStreamer.IsValid())
		{
			return;
		}

		TSharedPtr<IPixelStreaming2InputHandler> InputHandler = EditorStreamer->GetInputHandler().Pin();
		if (!InputHandler.IsValid())
		{
			return;
		}

		// Add custom handle for { type: "Command", Resolution.Width: "1920", Resolution.Height: "1080" } when doing Editor streaming
		// because we cannot resize the game viewport, but instead want to resize the parent window.
		InputHandler->SetCommandHandler("Resolution.Width",
			[InStreamType](FString, FString Descriptor, FString WidthString) {
				bool	bSuccess;
				FString HeightString;
				UE::PixelStreaming2::ExtractJsonFromDescriptor(Descriptor, TEXT("Resolution.Height"), HeightString, bSuccess);
				int Width = FCString::Atoi(*WidthString);
				int Height = FCString::Atoi(*HeightString);
				if (Width < 1 || Height < 1)
				{
					return;
				}

				if (InStreamType == EPixelStreaming2EditorStreamTypes::LevelEditorViewport)
				{
					ResizeViewport(Width, Height);
				}
				else
				{
					ResizeEditor(Width, Height);
				}
			});

		switch (InStreamType)
		{
			case EPixelStreaming2EditorStreamTypes::LevelEditorViewport:
			{
				FLevelEditorModule&		   LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
				TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
				if (!ActiveLevelViewport.IsValid())
				{
					return;
				}

				FLevelEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetLevelViewportClient();
				FSceneViewport*				SceneViewport = static_cast<FSceneViewport*>(LevelViewportClient.Viewport);
				InputHandler->SetTargetViewport(SceneViewport->GetViewportWidget());
				InputHandler->SetTargetWindow(SceneViewport->FindWindow());
				InputHandler->SetInputType(EPixelStreaming2InputType::RouteToWindow);
				EditorStreamer->SetVideoProducer(FVideoProducerLevelEditor::Create());
			}
			break;
			case EPixelStreaming2EditorStreamTypes::Editor:
			{
				InputHandler->SetTargetViewport(nullptr);
				InputHandler->SetTargetWindow(nullptr);
				InputHandler->SetInputType(EPixelStreaming2InputType::RouteToWindow);

				TSharedPtr<FVideoProducerBackBufferComposited> VideoProducer = FVideoProducerBackBufferComposited::Create();
				VideoProducer->OnFrameSizeChanged.AddSP(InputHandler.ToSharedRef(), &IPixelStreaming2InputHandler::SetTargetScreenRect);
				EditorStreamer->SetVideoProducer(VideoProducer);
			}
			break;
			default:
			{
				UE_LOGF(LogPixelStreaming2Editor, Warning, "Specified Stream Type doesn't have an associated FPixelStreaming2VideoProducer");
			}
				// Return here as we don't want to start streaming if we didn't set a viewport
				return;
		}

		if (!UPixelStreaming2PluginSettings::CVarEditorUseRemoteSignallingServer.GetValueOnAnyThread())
		{
			EditorStreamer->SetConnectionURL(FString::Printf(TEXT("%s:%d"), *SignallingDomain, StreamerPort));
			StartSignalling();
		}

		// If the level viewport has resized from the stream, this will reset it.
		EditorStreamer->OnStreamingStopped().AddLambda([InStreamType](IPixelStreaming2Streamer*) {
			if (InStreamType == EPixelStreaming2EditorStreamTypes::LevelEditorViewport)
			{
				DoOnGameThread([]() {
					ResizeViewport(0, 0);
				});
			}
		});

		EditorStreamer->StartStreaming();
	}

	void FPixelStreaming2EditorModule::StopStreaming()
	{
		if (EditorStreamer)
		{
			EditorStreamer->StopStreaming();
		}
		
		StopSignalling(false);
	}

	void FPixelStreaming2EditorModule::StartSignalling()
	{
		bool bAlreadyLaunched = SignallingServer.IsValid() && SignallingServer->HasLaunched();
		if (bAlreadyLaunched)
		{
			return;
		}

		// Download Pixel Streaming servers/frontend if we want to use a browser to view Pixel Streaming output
		// but only attempt this is we haven't already started a download before.
		if (!DownloadProcess.IsValid())
		{
			// We set bSkipIfPresent to false, which means the get_ps_servers script will always be run, that script will choose whether to download or not
			DownloadProcess = UE::PixelStreaming2Servers::DownloadPixelStreaming2Servers(false /* bSkipIfPresent */);
			if (DownloadProcess.IsValid())
			{
				DownloadProcess->OnCompleted().BindLambda([this](int ExitCode) {
					StopSignalling(true);
					StartSignalling();
				});

				return;
			}
		}

		// Launch signalling server
		SignallingServer = UE::PixelStreaming2Servers::MakeSignallingServer();

		UE::PixelStreaming2Servers::FLaunchArgs LaunchArgs;
		LaunchArgs.bPollUntilReady = false;
		LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
		LaunchArgs.ReconnectionIntervalSeconds = 2.0f;

		FString CertificateArgs = FString::Printf(TEXT("--CertificatePath=%s --PrivateKeyPath=%s"), *SSLCertificatePath, *SSLPrivateKeyPath);
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --StreamerPort=%d --ServeHttps=%s --EnableIPv6=true %s"), ViewerPort, StreamerPort, bServeHttps ? TEXT("true") : TEXT("false"), bServeHttps ? *CertificateArgs : TEXT(""));

		FString AllowedOrigins = UPixelStreaming2PluginSettings::CVarAllowedOrigins.GetValueOnAnyThread();
		if (!AllowedOrigins.IsEmpty())
		{
			LaunchArgs.ProcessArgs += FString::Printf(TEXT(" --AllowedOrigins=%s"), *AllowedOrigins);
		}

		SignallingServer->Launch(LaunchArgs);
	}

	void FPixelStreaming2EditorModule::StopSignalling(bool bForce)
	{
		if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
		{
			return;
		}

		// Force stop and reset the signalling server if desired
		if (bForce)
		{
			SignallingServer->Stop();
			SignallingServer.Reset();
			return;
		}

		// If we're not force stopping signalling, we delay 2 frames to make sure all disconnect messages have been received
		StopSignallingAfterFrames(2);
	}

	TSharedPtr<UE::PixelStreaming2Servers::IServer> FPixelStreaming2EditorModule::GetSignallingServer()
	{
		if (SignallingServer.IsValid())
		{
			return SignallingServer;
		}
		return nullptr;
	}

	void FPixelStreaming2EditorModule::SetSignallingDomain(const FString& InSignallingDomain)
	{
		SignallingDomain = InSignallingDomain;
	}

	void FPixelStreaming2EditorModule::SetStreamerPort(int32 InStreamerPort)
	{
		StreamerPort = InStreamerPort;
	}

	void FPixelStreaming2EditorModule::SetViewerPort(int32 InViewerPort)
	{
		ViewerPort = InViewerPort;
	}

	void FPixelStreaming2EditorModule::SetServeHttps(bool bInServeHttps)
	{
		bServeHttps = bInServeHttps;
	}

	void FPixelStreaming2EditorModule::SetSSLCertificatePath(const FString& Path)
	{
		SSLCertificatePath = Path;
	}

	void FPixelStreaming2EditorModule::SetSSLPrivateKeyPath(const FString& Path)
	{
		SSLPrivateKeyPath = Path;
	}

	bool FPixelStreaming2EditorModule::ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY)
	{
		if (*InResolution)
		{
			FString CmdString(InResolution);
			CmdString = CmdString.TrimStartAndEnd().ToLower();

			// Retrieve the X dimensional value
			const uint32 X = FMath::Max(FCString::Atof(*CmdString), 0.0f);

			// Determine whether the user has entered a resolution and extract the Y dimension.
			FString YString;

			// Find separator between values (Example of expected format: 1280x768)
			const TCHAR* YValue = nullptr;
			if (FCString::Strchr(*CmdString, 'x'))
			{
				YValue = FCString::Strchr(*CmdString, 'x') + 1;
				YString = YValue;
				// Remove any whitespace from the end of the string
				YString = YString.TrimStartAndEnd();
			}

			// If the Y dimensional value exists then setup to use the specified resolution.
			uint32 Y = 0;
			if (YValue && YString.Len() > 0)
			{
				if (YString.IsNumeric())
				{
					Y = FMath::Max(FCString::Atof(YValue), 0.0f);
					OutX = X;
					OutY = Y;
					return true;
				}
			}
		}
		return false;
	}

	void FPixelStreaming2EditorModule::MaybeResizeEditor(TSharedPtr<SWindow> RootWindow)
	{
		uint32	ResolutionX, ResolutionY = 0;
		FString ResolutionStr;
		bool	bSuccess = FParse::Value(FCommandLine::Get(), TEXT("EditorPixelStreamingRes="), ResolutionStr);
		if (bSuccess)
		{
			bSuccess = ParseResolution(*ResolutionStr, ResolutionX, ResolutionY);
		}
		else
		{
			bool UserSpecifiedWidth = FParse::Value(FCommandLine::Get(), TEXT("EditorPixelStreamingResX="), ResolutionX);
			bool UserSpecifiedHeight = FParse::Value(FCommandLine::Get(), TEXT("EditorPixelStreamingResY="), ResolutionY);
			bSuccess = UserSpecifiedWidth || UserSpecifiedHeight;

			const float AspectRatio = 16.0f / 9.0f;
			if (UserSpecifiedWidth && !UserSpecifiedHeight)
			{
				ResolutionY = int32(ResolutionX / AspectRatio);
			}
			else if (UserSpecifiedHeight && !UserSpecifiedWidth)
			{
				ResolutionX = int32(ResolutionY * AspectRatio);
			}
		}

		if (bSuccess)
		{
			// Update editor window size
			RootWindow->Resize(FVector2D(ResolutionX, ResolutionY));
			FSlateApplication::Get().OnSizeChanged(RootWindow->GetNativeWindow().ToSharedRef(), ResolutionX, ResolutionY);
			// Triggers the NullApplication to rebuild its DisplayMetrics with the new resolution and inform slate
			// about the updated virtual desktop size
			FSystemResolution::RequestResolutionChange(ResolutionX, ResolutionY, GSystemResolution.WindowMode);
			IConsoleManager::Get().CallAllConsoleVariableSinks();
		}
	}

	void FPixelStreaming2EditorModule::RestoreCPUThrottlingSetting(bool bForce)
	{
		// Test the set count, it can be zero because the restore callback fires on destruct
		// If bForce is set, this doesn't correspond to a particular call, but should only occur if a disable has occured
		if (CpuThrottlingSetCount && (bForce || (--CpuThrottlingSetCount == 0)))
		{
			UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();
			Settings->bThrottleCPUWhenNotForeground = bOldCPUThrottlingSetting;
			Settings->PostEditChange();
		}
	}

	void FPixelStreaming2EditorModule::DisableCPUThrottlingSetting()
	{
		if (++CpuThrottlingSetCount == 1)
		{
			// Update editor settings so that editor won't slow down if not in focus
			UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();

			// Store whatever value the user had in here so we can restore it when we are done streaming.
			bOldCPUThrottlingSetting = Settings->bThrottleCPUWhenNotForeground;

			if (Settings->bThrottleCPUWhenNotForeground)
			{
				Settings->bThrottleCPUWhenNotForeground = false;
				Settings->PostEditChange();

				// Let the user know we are forcing this editor setting (so they know why their setting is not working potentially)
				// We can also use the new keyword here because the notification manager will call delete once the notification has been displayed
				FNotificationInfo* Info = new FNotificationInfo(LOCTEXT("PixelStreaming2EditorModule_CPUThrottlingNotification", "Pixel Streaming: Disabling setting \"Use less CPU in background\" for streaming performance."));
				Info->ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().QueueNotification(Info);
			}
		}
	}

	void FPixelStreaming2EditorModule::StopSignallingAfterFrames(uint8 FrameDelay)
	{
		// NOTE (william.belcher): Because the signalling server websockets are ticked, we have to wait two frames before calling StopSignalling as
		// otherwise the server will still report having this PIEStreamer connected
		TWeakPtr<UE::PixelStreaming2Servers::IServer> WeakServer = SignallingServer; // Use a weak pointer to the signalling server to track module validity
		uint8 NumFramesToWait = FrameDelay;
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakServer, NumFramesToWait](float) mutable {
			NumFramesToWait--;
			if (NumFramesToWait > 0)
			{
				// If we still have frames to wait, return true to continue ticking
				return true;
			}

			if (WeakServer.IsValid())
			{
				// A `this` capture is safe here because the lambda is called synchronously before the function returns
				this->SignallingServer->GetNumStreamers([this](uint16 NumStreamers) mutable {
					if (NumStreamers == 0)
					{
						this->SignallingServer->Stop();
						this->SignallingServer.Reset();
					}
				});
			}

			return false;
		}));
	}

#if WITH_EDITOR
	void FPixelStreaming2EditorModule::OnBeginPIE(bool bIsSimulating)
	{
		if (!UPixelStreaming2PluginSettings::CVarAutoStreamPIE.GetValueOnAnyThread())
		{
			return;
		}

		IPixelStreaming2Module& Module = IPixelStreaming2Module::Get();
		PIEStreamer = Module.CreateStreamer(Module.GetDefaultStreamerID());
		// Give the PIE streamer the default url if the user hasn't specified one when launching the editor
		if (PIEStreamer->GetConnectionURL().IsEmpty())
		{
			// No URL was passed on the command line, initialize defaults
			PIEStreamer->SetConnectionURL(FString::Printf(TEXT("%s:%d"), *SignallingDomain, StreamerPort));
		}

		if (!UPixelStreaming2PluginSettings::CVarEditorUseRemoteSignallingServer.GetValueOnAnyThread())
		{
			StartSignalling();
		}

		// Bind to start/stop streaming so we disable/restore relevant editor settings
		PIEStreamer->OnStreamingStarted().AddLambda([this](IPixelStreaming2Streamer* Streamer) {
			DisableCPUThrottlingSetting();
		});
		PIEStreamer->OnStreamingStopped().AddLambda([this](IPixelStreaming2Streamer* Streamer) {
			if (!IsEngineExitRequested())
			{
				RestoreCPUThrottlingSetting();
			}
		});

		PIEStreamer->SetVideoProducer(FVideoProducerPIEViewport::Create());

		TSharedPtr<IPixelStreaming2InputHandler> InputHandler = PIEStreamer->GetInputHandler().Pin();
		if (!InputHandler.IsValid())
		{
			return;
		}

		UGameViewportClient* Viewport = nullptr;
		// Iterate through world contexts to get the first PIE viewport that is for a client or a standalone game
		for (const FWorldContext& Context : GEditor->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE && Context.World())
			{
				ENetMode NetMode = Context.World()->GetNetMode();
        		if (NetMode == NM_Client || NetMode == NM_Standalone)
        		{
            		Viewport = Context.GameViewport;
            		break;
        		}
			}
		}

		if (!Viewport)
		{
			UE_LOGFMT(LogPixelStreaming2Editor, Warning, "Failed to get PIE viewport. PIE streamer will not stream!");
			return;
		}

		InputHandler->SetTargetViewport(Viewport->GetGameViewportWidget());
		InputHandler->SetTargetWindow(Viewport->GetWindow());
		InputHandler->SetInputType(EPixelStreaming2InputType::RouteToWindow);
		PIEStreamer->StartStreaming();
	}

	void FPixelStreaming2EditorModule::OnEndPIE(bool bIsSimulating)
	{
		if (PIEStreamer)
		{
			PIEStreamer->StopStreaming();
			
			// Remove from the module so that the PIE streamer doesn't pollute the toolbar
			IPixelStreaming2Module::Get().DeleteStreamer(PIEStreamer);
		}

		StopSignalling(false);
	}
#endif
} // namespace UE::EditorPixelStreaming2

#undef IMAGE_BRUSH_SVG

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::EditorPixelStreaming2::FPixelStreaming2EditorModule, PixelStreaming2Editor)
