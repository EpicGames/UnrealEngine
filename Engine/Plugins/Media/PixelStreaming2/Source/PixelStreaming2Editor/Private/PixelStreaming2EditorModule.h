// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2EditorModule.h"
#include "IPixelStreaming2RTCModule.h"

#include <atomic>

namespace UE::EditorPixelStreaming2
{
	class FPixelStreaming2Toolbar;

	class FPixelStreaming2EditorModule : public IPixelStreaming2EditorModule
	{
	public:
		/** IModuleInterface implementation */
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		virtual void StartStreaming(EPixelStreaming2EditorStreamTypes InStreamType) override;
		virtual void StopStreaming() override;

		virtual void											StartSignalling() override;
		virtual void											StopSignalling(bool bForce) override;
		virtual TSharedPtr<UE::PixelStreaming2Servers::IServer> GetSignallingServer() override;

		virtual void	SetSignallingDomain(const FString& InSignallingDomain) override;
		virtual FString GetSignallingDomain() override { return SignallingDomain; };
		virtual void	SetStreamerPort(int32 InStreamerPort) override;
		virtual int32	GetStreamerPort() override { return StreamerPort; };
		virtual void	SetViewerPort(int32 InViewerPort) override;
		virtual int32	GetViewerPort() override { return ViewerPort; };
		virtual void	SetServeHttps(bool bServeHttps) override;
		virtual bool	GetServeHttps() override { return bServeHttps; }
		virtual void	SetSSLCertificatePath(const FString& Path) override;
		virtual FString GetSSLCertificatePath() override { return SSLCertificatePath; }
		virtual void	SetSSLPrivateKeyPath(const FString& Path) override;
		virtual FString GetSSLPrivateKeyPath() override { return SSLPrivateKeyPath; }
		/** End IModuleInterface implementation */

		virtual FString GetEditorStreamerID() const { return EditorStreamerID; }

	private:
		void InitEditorStreaming(IPixelStreaming2RTCModule& Module);
		bool ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY);
		void MaybeResizeEditor(TSharedPtr<SWindow> RootWindow);
		void DisableCPUThrottlingSetting();
		void RestoreCPUThrottlingSetting(bool bForce = false);
		void StopSignallingAfterFrames(uint8 FrameDelay);

#if WITH_EDITOR
		void OnBeginPIE(bool bIsSimulating);
		void OnEndPIE(bool bIsSimulating);
#endif

		TSharedPtr<FPixelStreaming2Toolbar> Toolbar;
		// Signalling/webserver
		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer;
		// Download process for PS web frontend files (if we want to view output in the browser)
		TSharedPtr<FMonitoredProcess> DownloadProcess;
		// The signalling server host: eg ws://127.0.0.1
		FString SignallingDomain;
		// The port the streamer will connect to. eg 8888
		int32 StreamerPort;
		// The port the streams can be viewed at on the browser. eg 80 or 8080
#if PLATFORM_LINUX
		int32 ViewerPort = 8080; // ports <1000 require superuser privileges on Linux
#else
		int32 ViewerPort = 80;
#endif
		//
		bool bServeHttps = false;
		//
		FString SSLCertificatePath;
		//
		FString SSLPrivateKeyPath;

		FString EditorStreamerID;

		// The streamer used by the PixelStreaming2Editor module
		TSharedPtr<IPixelStreaming2Streamer> EditorStreamer;

#if WITH_EDITOR
		TSharedPtr<IPixelStreaming2Streamer> PIEStreamer;
#endif

		// The original throttling setting prior to first streamer disabling it
		bool bOldCPUThrottlingSetting = false;
		// Number of streamers that have requested the throttling be disabled
		std::atomic<int> CpuThrottlingSetCount = 0;
	};
} // namespace UE::EditorPixelStreaming2
