// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Style.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

namespace UE::EditorPixelStreaming2
{
    TSharedPtr<FSlateStyleSet> FPixelStreaming2Style::StyleInstance = NULL;
    void FPixelStreaming2Style::Initialize()
    {
        if(!StyleInstance.IsValid())
        {
            StyleInstance = Create();
            FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
        }
    }

    void FPixelStreaming2Style::Shutdown()
    {
        FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
    	ensure(StyleInstance.IsUnique());
	    StyleInstance.Reset();
    }

    FName FPixelStreaming2Style::GetStyleSetName()
    {
        static FName StyleSetName(TEXT("PixelStreaming2Style"));
        return StyleSetName;
    }

    const FVector2D Icon16x16(16.0f, 16.0f);
    const FVector2D Icon20x20(20.0f, 20.0f);
    const FVector2D Icon64x64(64.0f, 64.0f);
    
    TSharedRef<FSlateStyleSet> FPixelStreaming2Style::Create()
    {
        TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("PixelStreaming2Style"));
        Style->SetContentRoot(IPluginManager::Get().FindPlugin("PixelStreaming2")->GetBaseDir() / TEXT("Resources"));
        Style->Set("PixelStreaming2.Icon", new IMAGE_BRUSH_SVG("PixelStreaming2_16", Icon16x16));
        return Style;
    }
    
    void FPixelStreaming2Style::ReloadTextures()
    {
        if(FSlateApplication::IsInitialized())
        {
            FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
        }
    }
    
    FSlateStyleSet& FPixelStreaming2Style::Get()
    {
        return *StyleInstance;
    }
}

#undef RootToContentDir