// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Toolbar.h"
#include "ToolMenus.h"
#include "PixelStreaming2Commands.h"
#include "PixelStreaming2Style.h"
#include "IPixelStreaming2Module.h"
#include "Widgets/SViewport.h"
#include "PixelStreaming2EditorModule.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include <SocketSubsystem.h>
#include "Widgets/Input/SNumericEntryBox.h"
#include "PixelStreaming2PluginSettings.h"
#include "Logging.h"
#include "UtilsCoder.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"

#define LOCTEXT_NAMESPACE "PixelStreaming2Editor"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreaming2Toolbar, Log, All);
DEFINE_LOG_CATEGORY(LogPixelStreaming2Toolbar);

void SetCodec(EVideoCodec Codec)
{
	UPixelStreaming2PluginSettings::CVarEncoderCodec.AsVariable()->SetWithCurrentPriority(*UE::PixelStreaming2::GetCVarStringFromEnum(Codec));
}

EVideoCodec GetCodec()
{
	return UE::PixelStreaming2::GetEnumFromCVar<EVideoCodec>(UPixelStreaming2PluginSettings::CVarEncoderCodec);
}

void SetUseRemoteSignallingServer(bool UseRemoteSignallingServer)
{
	UPixelStreaming2PluginSettings::CVarEditorUseRemoteSignallingServer.AsVariable()->SetWithCurrentPriority(UseRemoteSignallingServer);
}

bool GetUseRemoteSignallingServer()
{
	return UPixelStreaming2PluginSettings::CVarEditorUseRemoteSignallingServer.GetValueOnAnyThread();
}

void SetServeHttps(bool bServeHttps)
{
	IPixelStreaming2EditorModule::Get().SetServeHttps(bServeHttps);
}

bool GetServeHttps()
{
	return IPixelStreaming2EditorModule::Get().GetServeHttps();
}

void SetSSLCertificatePath(const FString& Path)
{
	IPixelStreaming2EditorModule::Get().SetSSLCertificatePath(Path);
}

FString GetSSLCertificatePath()
{
	return IPixelStreaming2EditorModule::Get().GetSSLCertificatePath();
}

void SetSSLPrivateKeyPath(const FString& Path)
{
	IPixelStreaming2EditorModule::Get().SetSSLPrivateKeyPath(Path);
}

FString GetSSLPrivateKeyPath()
{
	return IPixelStreaming2EditorModule::Get().GetSSLPrivateKeyPath();
}

namespace UE::EditorPixelStreaming2
{
	TSharedRef<SWidget> CreateTextboxWithFileSelector(const FString& Label, float MaxWidth, TFunction<FString()> GetTextLambda, TFunction<void(const FText&)> OnTextChangedLambda, TFunction<void(const FText&, const ETextCommit::Type)> OnTextCommittedLambda, TFunction<void()> OnClickedLambda, TFunction<EVisibility()> GetVisibilityLambda, TFunction<bool()> IsEnabledLambda)
	{
		TSharedRef<SWidget> Widget = SNew(SHorizontalBox)
			.Visibility_Lambda([GetVisibilityLambda]() {
				return GetVisibilityLambda();
			})
			+ SHorizontalBox::Slot()
				  .AutoWidth()
				  .VAlign(VAlign_Center)
				  .Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
					  [SNew(STextBlock)
							  .Text(FText::FromString(Label))
							  .ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))]
			+ SHorizontalBox::Slot()
				  .FillWidth(1.0f)
				  .VAlign(VAlign_Center)
				  .MaxWidth(MaxWidth)
					  [SNew(SEditableTextBox)
							  .OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
							  .Text_Lambda([GetTextLambda]() {
								  return FText::FromString(GetTextLambda());
							  })
							  .OnTextChanged_Lambda([OnTextChangedLambda](const FText& Text) {
								  OnTextChangedLambda(Text);
							  })
							  .OnTextCommitted_Lambda([OnTextCommittedLambda](const FText& Text, const ETextCommit::Type CommitType) {
								  OnTextCommittedLambda(Text, CommitType);
							  })
							  .IsEnabled_Lambda([IsEnabledLambda]() {
								  return IsEnabledLambda();
							  })
							]
			+ SHorizontalBox::Slot()
				  .AutoWidth()
				  .HAlign(HAlign_Right)
					  [SNew(SButton)
							  .OnClicked_Lambda([OnClickedLambda]() {
								  OnClickedLambda();
								  return FReply::Handled();
							  })
							  .IsEnabled_Lambda([IsEnabledLambda]() {
								  return IsEnabledLambda();
							  })
							  .ButtonStyle(FAppStyle::Get(), "SimpleButton")
								  [SNew(SImage)
										  .Image(FAppStyle::Get().GetBrush("Icons.BrowseContent"))
										  .ColorAndOpacity(FSlateColor::UseForeground())]];

		return Widget;
	}

	FPixelStreaming2Toolbar::FPixelStreaming2Toolbar()
	{
		FPixelStreaming2Commands::Register();

		PluginCommands = MakeShared<FUICommandList>();

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().ExternalSignalling,
			FExecuteAction::CreateLambda([]() {
				SetUseRemoteSignallingServer(!GetUseRemoteSignallingServer());
				IPixelStreaming2EditorModule::Get().StopSignalling(true);
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
				if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}),
			FIsActionChecked::CreateLambda([]() {
				return GetUseRemoteSignallingServer();
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().ServeHttps,
			FExecuteAction::CreateLambda([]() {
				SetServeHttps(!GetServeHttps());
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
				if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}),
			FIsActionChecked::CreateLambda([]() {
				return GetServeHttps();
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().StreamLevelEditor,
			FExecuteAction::CreateLambda([]() {
				IPixelStreaming2EditorModule::Get().StartStreaming(EPixelStreaming2EditorStreamTypes::LevelEditorViewport);
			}),
			FCanExecuteAction::CreateLambda([] {
				FPixelStreaming2EditorModule& EditorModule = static_cast<FPixelStreaming2EditorModule&>(IPixelStreaming2EditorModule::Get());
				if (TSharedPtr<IPixelStreaming2Streamer> Streamer = IPixelStreaming2Module::Get().FindStreamer(EditorModule.GetEditorStreamerID()))
				{
					return !Streamer->IsStreaming();
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().StreamEditor,
			FExecuteAction::CreateLambda([]() {
				IPixelStreaming2EditorModule::Get().StartStreaming(EPixelStreaming2EditorStreamTypes::Editor);
			}),
			FCanExecuteAction::CreateLambda([] {
				FPixelStreaming2EditorModule& EditorModule = static_cast<FPixelStreaming2EditorModule&>(IPixelStreaming2EditorModule::Get());
				if (TSharedPtr<IPixelStreaming2Streamer> Streamer = IPixelStreaming2Module::Get().FindStreamer(EditorModule.GetEditorStreamerID()))
				{
					return !Streamer->IsStreaming();
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().StartSignalling,
			FExecuteAction::CreateLambda([]() {
				IPixelStreaming2EditorModule::Get().StartSignalling();
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
				if (!SignallingServer.IsValid() || !SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().StopSignalling,
			FExecuteAction::CreateLambda([]() {
				IPixelStreaming2EditorModule::Get().StopSignalling(true);
			}),
			FCanExecuteAction::CreateLambda([] {
				TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
				if (SignallingServer.IsValid() && SignallingServer->HasLaunched())
				{
					return true;
				}
				return false;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().VP8,
			FExecuteAction::CreateLambda([]() {
				SetCodec(EVideoCodec::VP8);
			}),
			FCanExecuteAction::CreateLambda([] {
				return true;
			}),
			FIsActionChecked::CreateLambda([]() {
				return GetCodec() == EVideoCodec::VP8;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().VP9,
			FExecuteAction::CreateLambda([]() {
				SetCodec(EVideoCodec::VP9);
			}),
			FCanExecuteAction::CreateLambda([] {
				return true;
			}),
			FIsActionChecked::CreateLambda([]() {
				return GetCodec() == EVideoCodec::VP9;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().H264,
			FExecuteAction::CreateLambda([]() {
				SetCodec(EVideoCodec::H264);
			}),
			FCanExecuteAction::CreateLambda([] {
				bool bCanChangeCodec = UE::PixelStreaming2::IsEncoderSupported<FVideoEncoderConfigH264>();
				return bCanChangeCodec;
			}),
			FIsActionChecked::CreateLambda([]() {
				return GetCodec() == EVideoCodec::H264;
			}));

		PluginCommands->MapAction(
			FPixelStreaming2Commands::Get().AV1,
			FExecuteAction::CreateLambda([]() {
				SetCodec(EVideoCodec::AV1);
			}),
			FCanExecuteAction::CreateLambda([] {
				bool bCanChangeCodec = UE::PixelStreaming2::IsEncoderSupported<FVideoEncoderConfigAV1>();
				return bCanChangeCodec;
			}),
			FIsActionChecked::CreateLambda([]() {
				return GetCodec() == EVideoCodec::AV1;
			}));

		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPixelStreaming2Toolbar::RegisterMenus));
	}

	FPixelStreaming2Toolbar::~FPixelStreaming2Toolbar()
	{
		FPixelStreaming2Commands::Unregister();
	}

	void FPixelStreaming2Toolbar::RegisterMenus()
	{
		if (!GetDefault<UPixelStreaming2PluginSettings>()->EnablePixelStreamingToolbar)
		{
			return;
		}

		FToolMenuOwnerScoped OwnerScoped(this);
		{
			UToolMenu* CustomToolBar = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
			{
				FToolMenuSection& Section = CustomToolBar->AddSection("PixelStreaming2");
				Section.AddSeparator("PixelStreaming2Seperator");
				{
					// Settings dropdown
					FToolMenuEntry SettingsEntry = FToolMenuEntry::InitComboButton(
						"PixelStreaming2Menus",
						FUIAction(),
						FOnGetContent::CreateLambda(
							[&]() {
								FMenuBuilder MenuBuilder(false, PluginCommands, MakeShared<FExtender>(), false, &FCoreStyle::Get(), false);

								// Use external signalling server option
								MenuBuilder.BeginSection("Signalling Server Location", LOCTEXT("PixelStreaming2SSLocation", "Signalling Server Location"));
								MenuBuilder.AddMenuEntry(FPixelStreaming2Commands::Get().ExternalSignalling);
								MenuBuilder.EndSection();

								// Embedded Signalling Server Config (streamer port & http port)
								RegisterEmbeddedSignallingServerConfig(MenuBuilder);

								// Signalling Server Viewer URLs
								RegisterSignallingServerURLs(MenuBuilder);

								// Remote Signalling Server Config (URL)
								RegisterRemoteSignallingServerConfig(MenuBuilder);

								// Pixel Streaming streamer controls
								RegisterStreamerControls(MenuBuilder);

								// Codec Config
								RegisterCodecConfig(MenuBuilder);

								return MenuBuilder.MakeWidget();
							}),
						LOCTEXT("PixelStreaming2Menu", "Pixel Streaming 2"),
						LOCTEXT("PixelStreaming2MenuTooltip", "Configure Pixel Streaming"),
						FSlateIcon(FPixelStreaming2Style::GetStyleSetName(), "PixelStreaming2.Icon"),
						false,
						"PixelStreaming2Menu");
					SettingsEntry.StyleNameOverride = "CalloutToolbar";
					SettingsEntry.SetCommandList(PluginCommands);
					Section.AddEntry(SettingsEntry);
				}
			}
		}
	}

	void FPixelStreaming2Toolbar::RegisterEmbeddedSignallingServerConfig(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection(
			"Signalling Server Options", 
			LOCTEXT("PixelStreaming2EmbeddedSSOptions", "Embedded Signalling Server Options"),
			TAttribute<EVisibility>::Create([]() {
        		return !GetUseRemoteSignallingServer()
            		? EVisibility::Visible
            		: EVisibility::Collapsed;
    		}));

		MenuBuilder.AddMenuEntry(
    		FPixelStreaming2Commands::Get().ServeHttps,       // Command
    		NAME_None,                                              // InExtensionHook
    		TAttribute<FText>(),                                    // Label override (leave empty to use default)
    		TAttribute<FText>(),                                    // Tooltip override (leave empty to use default)
    		FSlateIcon(),                                           // Icon override (leave default)
    		NAME_None,                                              // Tutorial highlight
    		TAttribute<EVisibility>::Create([]() {                 // Visibility lambda
    		    TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
    		    return !GetUseRemoteSignallingServer()
    		        ? EVisibility::Visible
    		        : EVisibility::Collapsed;
    		})
		);

		TSharedRef<SWidget> SSLCertificateInputBlock = CreateTextboxWithFileSelector(
			TEXT("SSL Certificate: "),																			   // Label
			150,																								   // MaxWidth
			[]() { return GetSSLCertificatePath(); },															   // GetTextLambda
			[](const FText& Text) { SetSSLCertificatePath(Text.ToString()); },									   // OnTextChangedLambda
			[](const FText& Text, const ETextCommit::Type CommitType) { SetSSLCertificatePath(Text.ToString()); }, // OnTextCommittedLambda
			[this]() { OnOpenFileBrowserClicked(EFileType::Certificate); },										   // OnClickedLambda
			[]() { return !GetUseRemoteSignallingServer() &&GetServeHttps() ? EVisibility::Visible : EVisibility::Collapsed; },					   // GetVisibilityLambda
			[]() { 
				TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
				return !(SignallingServer.IsValid() && SignallingServer->HasLaunched()); 
			}					   // IsEnabledLambda
		);
		MenuBuilder.AddWidget(SSLCertificateInputBlock, FText(), true);

		TSharedRef<SWidget> SSLPrivateKeyInputBlock = CreateTextboxWithFileSelector(
			TEXT("SSL Private Key: "),																			  // Label
			150,																								  // MaxWidth
			[]() { return GetSSLPrivateKeyPath(); },															  // GetTextLambda
			[](const FText& Text) { SetSSLPrivateKeyPath(Text.ToString()); },									  // OnTextChangedLambda
			[](const FText& Text, const ETextCommit::Type CommitType) { SetSSLPrivateKeyPath(Text.ToString()); }, // OnTextCommittedLambda
			[this]() { OnOpenFileBrowserClicked(EFileType::PrivateKey); },										  // OnClickedLambda
			[]() { return !GetUseRemoteSignallingServer() && GetServeHttps() ? EVisibility::Visible : EVisibility::Collapsed; },					   // GetVisibilityLambda
			[]() { 
				TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
				return !(SignallingServer.IsValid() && SignallingServer->HasLaunched()); 
			}					   // IsEnabledLambda
		);
		MenuBuilder.AddWidget(SSLPrivateKeyInputBlock, FText(), true);

		TSharedRef<SWidget> StreamerPortInputBlock = SNew(SHorizontalBox)
			.Visibility_Lambda([]() {
				return !GetUseRemoteSignallingServer() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
			[
				SNew(STextBlock)
					.Text(FText::FromString(TEXT("Streamer Port: ")))
					.ToolTipText(LOCTEXT("PixelStreaming2StreamerPortTooltip", "The port the signalling server will listen for streamer connections on."))
					.ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SNumericEntryBox<int32>)
					.MinValue(1)
					.Value_Lambda([]() {
						return IPixelStreaming2EditorModule::Get().GetStreamerPort();
					})
					.OnValueChanged_Lambda([](int32 InStreamerPort) {
						IPixelStreaming2EditorModule::Get().SetStreamerPort(InStreamerPort);
					})
					.OnValueCommitted_Lambda([](int32 InStreamerPort, ETextCommit::Type InCommitType) {
						IPixelStreaming2EditorModule::Get().SetStreamerPort(InStreamerPort);
					})
					.IsEnabled_Lambda([]() {
						TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
						return !(SignallingServer.IsValid() && SignallingServer->HasLaunched()); 
					})
			];
		MenuBuilder.AddWidget(StreamerPortInputBlock, FText(), true);

		TSharedRef<SWidget> ViewerPortInputBlock = SNew(SHorizontalBox)
			.Visibility_Lambda([]() {
				return !GetUseRemoteSignallingServer() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
			[
				SNew(STextBlock)
					.Text(FText::FromString(TEXT("Viewer Port: ")))
					.ToolTipText(LOCTEXT("PixelStreaming2ViewerPortTooltip", "The port the signalling server will listen for viewer connections on."))
					.ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SNumericEntryBox<int32>)
					.MinValue(1)
					.Value_Lambda([]() {
						return IPixelStreaming2EditorModule::Get().GetViewerPort();
					})
					.OnValueChanged_Lambda([](int32 InViewerPort) {
						IPixelStreaming2EditorModule::Get().SetViewerPort(InViewerPort);
					})
					.OnValueCommitted_Lambda([](int32 InViewerPort, ETextCommit::Type InCommitType) {
						IPixelStreaming2EditorModule::Get().SetViewerPort(InViewerPort);
					})
					.IsEnabled_Lambda([]() {
						TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
						return !(SignallingServer.IsValid() && SignallingServer->HasLaunched()); 
					})
			];
		MenuBuilder.AddWidget(ViewerPortInputBlock, FText(), true);

		TSharedRef<SWidget> AllowedOriginsInputBlock = SNew(SHorizontalBox)
			.Visibility_Lambda([]() {
				return !GetUseRemoteSignallingServer() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
			[
				SNew(STextBlock)
					.Text(FText::FromString(TEXT("Allowed Origins: ")))
					.ToolTipText(LOCTEXT("PixelStreaming2AllowedOriginsTooltip", "Comma-separated list of allowed origins for WebSocket connection filtering. Supports wildcards (*). Use '*' to allow all origins."))
					.ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.MaxWidth(200.0f)
			[
				SNew(SEditableTextBox)
					.Text_Lambda([]() {
						return FText::FromString(UPixelStreaming2PluginSettings::CVarAllowedOrigins.GetValueOnAnyThread());
					})
					.OnTextChanged_Lambda([](const FText& InText) {
						UPixelStreaming2PluginSettings::CVarAllowedOrigins.AsVariable()->SetWithCurrentPriority(*InText.ToString());
					})
					.OnTextCommitted_Lambda([](const FText& InText, ETextCommit::Type InCommitType) {
						UPixelStreaming2PluginSettings::CVarAllowedOrigins.AsVariable()->SetWithCurrentPriority(*InText.ToString());
						// Save to ini so it persists across sessions
						if (UPixelStreaming2PluginSettings* Settings = GetMutableDefault<UPixelStreaming2PluginSettings>())
						{
							Settings->AllowedOrigins = InText.ToString();
							Settings->SaveConfig();
						}
					})
					.IsEnabled_Lambda([]() {
						TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
						return !(SignallingServer.IsValid() && SignallingServer->HasLaunched());
					})
			];
		MenuBuilder.AddWidget(AllowedOriginsInputBlock, FText(), true);

		MenuBuilder.AddMenuEntry(
    		FPixelStreaming2Commands::Get().StartSignalling,       // Command
    		NAME_None,                                              // InExtensionHook
    		TAttribute<FText>(),                                    // Label override (leave empty to use default)
    		TAttribute<FText>(),                                    // Tooltip override (leave empty to use default)
    		FSlateIcon(),                                           // Icon override (leave default)
    		NAME_None,                                              // Tutorial highlight
    		TAttribute<EVisibility>::Create([]() {                 // Visibility lambda
    		    TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
    		    return !GetUseRemoteSignallingServer() && !(SignallingServer.IsValid() && SignallingServer->HasLaunched())
    		        ? EVisibility::Visible
    		        : EVisibility::Collapsed;
    		})
		);

		MenuBuilder.AddMenuEntry(
    		FPixelStreaming2Commands::Get().StopSignalling,       // Command
    		NAME_None,                                              // InExtensionHook
    		TAttribute<FText>(),                                    // Label override (leave empty to use default)
    		TAttribute<FText>(),                                    // Tooltip override (leave empty to use default)
    		FSlateIcon(),                                           // Icon override (leave default)
    		NAME_None,                                              // Tutorial highlight
    		TAttribute<EVisibility>::Create([]() {                 // Visibility lambda
    		    TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
    		    return !GetUseRemoteSignallingServer() &&(SignallingServer.IsValid() && SignallingServer->HasLaunched())
    		        ? EVisibility::Visible
    		        : EVisibility::Collapsed;
    		})
		);

		MenuBuilder.EndSection();
	}

	void FPixelStreaming2Toolbar::RegisterRemoteSignallingServerConfig(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection(
			"Remote Signalling Server Options", 
			LOCTEXT("PixelStreaming2RemoteSSOptions", "Remote Signalling Server Options"),
			TAttribute<EVisibility>::Create([]() {
        		return GetUseRemoteSignallingServer()
            		? EVisibility::Visible
            		: EVisibility::Collapsed;
    		}));

		{
			TSharedRef<SWidget> URLInputBlock = SNew(SHorizontalBox)
				.Visibility_Lambda([]() {
					return GetUseRemoteSignallingServer() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(36.0f, 3.0f, 8.0f, 3.0f))
				[
					SNew(STextBlock)
						.Text(FText::FromString(TEXT("Remote Signalling Server URL")))
						.ColorAndOpacity(FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f)))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SEditableTextBox)
						.Text_Lambda([]() {
							FPixelStreaming2EditorModule& EditorModule = static_cast<FPixelStreaming2EditorModule&>(IPixelStreaming2EditorModule::Get());
							TSharedPtr<IPixelStreaming2Streamer> Streamer = IPixelStreaming2Module::Get().FindStreamer(EditorModule.GetEditorStreamerID());
							return FText::FromString(Streamer->GetConnectionURL());
						})
						.OnTextChanged_Lambda([](const FText& InText) {
							IPixelStreaming2Module::Get().ForEachStreamer([InText](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
								Streamer->SetConnectionURL(InText.ToString());
							});
						})
						.OnTextCommitted_Lambda([](const FText& InText, ETextCommit::Type InTextCommit) {
							IPixelStreaming2Module::Get().ForEachStreamer([InText](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
								Streamer->SetConnectionURL(InText.ToString());
							});
						})
						.IsEnabled_Lambda([]() {
							bool bCanChangeURL = true;
							IPixelStreaming2Module::Get().ForEachStreamer([&bCanChangeURL](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
								bCanChangeURL &= !Streamer->IsStreaming();
							});
							return bCanChangeURL;
						})];
			MenuBuilder.AddWidget(URLInputBlock, FText(), true);
		}
		MenuBuilder.EndSection();
	}

	void FPixelStreaming2Toolbar::RegisterSignallingServerURLs(FMenuBuilder& MenuBuilder)
	{
		TSharedRef<SVerticalBox> AdapterBox = SNew(SVerticalBox);
		
		TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(AdapterAddresses);
		for (TSharedPtr<FInternetAddr> AdapterAddress : AdapterAddresses)
		{
		    AdapterBox->AddSlot()
		    .AutoHeight()
		    .Padding(FMargin(32.f, 3.f, 0.f, 3.f))
		    [
		        SNew(STextBlock)
		        	.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		        	.Text_Lambda([AdapterAddress]() {
		        	    return FText::FromString(FString::Printf(
		        	        TEXT("%s://%s:%d"),
		        	        GetServeHttps() ? TEXT("https") : TEXT("http"),
		        	        *AdapterAddress->ToString(false),
		        	        IPixelStreaming2EditorModule::Get().GetViewerPort()
		        	    ));
		        	})
		    ];
		}


		MenuBuilder.BeginSection(
			"Signalling Server URLs", 
			LOCTEXT("PixelStreaming2SignallingURLs", "Signalling Server URLs"),
			TAttribute<EVisibility>::Create([]() {
        		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
        		return (SignallingServer.IsValid() && SignallingServer->HasLaunched())
            		? EVisibility::Visible
            		: EVisibility::Collapsed;
    		})
		);
		{
			MenuBuilder.AddWidget(
				SNew(SVerticalBox)
					.Visibility_Lambda([]() {
						TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = IPixelStreaming2EditorModule::Get().GetSignallingServer();
						return (SignallingServer.IsValid() && SignallingServer->HasLaunched())
						? EVisibility::Visible
						: EVisibility::Collapsed;
					})
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(16.0f, 3.0f))
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(LOCTEXT("SignallingTip", "The Signalling Server is running and may be accessed via the following URLs (network settings permitting):"))
					.WrapTextAt(400)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(32.0f, 3.0f))
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text_Lambda([]() {
						return FText::FromString(
							FString::Printf(TEXT("%s://127.0.0.1:%d"),
								GetServeHttps() ? TEXT("https") : TEXT("http"),
								IPixelStreaming2EditorModule::Get().GetViewerPort()));
					})
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					AdapterBox
				]
			, FText() // No label needed
			);
		}
		MenuBuilder.EndSection();
	}

	void FPixelStreaming2Toolbar::RegisterStreamerControls(FMenuBuilder& MenuBuilder)
	{
		IPixelStreaming2Module::Get().ForEachStreamer([&](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
			FString StreamerId = Streamer->GetId();
			MenuBuilder.BeginSection(FName(*StreamerId), FText::FromString(FString::Printf(TEXT("Streamer - %s"), *StreamerId)));
			{
				MenuBuilder.AddWidget(
				    SNew(SVerticalBox)
				    + SVerticalBox::Slot()
				    .AutoHeight()
				    [
				        SNew(SVerticalBox)
				        + SVerticalBox::Slot()
				        .AutoHeight()
				        [
				            SNew(SBox)
				            .Padding(FMargin(16.f, 3.f))
				            [
				                SNew(STextBlock)
				                .ColorAndOpacity(FSlateColor::UseSubduedForeground())
				                .Text_Lambda([Streamer]() -> FText {
				                    if (Streamer->IsStreaming())
				                    {
				                        FString VideoProducer = TEXT("nothing (no video input)");
				                        if (TSharedPtr<IPixelStreaming2VideoProducer> Video = Streamer->GetVideoProducer().Pin())
				                        {
				                            VideoProducer = Video->ToString();
				                        }
				                        return FText::FromString(FString::Printf(TEXT("Streaming %s"), *VideoProducer));
				                    }
				                    else
				                    {
				                        return FText::GetEmpty();
				                    }
				                })
								.Visibility_Lambda([Streamer]() {
									return Streamer->IsStreaming() ? EVisibility::Visible : EVisibility::Collapsed;
								})
				                .WrapTextAt(400)
				            ]
				        ]
				    ],
				    FText()
				);

				FPixelStreaming2EditorModule& EditorModule = static_cast<FPixelStreaming2EditorModule&>(IPixelStreaming2EditorModule::Get());
				if (Streamer->GetId() == EditorModule.GetEditorStreamerID())
				{
					MenuBuilder.AddMenuEntry(
    					FPixelStreaming2Commands::Get().StreamLevelEditor,       // Command
    					NAME_None,                                              // InExtensionHook
    					TAttribute<FText>(),                                    // Label override (leave empty to use default)
    					TAttribute<FText>(),                                    // Tooltip override (leave empty to use default)
    					FSlateIcon(),                                           // Icon override (leave default)
    					NAME_None,                                              // Tutorial highlight
    					TAttribute<EVisibility>::Create([Streamer]() { return !Streamer->IsStreaming() ? EVisibility::Visible : EVisibility::Collapsed; })
					);

					MenuBuilder.AddMenuEntry(
    					FPixelStreaming2Commands::Get().StreamEditor,       // Command
    					NAME_None,                                              // InExtensionHook
    					TAttribute<FText>(),                                    // Label override (leave empty to use default)
    					TAttribute<FText>(),                                    // Tooltip override (leave empty to use default)
    					FSlateIcon(),                                           // Icon override (leave default)
    					NAME_None,                                              // Tutorial highlight
    					TAttribute<EVisibility>::Create([Streamer]() { return !Streamer->IsStreaming() ? EVisibility::Visible : EVisibility::Collapsed; })
					);
				}
				else
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("PixelStreaming2_StartStreaming", "Start Streaming"),
						LOCTEXT("PixelStreaming2_StartStreamingToolTip", "Start this streamer"),
						FSlateIcon(),
						FExecuteAction::CreateLambda([Streamer]() { Streamer->StartStreaming(); }),
						NAME_None,
						EUserInterfaceActionType::Button,
						NAME_None,
						TAttribute<FText>(),
						TAttribute<EVisibility>::Create([Streamer]() { return !Streamer->IsStreaming() ? EVisibility::Visible : EVisibility::Collapsed; })
					);
				}

				MenuBuilder.AddMenuEntry(
						LOCTEXT("PixelStreaming2_StopStreaming", "Stop Streaming"),
						LOCTEXT("PixelStreaming2_StopStreamingToolTip", "Stop this streamer"),
						FSlateIcon(),
						FExecuteAction::CreateLambda([Streamer]() { Streamer->StopStreaming(); }),
						NAME_None,
						EUserInterfaceActionType::Button,
						NAME_None,
						TAttribute<FText>(),
						TAttribute<EVisibility>::Create([Streamer]() { return Streamer->IsStreaming() ? EVisibility::Visible : EVisibility::Collapsed; })
				);
			}
			MenuBuilder.EndSection();
		});
	}

	void FPixelStreaming2Toolbar::RegisterCodecConfig(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("Codec", LOCTEXT("PixelStreaming2CodecSettings", "Codec"));
		{
			MenuBuilder.AddMenuEntry(FPixelStreaming2Commands::Get().H264);
			MenuBuilder.AddMenuEntry(FPixelStreaming2Commands::Get().AV1);
			MenuBuilder.AddMenuEntry(FPixelStreaming2Commands::Get().VP8);
			MenuBuilder.AddMenuEntry(FPixelStreaming2Commands::Get().VP9);
		}
		MenuBuilder.EndSection();
	}

	void FPixelStreaming2Toolbar::OnOpenFileBrowserClicked(EFileType FileType)
	{
		bool bFileSelected = false;

		if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
		{
			const FString Title = LOCTEXT("PixelStreaming2_FileBrowser", "Choose file").ToString();

			if (LastBrowsePath.IsEmpty())
			{
				LastBrowsePath = FPaths::ProjectPluginsDir();
			}

			FString Filter = TEXT("");

			TArray<FString> OutFiles;
			bFileSelected = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				Title,
				LastBrowsePath,
				LastBrowsePath,
				Filter,
				EFileDialogFlags::None,
				OutFiles);

			if (!OutFiles.IsEmpty())
			{
				FString OutFilename = OutFiles[0];
				switch (FileType)
				{
					case EFileType::Certificate:
						UE_LOGFMT(LogPixelStreaming2Editor, Display, "Selecting certificate: {0}", OutFilename);
						SetSSLCertificatePath(OutFilename);
						break;
					case EFileType::PrivateKey:
						UE_LOGFMT(LogPixelStreaming2Editor, Display, "Selecting private key: {0}", OutFilename);
						SetSSLPrivateKeyPath(OutFilename);
						break;
					default:
						checkNoEntry();
				}
			}
		}
	}

	TSharedRef<SWidget> FPixelStreaming2Toolbar::GeneratePixelStreaming2MenuContent(TSharedPtr<FUICommandList> InCommandList)
	{
		FToolMenuContext MenuContext(InCommandList);
		return UToolMenus::Get()->GenerateWidget("LevelEditor.LevelEditorToolBar.AddQuickMenu", MenuContext);
	}
} // namespace UE::EditorPixelStreaming2

#undef LOCTEXT_NAMESPACE