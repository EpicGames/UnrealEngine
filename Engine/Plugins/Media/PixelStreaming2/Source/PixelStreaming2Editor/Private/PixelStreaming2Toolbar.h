// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

namespace UE::EditorPixelStreaming2
{
	class FPixelStreaming2Toolbar
	{
	public:
		FPixelStreaming2Toolbar();
		virtual ~FPixelStreaming2Toolbar();
		void					   StartStreaming();
		void					   StopStreaming();
		static TSharedRef<SWidget> GeneratePixelStreaming2MenuContent(TSharedPtr<FUICommandList> InCommandList);
		static FText			   GetActiveViewportName();
		static const FSlateBrush*  GetActiveViewportIcon();

	private:
		void RegisterMenus();
		void RegisterEmbeddedSignallingServerConfig(FMenuBuilder& MenuBuilder);
		void RegisterRemoteSignallingServerConfig(FMenuBuilder& MenuBuilder);
		void RegisterSignallingServerURLs(FMenuBuilder& MenuBuilder);
		void RegisterStreamerControls(FMenuBuilder& MenuBuilder);
		void RegisterVCamControls(FMenuBuilder& MenuBuilder);
		void RegisterCodecConfig(FMenuBuilder& MenuBuilder);

		enum class EFileType : uint8
		{
			Certificate,
			PrivateKey
		};

		void OnOpenFileBrowserClicked(EFileType FileType);

		TSharedPtr<class FUICommandList> PluginCommands;

		// Store the last opened path so users don't have to constantly re-navigate to a certs folder
		// when choosing cert and key
		FString LastBrowsePath;
	};
} // namespace UE::EditorPixelStreaming2
