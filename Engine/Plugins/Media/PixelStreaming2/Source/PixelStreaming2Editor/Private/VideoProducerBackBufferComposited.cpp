// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerBackBufferComposited.h"

#include "Async/Async.h"
#include "EngineModule.h"
#include "Framework/Application/SlateApplication.h"
#include "MediaShaders.h"
#include "PixelCaptureInputFrameRHI.h"
#include "ScreenPass.h"
#include "Slate/SlateViewportProvider.h"
#include "SceneView.h"
#include "RenderGraph.h"
#include "RHI.h"
#include "GenericPlatform/GenericPlatformTime.h"

namespace UE::EditorPixelStreaming2
{

	DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreaming2BackBufferComposited, Log, VeryVerbose);
	DEFINE_LOG_CATEGORY(LogPixelStreaming2BackBufferComposited);

	TSharedPtr<FVideoProducerBackBufferComposited> FVideoProducerBackBufferComposited::Create()
	{
		TSharedPtr<FVideoProducerBackBufferComposited> NewInput = TSharedPtr<FVideoProducerBackBufferComposited>(new FVideoProducerBackBufferComposited());
		TWeakPtr<FVideoProducerBackBufferComposited>   WeakInput = NewInput;
		// Set up the callback on the game thread since FSlateApplication::Get() can only be used there
		AsyncTask(ENamedThreads::GameThread, [WeakInput]() {
			if (TSharedPtr<FVideoProducerBackBufferComposited> Input = WeakInput.Pin())
			{
				FSlateApplication& SlateApplication = FSlateApplication::Get();
				Input->OnBackBufferReadyToPresentHandle = SlateApplication.GetRenderer()->OnBackBufferReadyToPresent().AddSP(Input.ToSharedRef(), &FVideoProducerBackBufferComposited::OnBackBufferReady);
				Input->OnPreTickHandle = SlateApplication.OnPreTick().AddSP(Input.ToSharedRef(), &FVideoProducerBackBufferComposited::OnPreTick);
			} });

		return NewInput;
	}

	FVideoProducerBackBufferComposited::FVideoProducerBackBufferComposited()
		: SharedFrameRect(MakeShared<FIntRect>())
	{
	}

	FVideoProducerBackBufferComposited::~FVideoProducerBackBufferComposited()
	{
		if (!IsEngineExitRequested())
		{
			AsyncTask(ENamedThreads::GameThread, [OnBackBufferReadyToPresentCopy = OnBackBufferReadyToPresentHandle, OnPreTickCopy = OnPreTickHandle]() {
				FSlateApplication& SlateApplication = FSlateApplication::Get();
				SlateApplication.GetRenderer()->OnBackBufferReadyToPresent().Remove(OnBackBufferReadyToPresentCopy);
				SlateApplication.OnPreTick().Remove(OnPreTickCopy);
			});
		}
	}

	void FVideoProducerBackBufferComposited::OnPreTick(float DeltaTime)
	{
		FScopeLock					Lock(&TopLevelWindowsCriticalSection);
		TArray<TSharedRef<SWindow>> TopLevelSlateWindows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(TopLevelSlateWindows);

		// We store all the necessary window information in structs. This prevents window information from updating
		// underneath us while we composite and also means we aren't holding on to any shared refs between compositions
		TArray<FTexturedWindow> TempWindows = TopLevelWindows;
		TopLevelWindows.Empty();

		for (TSharedRef<SWindow> CurrentWindow : TopLevelSlateWindows)
		{
			FVector2D WindowExtent = CurrentWindow->GetPositionInScreen() - CurrentWindow->GetSizeInScreen();

			// No need to keep track of "invalid" windows
			if (CurrentWindow->GetOpacity() == 0.f || CurrentWindow->GetSizeInScreen() == FVector2D(0, 0) || CurrentWindow->GetPositionInScreen().X > 16384 || CurrentWindow->GetPositionInScreen().X < -16384 || CurrentWindow->GetPositionInScreen().Y > 16384 || CurrentWindow->GetPositionInScreen().Y < -16384)
			{
				continue;
			}

			TopLevelWindows.Add(FTexturedWindow(CurrentWindow->GetPositionInScreen(), CurrentWindow->GetSizeInScreen(), CurrentWindow->GetOpacity(), CurrentWindow->GetType(), &CurrentWindow.Get()));

			// HACK (william.belcher): This following section is a bit of a nasty work-around to the fact that when a modal is displayed, previous windows (eg the editor) won't
			// trigger the OnBackBufferReady delegate. This means that we need to check if we have previously seen this top level window, and if we have we need to copy across
			// its staging texture just in case we won't see it again until after the modal is closed
			int32 Index = TempWindows.IndexOfByPredicate([&CurrentWindow](FTexturedWindow Window) {
				return Window.GetOwningWindow() == &CurrentWindow.Get();
			});

			if (Index != INDEX_NONE)
			{
				// This isn't the first time we've seen this window, so copy across its staging texture
				TopLevelWindows[TopLevelWindows.Num() - 1].SetTexture(TempWindows[Index].GetTexture());
			}
		}
	}

	void FVideoProducerBackBufferComposited::OnBackBufferReady(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
	{
		FRHITexture* FrameBuffer = ViewportProvider.GetBackBufferResource();

		/**
		 * When we receive a texture from this delegate, the texture will undergo a two copy process.
		 *
		 * The first copy performed in this function copies the texture we receive to the "Texture" member
		 * of the FWindow instance corresponding to the window provided. This is necessary as UE sometimes deletes
		 * before we have a chance to use them when compositing, so we need our own copy.
		 *
		 * The second copy is completed within CompositeWindows and applies a render pass to ensure format match
		 * (editor usually renders in RGB10A2 but WebRTC only supports RGBA8) before copying the texture to appropriate
		 * location in the composited frame.
		 *
		 * Finally, the composited frame is extracted from the RDG pipeline and we send it on its way through the PixelCapturer
		 *
		 */
		TSharedPtr<FVideoProducerUserData> UserData = MakeShared<FVideoProducerUserData>();
		UserData->ProducerName = TEXT("FVideoProducerBackBufferComposited");
		UserData->ProductionBeginCycles = FPlatformTime::Cycles64();

		{
			FScopeLock Lock(&TopLevelWindowsCriticalSection);
			if (TopLevelWindows.IsEmpty())
			{
				return;
			}

			// Find the index of the window that called this delegate in our array of windows + textures
			int32 Index = TopLevelWindows.IndexOfByPredicate([&SlateWindow](FTexturedWindow Window) { return Window.GetOwningWindow() == &SlateWindow; });

			if (Index == INDEX_NONE)
			{
				// Early out if we've received a texture without knowing if it's a part of GetAllVisibleWindowsOrdered()
				return;
			}

			{
				FRDGBuilder GraphBuilder(FRHICommandListImmediate::Get());

				// Register an external RDG texture from the provided frame buffer
				FRDGTextureRef InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(FrameBuffer, TEXT("VideoProducerBackBufferCompositedInput")));

				// Create an internal RDG texture with the same extent and format as the source
				FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(InputTexture->Desc.Extent, InputTexture->Desc.Format, FClearValueBinding::None, GetTexCreateFlags()), TEXT("VideoProducerBackBufferCompositedStaging"));
				// Bit cheeky, but when attempting to create two textures with the same description, RDG was just re-allocating which would lead to flickering. By converting to external, we force immediate allocation of the underlying pooled resource
				TopLevelWindows[Index].SetTexture(GraphBuilder.ConvertToExternalTexture(OutputTexture));

				AddCopyTexturePass(GraphBuilder, InputTexture, OutputTexture);

				GraphBuilder.Execute();
			}

			{
				// Check all of our windows have a texture
				bool bSkipComposite = TopLevelWindows.ContainsByPredicate([](FTexturedWindow Window) { return !Window.GetTexture(); });

				if (!bSkipComposite)
				{
					// All windows have a texture so we can composite
					CompositeWindows(UserData);
				}
			}
		}
	}

	void FVideoProducerBackBufferComposited::CompositeWindows(TSharedPtr<FVideoProducerUserData> UserData)
	{
		// Process all of the windows we will need to render. This processing step finds the extents of the
		// composited texture as well as the top-left point
		FIntPoint TopLeft = FIntPoint(MAX_int32, MAX_int32);
		FIntPoint BottomRight = FIntPoint(MIN_int32, MIN_int32);
		for (FTexturedWindow& CurrentWindow : TopLevelWindows)
		{
			FIntPoint TextureExtent = VectorMin(CurrentWindow.GetTexture()->GetDesc().Extent, CurrentWindow.GetSizeInScreen().IntPoint());
			FIntPoint WindowPosition = FIntPoint(CurrentWindow.GetPositionInScreen().X, CurrentWindow.GetPositionInScreen().Y);
			// Update the top left to be the element-wise minimum
			TopLeft = VectorMin(TopLeft, WindowPosition);
			// Update the bottom right to be the element-wise maximum
			BottomRight = VectorMax(BottomRight, (WindowPosition + TextureExtent));
		}

		// Shader globals used in the conversion pass
		FGlobalShaderMap*			 GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		int32										  ConversionOperation = 0; // None
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(ConversionOperation);

		TRefCountPtr<IPooledRenderTarget> OutTexture;
		{
			// FRDGBuilder uses a global allocator which can cause race conditions
			// To prevent issues its lifetime needs to end as soon as it has executed
			FRDGBuilder GraphBuilder(FRHICommandListImmediate::Get());

			// Clamp the texture dimensions to ensure no RHI crashes
			// Create an RDG texture that is the size of our extent for use as the composited frame
			FRDGTextureRef CompositedTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(VectorMin(BottomRight - TopLeft, FIntPoint(16384, 16384)), EPixelFormat::PF_B8G8R8A8, FClearValueBinding::None, GetTexCreateFlags()), TEXT("VideoProducerBackBufferCompositedCompositedTexture"));

			for (FTexturedWindow& CurrentWindow : TopLevelWindows)
			{
				FIntPoint WindowPosition = FIntPoint(CurrentWindow.GetPositionInScreen().X, CurrentWindow.GetPositionInScreen().Y) - TopLeft;

				FRHITexture* CurrentTexture = CurrentWindow.GetTexture()->GetRHI();

				FRDGTextureRef InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(CurrentTexture, TEXT("VideoProducerBackBufferCompositedStaging")));
				// There is only ever one tooltip and as such UE keeps the same texture for each and just rerenders the
				// content this can lead to small tooltips having a large texture from a previously displayed long tooltip
				// so we use the tooltips window size which is guaranteed to be correct
				FIntPoint Extent = VectorMin(CurrentTexture->GetDesc().Extent, CurrentWindow.GetSizeInScreen().IntPoint());

				// Ensure we have a valid extent (texture or window > 0,0)
				if (Extent.X == 0 || Extent.Y == 0)
				{
					continue;
				}

				// Configure our viewports appropriately
				FScreenPassTextureViewport InputViewport(InputTexture, FIntRect(FIntPoint(0, 0), Extent));
				FScreenPassTextureViewport OutputViewport(CompositedTexture, FIntRect(WindowPosition, WindowPosition + Extent));

				// Rectangle area to use from the source texture
				const FIntRect ViewRect(FIntPoint(0, 0), Extent);

				// Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
				FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
													   .SetTime(FGameTime()));
				FSceneViewInitOptions	ViewInitOptions;
				ViewInitOptions.ViewFamily = &ViewFamily;
				ViewInitOptions.SetViewRectangle(ViewRect);
				ViewInitOptions.ViewOrigin = FVector::ZeroVector;
				ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
				ViewInitOptions.ProjectionMatrix = FMatrix::Identity;

				GetRendererModule().CreateAndInitSingleView(GraphBuilder.RHICmdList, &ViewFamily, &ViewInitOptions);
				const FSceneView& View = *ViewFamily.Views[0];

				TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
				FModifyAlphaSwizzleRgbaPS::FParameters*	 PixelShaderParameters = GraphBuilder.AllocParameters<FModifyAlphaSwizzleRgbaPS::FParameters>();
				PixelShaderParameters->InputTexture = InputTexture;
				PixelShaderParameters->InputSampler = TStaticSamplerState<SF_Point>::GetRHI();
				PixelShaderParameters->RenderTargets[0] = FRenderTargetBinding{ CompositedTexture, ERenderTargetLoadAction::ELoad };

				// Add screen pass to convert whatever format the editor produces to BGRA8
				AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("VideoProducerBackBufferCompositedSwizzle"), View, OutputViewport, InputViewport, VertexShader, PixelShader, PixelShaderParameters);
			}
			
			// Final pass to extract the composited frames underlying RHI resource for passing to the rest of the pixel streaming pipeline
			// GraphBuilder.SetTextureAccessFinal(CompositedTexture, ERHIAccess::CopySrc);
			GraphBuilder.QueueTextureExtraction(CompositedTexture, &OutTexture);
			GraphBuilder.Execute();
		}

		UserData->ProductionEndCycles = FPlatformTime::Cycles64();
		PushFrame(FPixelCaptureInputFrameRHI(OutTexture->GetRHI(), UserData));

		// Update any subscribed streamers to let them know our composited frame size and position. This way it can correctly scale input from the browser
		*SharedFrameRect.Get() = FIntRect(TopLeft, BottomRight);
		OnFrameSizeChanged.Broadcast(SharedFrameRect);
	}

	FString FVideoProducerBackBufferComposited::ToString()
	{
		return TEXT("the Editor");
	}

} // namespace UE::EditorPixelStreaming2
