// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoProducerLevelEditor.h"

#include "Engine/GameViewportClient.h"
#include "IPixelStreaming2InputHandler.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "PixelCaptureInputFrameRHI.h"
#include "SLevelViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "UtilsAsync.h"

namespace UE::EditorPixelStreaming2
{
	TSharedPtr<FVideoProducerLevelEditor> FVideoProducerLevelEditor::Create()
	{
		TSharedPtr<FVideoProducerLevelEditor> NewInput = TSharedPtr<FVideoProducerLevelEditor>(new FVideoProducerLevelEditor());
		TWeakPtr<FVideoProducerLevelEditor> WeakInput = NewInput;

		UE::PixelStreaming2::DoOnGameThread([WeakInput]() {
			if (TSharedPtr<FVideoProducerLevelEditor> Input = WeakInput.Pin())
			{
				FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer();
				if(!Renderer)
				{
					return;
				}
 
				Renderer->OnBackBufferReadyToPresent().AddSP(Input.ToSharedRef(), &FVideoProducerLevelEditor::OnBackBufferReadyToPresent);
				FSlateApplication::Get().OnPreTick().AddSP(Input.ToSharedRef(), &FVideoProducerLevelEditor::OnPreTick);
			}
		});

		return NewInput;
	}

	FString FVideoProducerLevelEditor::ToString()
	{
		return TEXT("the Target Viewport");
	}

	bool FVideoProducerLevelEditor::ShouldCaptureViewport()
	{
		return true;
	}

	void FVideoProducerLevelEditor::OnPreTick(float DeltaTime)
	{
		if (!ShouldCaptureViewport())
		{
			return;
		}

		FLevelEditorModule&		   LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
		if (!ActiveLevelViewport.IsValid())
		{
			return;
		}

		TargetViewport = ActiveLevelViewport->GetViewportWidget();
		if (TargetViewport.IsValid())
		{
			TargetWindow = FSlateApplication::Get().FindWidgetWindow(TargetViewport.Pin().ToSharedRef());
			if (TargetWindow.IsValid())
			{
				CalculateCaptureRegion(TargetViewport.Pin().ToSharedRef(), TargetWindow.Pin().ToSharedRef());
			}
		}
	}

} // namespace UE::EditorPixelStreaming2