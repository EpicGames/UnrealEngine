// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PixelStreaming2SettingsEnums.h"
#include "PixelStreaming2Servers.h"

/**
 * Public interface that manages Pixel Streaming specific functionality within the Unreal Editor.
 * Provides various features for controlling Pixel Streaming, such as starting and stopping the streamer.
 */
class IPixelStreaming2EditorModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreaming2EditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreaming2EditorModule>("PixelStreaming2Editor");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("PixelStreaming2Editor"); }

	/**
	 * Starts the editor specific streamer.
	 *
	 * @param InStreamType The stream type for this streamer.
	 */
	virtual void StartStreaming(EPixelStreaming2EditorStreamTypes InStreamType) = 0;

	/**
	 * Starts the editor specific streamer.
	 *
	 */
	virtual void StopStreaming() = 0;

	/**
	 * Start the inbuilt C++ signalling server
	 *
	 */
	virtual void StartSignalling() = 0;

	/**
	 * Stop the inbuilt C++ signalling server
	 *
	 * @param bForce If true, will stop the server even if there are active streamers connected. Default is true.
	 */
	virtual void StopSignalling(bool bForce = true) = 0;

	/**
	 * Get the inbuilt C++ signalling server
	 *
	 * @return TSharedPtr<UE::PixelStreaming2Servers::IServer> The signalling server
	 */
	virtual TSharedPtr<UE::PixelStreaming2Servers::IServer> GetSignallingServer() = 0;

	/**
	 * Set the domain for the inbuilt c++ signalling server
	 *
	 * @param InSignallingDomain The domain for the inbuilt c++ signalling server
	 */
	virtual void SetSignallingDomain(const FString& InSignallingDomain) = 0;

	/**
	 * Get the domain for the inbuilt c++ signalling server
	 *
	 * @return FString The inbuilt c++ signalling server's domain
	 */
	virtual FString GetSignallingDomain() = 0;

	/**
	 * Set the port streamers connect to for the inbuilt c++ signalling server
	 *
	 * @param InStreamerPort The port streamers connect to for the inbuilt c++ signalling server
	 */
	virtual void SetStreamerPort(int32 InStreamerPort) = 0;

	/**
	 * Get the port streamers connect to for the inbuilt c++ signalling server
	 *
	 * @return int32 The inbuilt c++ signalling server's port for streamers connect to
	 */
	virtual int32 GetStreamerPort() = 0;

	/**
	 * Set the port viewers connect to for the inbuilt c++ signalling server
	 *
	 * @param InViewerPort The port viewers connect to for the inbuilt c++ signalling server
	 */
	virtual void SetViewerPort(int32 InViewerPort) = 0;

	/**
	 * Get the port viewers connect to for the inbuilt c++ signalling server
	 *
	 * @return int32 The inbuilt c++ signalling server's port for viewers connect to
	 */
	virtual int32 GetViewerPort() = 0;

	/**
	 * Set whether frontend content should be served of HTTPS for the inbuilt c++ signalling server
	 *
	 * @param bServeHttps
	 */
	virtual void SetServeHttps(bool bServeHttps) = 0;

	/**
	 * Get the bool indicating if frontend content is being served over HTTPS for the inbuilt c++ signalling server
	 *
	 * @return bool true if frontend content is being served over https, else false.
	 */
	virtual bool GetServeHttps() = 0;

	/**
	 * Set the path to file containing the SSL certificate. Required if frontend content is being served of HTTPS for the inbuilt c++ signalling server
	 *
	 * @param Path The path to file containing the SSL certificate.
	 */
	virtual void SetSSLCertificatePath(const FString& Path) = 0;

	/**
	 * Get the path to file containing the SSL certificate.
	 *
	 * @return FString The path to file containing the SSL certificate.
	 */
	virtual FString GetSSLCertificatePath() = 0;

	/**
	 * Set the path to file containing the SSL private key. Required if frontend content is being served of HTTPS for the inbuilt c++ signalling server
	 *
	 * @param Path The path to file containing the SSL private key.
	 */
	virtual void SetSSLPrivateKeyPath(const FString& Path) = 0;

	/**
	 * Get the path to file containing the SSL private key.
	 *
	 * @return FString The path to file containing the SSL private key.
	 */
	virtual FString GetSSLPrivateKeyPath() = 0;
};
