// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2HMD.h"

#include "Engine/GameViewportClient.h"
#include "GameFramework/WorldSettings.h"
#include "Logging.h"
#include "PixelStreaming2PluginSettings.h"
#include "SceneView.h"
#include "Widgets/SWindow.h"

FPixelStreaming2HMD::FPixelStreaming2HMD(const FAutoRegister& AutoRegister)
	: FHeadMountedDisplayBase(nullptr)
	, FHMDSceneViewExtension(AutoRegister)
	, CurHmdTransform(FTransform::Identity)
	, WorldToMeters(100.0f)
	, InterpupillaryDistance(0.0f)
	, bStereoEnabled(true)
{
}

FPixelStreaming2HMD::~FPixelStreaming2HMD()
{
}

void FPixelStreaming2HMD::SetEyeViews(FTransform Left, FMatrix LeftProj, FTransform Right, FMatrix RightProj, FTransform HMD)
{
	// This is our intialization message we can use this set the base position if it hasn't been set.
	if (BasePosition == FVector::ZeroVector)
	{
		SetBasePosition(HMD.GetLocation());
	}

	// Set the HMD transform
	SetTransform(HMD);

	// Make left and right relative the HMD
	FTransform HMDInv = HMD.Inverse();
	FTransform LeftRelative = Left * HMDInv;
	FTransform RightRelative = Right * HMDInv;

	// Undo rotation of HMD, then find relative positional offset between eyes and HMD
	LeftEyePosOffset = LeftRelative.GetLocation();
	RightEyePosOffset = RightRelative.GetLocation();

	float IPD = FVector::Dist(RightEyePosOffset, LeftEyePosOffset);
	// Set the IPD (in meters)
	SetInterpupillaryDistance(IPD / 100.0f);

	// Calculate left/right view orientation relative to HMD
	LeftEyeRotOffset = LeftRelative.GetRotation();
	RightEyeRotOffset = RightRelative.GetRotation();

	// Calculate the horizontal and vertical FoV from the projection matrix (left and right eye will have same FoVs)
	HFoVRads = 2.0f * FMath::Atan(1.0f / LeftProj.M[0][0]);
	VFoVRads = 2.0f * FMath::Atan(1.0f / LeftProj.M[1][1]);

	// Extract the left/right eye projection offsets
	CurLeftEyeProjOffsetX = -LeftProj.M[0][2];	 // 0.242512569
	CurLeftEyeProjOffsetY = -LeftProj.M[1][2];	 // 0.193187475
	CurRightEyeProjOffsetX = -RightProj.M[0][2]; // -0.242512569
	CurRightEyeProjOffsetY = -RightProj.M[1][2]; // 0.193187475

	// Extract near and farclip planes
	NearClip = LeftProj.M[3][2] / (LeftProj.M[2][2] - 1);
	FarClip = LeftProj.M[3][2] / (LeftProj.M[2][2] + 1);
	SetClippingPlanes(NearClip, FarClip);

	// Calculate target aspect ratio from the projection matrix (left and right eye will have same aspect ratio)
	TargetAspectRatio = tan(HFoVRads * 0.5f) / tan(VFoVRads * 0.5f);

	TSharedPtr<SWindow> TargetWindow = GEngine->GameViewport->GetWindow();
	FVector2f			SizeInScreen = TargetWindow->GetSizeInScreen();
	const float			InWidth = SizeInScreen.X / 2.f;
	const float			InHeight = SizeInScreen.Y;
	const float			AspectRatio = InWidth / InHeight;

	// If current resolution does not match remote device aspect ratio, we will change resolution to match aspect ratio (though we rate limit res change to every 5s)
	if (UPixelStreaming2PluginSettings::CVarHMDMatchAspectRatio.GetValueOnAnyThread() && FMath::Abs(AspectRatio - TargetAspectRatio) > 0.01 && !bReceivedTransforms)
	{
		int TargetHeight = InHeight;
		int TargetWidth = InHeight * TargetAspectRatio * 2.0f;
		UE_LOGF(LogPixelStreaming2HMD, Warning, "XR Pixel Streaming streaming resolution not matching remote device aspect ratio. Changing resolution to %dx%d", TargetWidth, TargetHeight);
		FString ChangeResCommand = FString::Printf(TEXT("r.SetRes %dx%d"), TargetWidth, TargetHeight);
		GEngine->Exec(GEngine->GetWorld(), *ChangeResCommand);
	}

	// If we know we are doing XR update some CVars for Pixel Streaming to optimise for it.
	if (!bReceivedTransforms)
	{
		// Couple engine's render rate and streaming rate
		UPixelStreaming2PluginSettings::CVarDecoupleFramerate->Set(false);

		// Set the rate at which we will stream
		UPixelStreaming2PluginSettings::CVarWebRTCFps->Set(90);

		// Set the MaxQuality to bound quality
		UPixelStreaming2PluginSettings::CVarEncoderMaxQuality->Set(70);

		// Necessary for coupled framerate
		UPixelStreaming2PluginSettings::CVarCaptureUseFence->Set(false);

		// Disable keyframes interval, only send them as needed
		UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval->Set(-1);
	}

	bReceivedTransforms = true;
}

float FPixelStreaming2HMD::GetWorldToMetersScale() const
{
	return WorldToMeters;
}

bool FPixelStreaming2HMD::IsHMDEnabled() const
{
	return UPixelStreaming2PluginSettings::CVarHMDEnable.GetValueOnAnyThread();
}

void FPixelStreaming2HMD::EnableHMD(bool enable)
{
	UPixelStreaming2PluginSettings::CVarHMDEnable->Set(enable, ECVF_SetByCode);
}

bool FPixelStreaming2HMD::GetHMDMonitorInfo(MonitorInfo& MonitorDesc)
{
	MonitorDesc.MonitorName = "PixelStreaming2HMD";
	MonitorDesc.MonitorId = 0;
	MonitorDesc.DesktopX = MonitorDesc.DesktopY = MonitorDesc.ResolutionX = MonitorDesc.ResolutionY = 0;
	return false;
}

void FPixelStreaming2HMD::GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const
{
	OutHFOVInDegrees = FMath::RadiansToDegrees(HFoVRads);
	OutVFOVInDegrees = FMath::RadiansToDegrees(VFoVRads);
}

bool FPixelStreaming2HMD::GetRelativeEyePose(int32 DeviceId, int32 ViewIndex, FQuat& OutOrientation, FVector& OutPosition)
{
	if (DeviceId != IXRTrackingSystem::HMDDeviceId)
	{
		return false;
	}

	if (UPixelStreaming2PluginSettings::CVarHMDApplyEyePosition.GetValueOnAnyThread())
	{
		// If not using override IPD get the actual translation of each eye from the HMD transform and apply that.
		OutPosition = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? LeftEyePosOffset : RightEyePosOffset;
	}

	// Apply eye rotation if this enabled (default: true)
	if (UPixelStreaming2PluginSettings::CVarHMDApplyEyeRotation.GetValueOnAnyThread())
	{
		OutOrientation = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? LeftEyeRotOffset : RightEyeRotOffset;
	}

	return false;
}

bool FPixelStreaming2HMD::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		OutDevices.Add(IXRTrackingSystem::HMDDeviceId);
		return true;
	}
	return false;
}

void FPixelStreaming2HMD::SetInterpupillaryDistance(float NewInterpupillaryDistance)
{
	InterpupillaryDistance = NewInterpupillaryDistance;
}

float FPixelStreaming2HMD::GetInterpupillaryDistance() const
{
	return InterpupillaryDistance;
}

bool FPixelStreaming2HMD::GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition)
{
	if (DeviceId != IXRTrackingSystem::HMDDeviceId)
	{
		return false;
	}
	CurrentOrientation = CurHmdTransform.GetRotation();
	CurrentPosition = CurHmdTransform.GetTranslation();
	return true;
}

bool FPixelStreaming2HMD::IsChromaAbCorrectionEnabled() const
{
	return false;
}

void FPixelStreaming2HMD::ResetOrientationAndPosition(float yaw)
{
	ResetOrientation(yaw);
	ResetPosition();
}

void FPixelStreaming2HMD::DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize)
{
	// Note: Left intentionally blank as we do not want to do any distortion on the UE side, the device will distort the image for us.
}

bool FPixelStreaming2HMD::IsStereoEnabled() const
{
	return bStereoEnabled;
}

bool FPixelStreaming2HMD::EnableStereo(bool stereo)
{
	bStereoEnabled = stereo;
	return bStereoEnabled;
}

void FPixelStreaming2HMD::AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const
{
	SizeX = SizeX / 2;
	X += SizeX * ViewIndex;
}

void FPixelStreaming2HMD::CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float InWorldToMeters, FVector& ViewLocation)
{
	if (ViewIndex == INDEX_NONE)
	{
		return;
	}

	float OverrideIPD = UPixelStreaming2PluginSettings::CVarHMDIPD.GetValueOnAnyThread();

	// If not received any transforms yet, just do default offset of half IPD
	if (!bReceivedTransforms)
	{
		float		IPDCentimeters = OverrideIPD > 0.0f ? OverrideIPD : InterpupillaryDistance * 100.0f;
		const float PassOffset = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? -IPDCentimeters * 0.5f : IPDCentimeters * 0.5f;
		ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(0, PassOffset, 0));
	}
	else
	{
		if (OverrideIPD > 0.0f)
		{
			// If using override IPD only translate along the horizontal plane.
			const float EyeTranslationOffset = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? -OverrideIPD * 0.5f : OverrideIPD * 0.5f;
			ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(0, EyeTranslationOffset, 0));
		}
		else if (UPixelStreaming2PluginSettings::CVarHMDApplyEyePosition.GetValueOnAnyThread())
		{
			// If not using override IPD get the actual translation of each eye from the HMD transform and apply that.
			ViewLocation += (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? ViewRotation.Quaternion().RotateVector(LeftEyePosOffset) : ViewRotation.Quaternion().RotateVector(RightEyePosOffset);
		}

		// Apply eye rotation if this enabled (default: true)
		if (UPixelStreaming2PluginSettings::CVarHMDApplyEyeRotation.GetValueOnAnyThread())
		{
			ViewRotation += (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? LeftEyeRotOffset.Rotator() : RightEyeRotOffset.Rotator();
		}
	}
}

FMatrix FPixelStreaming2HMD::GetStereoProjectionMatrix(const int32 ViewIndex) const
{

	float ProjOffsetX = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? CurLeftEyeProjOffsetX : CurRightEyeProjOffsetX;
	float ProjOffsetY = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? CurLeftEyeProjOffsetY : CurRightEyeProjOffsetY;

	// Check OverrideProjectOffset X & Y if they have been set by the user, use them instead of the values from WebXR
	{
		float OverrideProjectionOffsetX = UPixelStreaming2PluginSettings::CVarHMDProjectionOffsetX.GetValueOnAnyThread();
		float OverrideProjectionOffsetY = UPixelStreaming2PluginSettings::CVarHMDProjectionOffsetY.GetValueOnAnyThread();

		if (OverrideProjectionOffsetX >= 0.0f)
		{
			ProjOffsetX = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? OverrideProjectionOffsetX : -OverrideProjectionOffsetX;
		}
		if (OverrideProjectionOffsetY >= 0.0f)
		{
			ProjOffsetY = OverrideProjectionOffsetY;
		}
	}

	const float HFoVOverride = UPixelStreaming2PluginSettings::CVarHMDHFOV.GetValueOnAnyThread();
	const float VFoVOverride = UPixelStreaming2PluginSettings::CVarHMDVFOV.GetValueOnAnyThread();
	// FoV's are either passed in from the remote device or taken from the FoV override CVars.
	const float HalfVFov = VFoVOverride > 0.0f ? FMath::DegreesToRadians(VFoVOverride) * 0.5f : VFoVRads * 0.5f;
	const float HalfHFov = HFoVOverride > 0.0f ? FMath::DegreesToRadians(HFoVOverride) * 0.5f : HFoVRads * 0.5f;

	const float InNearZ = GNearClippingPlane;
	const float TanHalfHFov = tan(HalfHFov);
	const float TanHalfVFov = tan(HalfVFov);
	const float XS = 1.0f / TanHalfHFov;
	const float YS = 1.0f / TanHalfVFov;

	// Apply eye off-center translation
	const FTranslationMatrix OffCenterProjection = FTranslationMatrix(FVector(ProjOffsetX, ProjOffsetY, 0));
	const float				 ZNear = GNearClippingPlane_RenderThread;

	FMatrix ProjMatrix = FMatrix(
		FPlane(XS, 0.0f, 0.0f, 0.0f),
		FPlane(0.0f, YS, 0.0f, 0.0f),
		FPlane(0.0f, 0.0f, 0.0f, 1.0f),
		FPlane(0.0f, 0.0, ZNear, 0.0f));

	const FMatrix OutMatrix = ProjMatrix * OffCenterProjection;
	return OutMatrix;
}

void FPixelStreaming2HMD::GetEyeRenderParams_RenderThread(const FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const
{
	EyeToSrcUVOffsetValue = FVector2D::ZeroVector;
	EyeToSrcUVScaleValue = FVector2D(1.0f, 1.0f);
}

void FPixelStreaming2HMD::SetupViewFamily(FSceneViewFamily& InViewFamily)
{
	InViewFamily.EngineShowFlags.MotionBlur = 0;
	// Note: We do not want to apply any distortion on the UE side.
	InViewFamily.EngineShowFlags.HMDDistortion = false;
	InViewFamily.EngineShowFlags.StereoRendering = IsStereoEnabled();

	if (UWorld* World = GWorld)
	{
		WorldToMeters = World->GetWorldSettings()->WorldToMeters;
	}
}
