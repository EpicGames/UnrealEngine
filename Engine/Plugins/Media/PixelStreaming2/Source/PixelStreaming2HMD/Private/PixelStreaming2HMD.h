// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadMountedDisplayBase.h"
#include "IPixelStreaming2HMD.h"
#include "SceneViewExtension.h"

#define UE_API PIXELSTREAMING2HMD_API

class APlayerController;
class FSceneView;
class FSceneViewFamily;
class UCanvas;

/**
 * Pixel Streamed Head Mounted Display
 */
class FPixelStreaming2HMD : public IPixelStreaming2HMD, public FHeadMountedDisplayBase, public FHMDSceneViewExtension
{
public:
	// Begin IXRTrackingSystem
	virtual FName GetSystemName() const override
	{
		static FName DefaultName(TEXT("PixelStreaming2HMD"));
		return DefaultName;
	}

	int32			GetXRSystemFlags() const { return EXRSystemFlags::IsHeadMounted; }
	UE_API virtual bool	GetRelativeEyePose(int32 DeviceId, int32 ViewIndex, FQuat& OutOrientation, FVector& OutPosition) override;
	virtual void	SetBasePosition(const FVector& InBasePosition) override { BasePosition = InBasePosition; };
	virtual FVector GetBasePosition() const override { return BasePosition; }

	UE_API virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) override;

	UE_API virtual void  SetInterpupillaryDistance(float NewInterpupillaryDistance) override;
	UE_API virtual float GetInterpupillaryDistance() const override;

	UE_API virtual void ResetOrientationAndPosition(float yaw = 0.f) override;
	virtual void ResetOrientation(float Yaw = 0.f) override {}
	virtual void ResetPosition() override {}

	UE_API virtual bool	 GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition) override;
	virtual void	 SetBaseRotation(const FRotator& BaseRot) override {};
	virtual FRotator GetBaseRotation() const override { return FRotator::ZeroRotator; }

	virtual void  SetBaseOrientation(const FQuat& BaseOrient) override {}
	virtual FQuat GetBaseOrientation() const override { return FQuat::Identity; }

	virtual class IHeadMountedDisplay* GetHMDDevice() override { return this; }

	virtual class TSharedPtr<class IStereoRendering, ESPMode::ThreadSafe> GetStereoRenderingDevice() override
	{
		return SharedThis(this);
	}
	// End IXRTrackingSystem

protected:
	// Begin FXRTrackingSystemBase
	UE_API virtual float GetWorldToMetersScale() const override;
	// End FXRTrackingSystemBase

public:
	// Begin IHeadMountedDisplay
	virtual bool IsHMDConnected() override { return true; }
	UE_API virtual bool IsHMDEnabled() const override;
	UE_API virtual void EnableHMD(bool allow = true) override;
	UE_API virtual bool GetHMDMonitorInfo(MonitorInfo&) override;
	UE_API virtual void GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const override;
	UE_API virtual bool IsChromaAbCorrectionEnabled() const override;
	virtual bool GetHMDDistortionEnabled(EShadingPath ShadingPath) const override { return false; }
	UE_API virtual void DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize) override;
	// End IHeadMountedDisplay

	// Begin IStereoRendering
	UE_API virtual bool	IsStereoEnabled() const override;
	UE_API virtual bool	EnableStereo(bool stereo = true) override;
	UE_API virtual void	AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const override;
	UE_API virtual void	CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float InWorldToMeters, FVector& ViewLocation) override;
	UE_API virtual FMatrix GetStereoProjectionMatrix(const int32 ViewIndex) const override;
	UE_API virtual void	GetEyeRenderParams_RenderThread(const struct FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const override;
	// End IStereoRendering

	// Begin ISceneViewExtension
	UE_API virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override;
	// End ISceneViewExtension

	UE_API FPixelStreaming2HMD(const FAutoRegister&);

	UE_API virtual ~FPixelStreaming2HMD();

	/**
	 * @return	True if the HMD was initialized OK
	 */
	bool IsInitialized() const { return true; }

	virtual void SetTransform(FTransform Transform) override { CurHmdTransform = Transform; }
	UE_API virtual void SetEyeViews(FTransform Left, FMatrix LeftProj, FTransform Right, FMatrix RightProj, FTransform HMD) override;

private:
	FVector	   BasePosition = FVector::ZeroVector;
	FTransform CurHmdTransform;
	FVector	   LeftEyePosOffset;
	FVector	   RightEyePosOffset;
	FQuat	   LeftEyeRotOffset;
	FQuat	   RightEyeRotOffset;
	float	   WorldToMeters;
	float	   InterpupillaryDistance;
	float	   HFoVRads = FMath::DegreesToRadians(90.0f);
	float	   VFoVRads = FMath::DegreesToRadians(90.0f);
	float	   CurLeftEyeProjOffsetX = 0.0f;
	float	   CurLeftEyeProjOffsetY = 0.0f;
	float	   CurRightEyeProjOffsetX = 0.0f;
	float	   CurRightEyeProjOffsetY = 0.0f;
	float	   TargetAspectRatio = 9.0f / 16.0f;
	float	   NearClip = 10.0f;
	float	   FarClip = 10000.0f;
	bool	   bStereoEnabled;
	bool	   bReceivedTransforms = false;
	double	   LastResChangeSeconds = 0.0f;
};

#undef UE_API
