// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2HMDModule.h"

#include "PixelStreaming2HMD.h"
#include "PixelStreaming2PluginSettings.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

namespace UE::PixelStreaming2HMD
{
	/**
	 * IModuleInterface implementation
	 */
	void FPixelStreaming2HMDModule::StartupModule()
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
		ActiveXRSystem = EPixelStreaming2XRSystem::Unknown;
	}

	void FPixelStreaming2HMDModule::ShutdownModule()
	{
		// Remove the modules hold of the ptr
		// HMD.Reset();
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}
	/**
	 * End IModuleInterface implementation
	 */

	/**
	 *
	 */
	TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> FPixelStreaming2HMDModule::CreateTrackingSystem()
	{
		if (!UPixelStreaming2PluginSettings::CVarHMDEnable.GetValueOnAnyThread())
		{
			return nullptr;
		}

		TSharedRef<FPixelStreaming2HMD> PixelStreaming2HMD = FSceneViewExtensions::NewExtension<FPixelStreaming2HMD>();
		if (PixelStreaming2HMD->IsInitialized())
		{
			FGenericCrashContext::SetEngineData(TEXT("XR"), TEXT("XR PixelStreaming2"));
			return PixelStreaming2HMD;
		}
		return nullptr;
	}

	FPixelStreaming2HMD* FPixelStreaming2HMDModule::GetPixelStreaming2HMD() const
	{
		static FName SystemName(TEXT("PixelStreaming2HMD"));
		if (GEngine->XRSystem.IsValid() && (GEngine->XRSystem->GetSystemName() == SystemName))
		{
			return static_cast<FPixelStreaming2HMD*>(GEngine->XRSystem.Get());
		}

		return nullptr;
	}
} // namespace UE::PixelStreaming2HMD

IMPLEMENT_MODULE(UE::PixelStreaming2HMD::FPixelStreaming2HMDModule, PixelStreaming2HMD)