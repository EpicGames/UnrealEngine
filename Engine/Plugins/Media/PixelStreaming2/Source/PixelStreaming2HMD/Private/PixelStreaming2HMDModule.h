// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2HMDModule.h"
#include "PixelStreaming2HMD.h"

class IXRTrackingSystem;

namespace UE::PixelStreaming2HMD
{
	/**
	 * This module allows HMD input to be used with pixel streaming
	 */
	class FPixelStreaming2HMDModule : public IPixelStreaming2HMDModule
	{
	public:
		FPixelStreaming2HMD*	 GetPixelStreaming2HMD() const;
		EPixelStreaming2XRSystem GetActiveXRSystem() { return ActiveXRSystem; }
		void					 SetActiveXRSystem(EPixelStreaming2XRSystem System) { ActiveXRSystem = System; }

	private:
		// Begin IModuleInterface
		void StartupModule() override;
		void ShutdownModule() override;
		// End IModuleInterface

		// Begin IHeadMountedDisplayModule
		virtual TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> CreateTrackingSystem() override;
		FString													   GetModuleKeyName() const override { return FString(TEXT("PixelStreaming2HMD")); }
		bool													   IsHMDConnected() override { return true; }
		// End IHeadMountedDisplayModule

		TSharedPtr<FPixelStreaming2HMD, ESPMode::ThreadSafe> HMD;
		EPixelStreaming2XRSystem							 ActiveXRSystem;
	};
} // namespace UE::PixelStreaming2HMD