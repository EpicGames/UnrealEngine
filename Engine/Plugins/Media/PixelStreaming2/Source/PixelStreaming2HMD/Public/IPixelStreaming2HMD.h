// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

/**
 * Interface class that allows for setting the transform data for a Head Mounted Display (HMD).
 * The transform and projection matrix for each eye can be individually set.
 */
class IPixelStreaming2HMD
{
public:
	/**
	 * @brief Set the transform for the HMD.
	 * @param Transform The transform for the HMD to set.
	 */
	virtual void SetTransform(FTransform Transform) = 0;

	/**
	 * @brief Set the eye views, including the eye positions and their respective projection matrices
	 * @param Left The transform of the left eye relative to the HMD transform.
	 * @param LeftProj The Projection matrix of the left eye.
	 * @param Right The transform of the right eye relative to the HMD transform.
	 * @param RightProj The Projection matrix of the right eye.
	 * @param HMD The transform for the HMD to set.
	 */
	virtual void SetEyeViews(FTransform Left, FMatrix LeftProj, FTransform Right, FMatrix RightProj, FTransform HMD) = 0;
};
