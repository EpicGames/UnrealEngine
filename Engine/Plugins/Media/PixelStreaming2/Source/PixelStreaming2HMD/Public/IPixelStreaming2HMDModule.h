// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IHeadMountedDisplayModule.h"
#include "IPixelStreaming2HMD.h"
#include "Modules/ModuleManager.h"
#include "PixelStreaming2HMDEnums.h"

/**
 * The public interface provides functionality for interacting with Pixel Streaming and Head Mounted Display (HMD) systems.
 */
class IPixelStreaming2HMDModule : public IHeadMountedDisplayModule
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreaming2HMDModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreaming2HMDModule>("PixelStreaming2HMD");
	}

	/**
	 * Checks to see if this module is loaded.
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("PixelStreaming2HMD"); }

	/**
	 * @brief Get the Pixel Streaming HMD object.
	 * @return Pointer to the Pixel Streaming HMD.
	 */
	virtual IPixelStreaming2HMD* GetPixelStreaming2HMD() const = 0;

	/**
	 * @brief Get the Active XR System.
	 * @return The Active XR System.
	 */
	virtual EPixelStreaming2XRSystem GetActiveXRSystem() = 0;

	/**
	 * @brief Set the Active XR System.
	 * @param System The Active XR System to set.
	 */
	virtual void SetActiveXRSystem(EPixelStreaming2XRSystem System) = 0;
};
