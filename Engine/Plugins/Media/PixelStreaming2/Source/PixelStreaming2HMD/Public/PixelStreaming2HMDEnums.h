// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/**
 * Enums for different XR (Extended Reality) Systems that can be used with Pixel Streaming.
 */
enum class EPixelStreaming2XRSystem : uint8
{
	/** The XR system is unknown or not specified. */
	Unknown,
	/** HTC Vive XR system. */
	HTCVive,
	/** Oculus Quest XR system. */
	Quest,
};