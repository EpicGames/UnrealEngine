// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Slate/SceneViewport.h"
#include "IPixelStreaming2InputHandler.h"
#include "XRMotionControllerBase.h"
#include "PixelStreaming2HMDEnums.h"

#define UE_API PIXELSTREAMING2INPUT_API

namespace UE::PixelStreaming2Input
{
	class FPixelStreaming2InputHandler : public IPixelStreaming2InputHandler, public FXRMotionControllerBase
	{
	public:
		UE_API FPixelStreaming2InputHandler();
		UE_API virtual ~FPixelStreaming2InputHandler();

		UE_API virtual void Tick(float DeltaTime) override;

		// Poll for controller state and send events if needed
		virtual void SendControllerEvents() override {};

		// Set which MessageHandler will route input
		UE_API virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler) override;

		// Register a custom function to execute when command JSON is received.
		UE_API virtual void SetCommandHandler(const FString& CommandName, const CommandHandlerFn& Handler) override;

		UE_API virtual void SetElevatedCheck(const TFunction<bool(FString)>& CheckFn) override;
		UE_API virtual bool IsElevated(const FString& Id) override;

		UE_API virtual TSharedPtr<IPixelStreaming2DataProtocol> GetToStreamerProtocol() override;
		UE_API virtual TSharedPtr<IPixelStreaming2DataProtocol> GetFromStreamerProtocol() override;

		// Exec handler to allow console commands to be passed through for debugging
		UE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

		/**
		 * IInputInterface pass through functions
		 */
		UE_API virtual void				SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
		UE_API virtual void				SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values) override;
		UE_API virtual void				OnMessage(FString SourceId, TArray<uint8> Buffer) override;
		UE_API virtual void				SetTargetWindow(TWeakPtr<SWindow> InWindow) override;
		UE_API virtual TWeakPtr<SWindow>	GetTargetWindow() override;
		UE_API virtual void				SetTargetViewport(TWeakPtr<SViewport> InViewport) override;
		UE_API virtual TWeakPtr<SViewport> GetTargetViewport() override;
		UE_API virtual void				SetTargetScreenRect(TWeakPtr<FIntRect> InScreenRect) override;
		UE_API virtual TWeakPtr<FIntRect>	GetTargetScreenRect() override;
		virtual bool				IsFakingTouchEvents() const override { return bFakingTouchEvents; }
		UE_API virtual void				RegisterMessageHandler(const FString& MessageType, const MessageHandlerFn& Handler) override;
		UE_API virtual MessageHandlerFn	FindMessageHandler(const FString& MessageType) override;
		virtual void				SetInputType(EPixelStreaming2InputType InInputType) override { InputType = InInputType; };
		// IMotionController Interface
		UE_API virtual FName			GetMotionControllerDeviceTypeName() const override;
		UE_API virtual bool			GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const override;
		UE_API virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override;
		UE_API virtual void			EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const override;
		// End IMotionController Interface

		UE_API virtual bool OnKeyChar(TCHAR Character) override;
		UE_API virtual bool OnKeyDown(FKey Key, bool bIsRepeat) override;
		UE_API virtual bool OnKeyUp(FKey Key) override;

		UE_API virtual bool OnMouseEnter() override;
		UE_API virtual bool OnMouseLeave() override;
		UE_API virtual bool OnMouseDown(EMouseButtons::Type Button, FIntPoint ScreenPosition) override;
		UE_API virtual bool OnMouseUp(EMouseButtons::Type Button) override;
		UE_API virtual bool OnMouseMove(FIntPoint ScreenLocation, FIntPoint Delta) override;
		UE_API virtual bool OnMouseWheel(FIntPoint ScreenLocation, float MouseWheelDelta) override;
		UE_API virtual bool OnMouseDoubleClick(EMouseButtons::Type Button, FIntPoint ScreenPosition) override;

		UE_API virtual bool OnTouchStarted(FIntPoint TouchLocation, int32 TouchIndex, float Force) override;
		UE_API virtual bool OnTouchMoved(FIntPoint TouchLocation, int32 TouchIndex, float Force) override;
		UE_API virtual bool OnTouchEnded(FIntPoint TouchLocation, int32 TouchIndex) override;

		UE_API virtual uint8 OnControllerConnected() override;
		UE_API virtual bool  OnControllerAnalog(uint8 ControllerIndex, FKey Key, double AxisValue) override;
		UE_API virtual bool  OnControllerButtonPressed(uint8 ControllerIndex, FKey Key, bool bIsRepeat) override;
		UE_API virtual bool  OnControllerButtonReleased(uint8 ControllerIndex, FKey Key) override;
		UE_API virtual bool  OnControllerDisconnected(uint8 ControllerIndex) override;

		UE_API virtual bool OnXREyeViews(FTransform LeftEyeTransform, FMatrix LeftEyeProjectionMatrix, FTransform RightEyeTransform, FMatrix RightEyeProjectionMatrix, FTransform HMDTransform) override;
		UE_API virtual bool OnXRHMDTransform(FTransform HMDTransform) override;
		UE_API virtual bool OnXRControllerTransform(FTransform ControllerTransform, EControllerHand Handedness) override;
		UE_API virtual bool OnXRButtonTouched(EControllerHand Handedness, FKey Key, bool bIsRepeat) override;
		UE_API virtual bool OnXRButtonTouchReleased(EControllerHand Handedness, FKey Key) override;
		UE_API virtual bool OnXRButtonPressed(EControllerHand Handedness, FKey Key, bool bIsRepeat) override;
		UE_API virtual bool OnXRButtonReleased(EControllerHand Handedness, FKey Key) override;
		UE_API virtual bool OnXRAnalog(EControllerHand Handedness, FKey Key, double AnalogValue) override;
		UE_API virtual bool OnXRSystem(EPixelStreaming2XRSystem System) override;

	protected:
		UE_API FVector2D	ConvertToNormalizedScreenLocation(FVector2D Point);
		UE_API FIntPoint	ConvertFromNormalizedScreenLocation(const FVector2D& ScreenLocation, bool bIncludeOffset = true);
		UE_API FWidgetPath FindRoutingMessageWidget(const FVector2D& Location) const;
		UE_API FKey		TranslateMouseButtonToKey(const EMouseButtons::Type Button);

		struct FCachedTouchEvent
		{
			FVector2D Location;
			float	  Force;
			int32	  ControllerIndex;
		};

		struct FAnalogValue
		{
			/** The actual analog value from the controller axis, typical 0.0..1.0 */
			double Value;
			/** If value is non-zero then keep applying this analog values across frames.
			 * This is useful for trigger axis inputs where if a value is not transmitted
			 * UE will assume a gap in input means a full trigger press (which is not accurate if we were still pressing).
			 */
			bool bKeepUnlessZero = false;
			/** Has this key event already been fired once? */
			bool bIsRepeat = false;
		};

		// Keep a cache of the last touch events as we need to fire Touch Moved every frame while touch is down
		TMap<int32, FCachedTouchEvent> CachedTouchEvents;

		using FKeyId = uint8;
		/**
		 * If more values are received in a single tick (e.g. could be temp network issue),
		 * then we only forward the latest value.
		 *
		 * Reason: The input system seems to expect at most one raw analog value per FKey per Tick.
		 * If this is not done, the input system can get stuck on non-zero input value even if the user has
		 * already stopped moving the analog stick. It would stay stuck until the next time the user moves the stick.
		 *
		 * The values arrive in the order of recording: that means once the player releases the analog,
		 * the last analog value would be 0.
		 */
		TMap<FInputDeviceId, TMap<FKey, FAnalogValue>> AnalogEventsReceivedThisTick;

		/** Forwards the latest analog input received for each key this tick. */
		UE_API void ProcessLatestAnalogInputFromThisTick();

		// Track which touch events we processed this frame so we can avoid re-processing them
		TSet<int32> TouchIndicesProcessedThisFrame;

		// Sends Touch Moved events for any touch index which is currently down but wasn't already updated this frame
		UE_API void BroadcastActiveTouchMoveEvents();

		UE_API void FindFocusedWidget();

		/**
		 * Create an artificial mouse 'movement' to allow widgets to focus under a
		 * static mouse cursor. This will not actually change the cursor position.
		 */
		UE_API void SynthesizeMouseMove() const;

		TArray<FKey> FilteredKeys;
		UE_API void		 OnFilteredKeysChanged(IConsoleVariable* Var);
		UE_API bool		 FilterKey(const FKey& Key);

		struct FMessage
		{
			FString									 SourceId;
			TFunction<void(FString, FMemoryReader)>* Handler;
			TArray<uint8>							 Data;
		};

		TWeakPtr<SWindow>			  TargetWindow;
		TWeakPtr<SViewport>			  TargetViewport;
		TWeakPtr<FIntPoint>			  TargetScreenSize; // Deprecated functionality but remaining until it can be removed
		TWeakPtr<FIntRect>			  TargetScreenRect; // Manual size override used when we don't have a single window/viewport target
		uint8						  NumActiveTouches;
		bool						  bIsMouseActive;
		TQueue<FMessage>			  Messages;
		EPixelStreaming2InputType	  InputType = EPixelStreaming2InputType::RouteToWindow;
		FVector2D					  LastTouchLocation = FVector2D(EForceInit::ForceInitToZero);
		TMap<uint8, MessageHandlerFn> DispatchTable;

		/** Reference to the message handler which events should be passed to. */
		TSharedPtr<FGenericApplicationMessageHandler> MessageHandler;

		/** For convenience, we keep a reference to the application wrapper owned by the input channel */
		TSharedPtr<class FPixelStreaming2ApplicationWrapper> PixelStreamerApplicationWrapper;

		/**
		 * Is the application faking touch events by dragging the mouse along
		 * the canvas? If so then we must put the browser canvas in a special
		 * state to replicate the behavior of the application.
		 */
		bool bFakingTouchEvents;

		/**
		 * Padding for string parsing when handling messages.
		 * 1 character for the actual message and then
		 * 2 characters for the length which are skipped
		 */
		const size_t MessageHeaderOffset = 1;

		/**
		 * Touch only. A special position which indicates that no UI widget is
		 * focused.
		 */
		const FVector2D UnfocusedPos = FVector2D(-1.f, -1.f);

		/**
		 * Touch only. Location of the focused UI widget. If no UI widget is focused
		 * then this has the UnfocusedPos value.
		 */
		FVector2D FocusedPos;

		/**
		 * Whether an artificial mouse 'movement' should be created the next time
		 * a mouse button down event occurs. This allows a widget to be focused
		 * immediately when the browser window is focused.
		 */
		bool bSynthesizeMouseMoveForNextMouseDown = false;

		struct FPixelStreaming2XRController
		{
		public:
			FTransform		Transform;
			EControllerHand Handedness;
		};

		TMap<EControllerHand, FPixelStreaming2XRController> XRControllers;

		/**
		 * A map of named commands we respond to when we receive a datachannel message of type "command".
		 * Key = command name (e.g "Encoder.MaxQP")
		 * Value = The command handler lambda function whose parameters are as follows:
		 *  FString - the source id of the user who sent the message
		 *  FString - the descriptor (e.g. the full json payload of the command message)
		 *  FString - the parsed value of the command, e.g. if key was "Encoder.MaxQP" and descriptor was { type: "Command", "Encoder.MaxQP": 51 }, then parsed value is "51".
		 */
		TMap<FString, CommandHandlerFn> CommandHandlers;

		TFunction<bool(FString)> ElevatedCheck;

		TSharedPtr<IPixelStreaming2DataProtocol> ToStreamerProtocol;
		TSharedPtr<IPixelStreaming2DataProtocol> FromStreamerProtocol;

		float uint16_MAX = (float)UINT16_MAX;
		float int16_MAX = (float)SHRT_MAX;

	private:
		FDelegateHandle OnInputKeyFilterChangedHandle;
	};
} // namespace UE::PixelStreaming2Input

#undef UE_API
