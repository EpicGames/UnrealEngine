// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2InputMessage.h"
#include "IPixelStreaming2DataProtocol.h"
#include "PixelStreaming2InputEnums.h"

#define UE_API PIXELSTREAMING2INPUT_API

namespace UE::PixelStreaming2Input
{

	/**
	 * @brief An map type that broadcasts the OnProtocolUpdated whenever
	 * it's inner map is updated
	 */
	class FInputProtocolMap : public IPixelStreaming2DataProtocol
	{
	public:
		FInputProtocolMap(EPixelStreaming2MessageDirection InDirection)
			: Direction(InDirection) {}

		// Begin IPixelStreaming2DataProtocol interface
		UE_API TSharedPtr<IPixelStreaming2InputMessage> Add(FString Key) override;
		UE_API TSharedPtr<IPixelStreaming2InputMessage> Add(FString Key, TArray<EPixelStreaming2MessageTypes> InStructure) override;
		UE_API TSharedPtr<IPixelStreaming2InputMessage> Find(FString Key) override;
		FOnProtocolUpdated&						 OnProtocolUpdated() override { return OnProtocolUpdatedDelegate; };
		UE_API TSharedPtr<FJsonObject>					 ToJson() override;
		// End IPixelStreaming2DataProtocol interface

		UE_API bool										   AddInternal(FString Key, uint8 Id);
		UE_API bool										   AddInternal(FString Key, uint8 Id, TArray<EPixelStreaming2MessageTypes> InStructure);
		UE_API int											   Remove(FString Key);
		UE_API const TSharedPtr<IPixelStreaming2InputMessage> Find(FString Key) const;

		UE_API void Clear();
		UE_API bool IsEmpty() const;

		UE_API void Apply(const TFunction<void(FString, TSharedPtr<IPixelStreaming2InputMessage>)>& Visitor);

	private:
		UE_API TSharedPtr<IPixelStreaming2InputMessage> AddMessageInternal(FString Key, uint8 Id, TArray<EPixelStreaming2MessageTypes> InStructure);

	private:
		TSet<uint8>												Ids;
		TMap<FString, TSharedPtr<IPixelStreaming2InputMessage>> InnerMap;
		FOnProtocolUpdated										OnProtocolUpdatedDelegate;
		EPixelStreaming2MessageDirection						Direction;
		uint8													UserMessageId = 200;
	};

} // namespace UE::PixelStreaming2Input

#undef UE_API
