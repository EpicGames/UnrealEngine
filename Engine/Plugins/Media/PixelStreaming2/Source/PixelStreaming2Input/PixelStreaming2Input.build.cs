// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PixelStreaming2Input : ModuleRules
	{
		public PixelStreaming2Input(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Json",
					"SlateCore",
					"Slate",
					"DeveloperSettings",
					"HeadMountedDisplay",
					"PixelStreaming2Settings"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"Core",
					"InputCore",
					"InputDevice",
					"XRBase",
					"PixelStreaming2HMD"
				}
			);
		}
	}
}