// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApplicationWrapper.h"

#include "Logging.h"

/**
 * Macro to safely call a method on the wrapped application or return void if invalid.
 */
#define SAFE_CALL_WRAPPED_APPLICATION_VOID(Method, ...) \
	if (!WrappedApplication.IsValid()) \
	{ \
		UE_LOGFMT(LogPixelStreaming2Input, Warning, "{FUNCTIONNAME}\nFailed to call {METHODNAME}. Wrapped application is no longer valid", ("FUNCTIONNAME", __FUNCTION__), ("METHODNAME", #Method)); \
		return; \
	} \
	return WrappedApplication->Method(__VA_ARGS__);

/**
 * Macro to safely call a method on the wrapped application or return DefaultRetVal if invalid.
 */
#define SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(DefaultRetVal, Method, ...) \
	if (!WrappedApplication.IsValid()) \
	{ \
		UE_LOGFMT(LogPixelStreaming2Input, Warning, "{FUNCTIONNAME}\nFailed to call {METHODNAME}. Wrapped application is no longer valid", ("FUNCTIONNAME", __FUNCTION__), ("METHODNAME", #Method)); \
		return DefaultRetVal; \
	} \
	return WrappedApplication->Method(__VA_ARGS__);

namespace UE::PixelStreaming2Input
{
	FPixelStreaming2ApplicationWrapper::FPixelStreaming2ApplicationWrapper(TSharedPtr<GenericApplication> InWrappedApplication)
		: GenericApplication(MakeShareable(new FCursor()))
		, WrappedApplication(InWrappedApplication)
	{
		InitModifierKeys();

		OnEnginePreExitDelegateHandle = FCoreDelegates::OnEnginePreExit.AddRaw(this, &FPixelStreaming2ApplicationWrapper::OnEnginePreExit);
	}

	FPixelStreaming2ApplicationWrapper::~FPixelStreaming2ApplicationWrapper()
	{
		FCoreDelegates::OnEnginePreExit.Remove(OnEnginePreExitDelegateHandle);
	}

	void FPixelStreaming2ApplicationWrapper::OnEnginePreExit()
	{
		// Reset the wrapped application before engine exits to ensure we're not the one holding on to the platform application during destruction
		WrappedApplication.Reset();
	}

	void FPixelStreaming2ApplicationWrapper::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(SetMessageHandler, InMessageHandler)
	}

	void FPixelStreaming2ApplicationWrapper::PollGameDeviceState(const float TimeDelta)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(PollGameDeviceState, TimeDelta)
	}

	void FPixelStreaming2ApplicationWrapper::PumpMessages(const float TimeDelta)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(PumpMessages, TimeDelta)
	}

	void FPixelStreaming2ApplicationWrapper::ProcessDeferredEvents(const float TimeDelta)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(ProcessDeferredEvents, TimeDelta)
	}

	void FPixelStreaming2ApplicationWrapper::Tick(const float TimeDelta)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(Tick, TimeDelta)
	}

	TSharedRef<FGenericWindow> FPixelStreaming2ApplicationWrapper::MakeWindow()
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(MakeShared<FGenericWindow>(), MakeWindow)
	}

	void FPixelStreaming2ApplicationWrapper::InitializeWindow(const TSharedRef<FGenericWindow>& Window, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FGenericWindow>& InParent, const bool bShowImmediately)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(InitializeWindow, Window, InDefinition, InParent, bShowImmediately)
	}

	void FPixelStreaming2ApplicationWrapper::SetCapture(const TSharedPtr<FGenericWindow>& InWindow)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(SetCapture, InWindow)
	}

	void* FPixelStreaming2ApplicationWrapper::GetCapture(void) const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(nullptr, GetCapture)
	}

	void FPixelStreaming2ApplicationWrapper::SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(SetHighPrecisionMouseMode, Enable, InWindow)
	}

	bool FPixelStreaming2ApplicationWrapper::IsUsingHighPrecisionMouseMode() const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(false, IsUsingHighPrecisionMouseMode)
	}

	bool FPixelStreaming2ApplicationWrapper::IsUsingTrackpad() const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(false, IsUsingTrackpad)
	}

	bool FPixelStreaming2ApplicationWrapper::IsGamepadAttached() const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(false, IsGamepadAttached)
	}

	void FPixelStreaming2ApplicationWrapper::RegisterConsoleCommandListener(const FOnConsoleCommandListener& InListener)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(RegisterConsoleCommandListener, InListener)
	}

	void FPixelStreaming2ApplicationWrapper::AddPendingConsoleCommand(const FString& InCommand)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(AddPendingConsoleCommand, InCommand)
	}

	FPlatformRect FPixelStreaming2ApplicationWrapper::GetWorkArea(const FPlatformRect& CurrentWindow) const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(FPlatformRect(), GetWorkArea, CurrentWindow)
	}

	bool FPixelStreaming2ApplicationWrapper::TryCalculatePopupWindowPosition(const FPlatformRect& InAnchor, const FVector2D& InSize, const FVector2D& ProposedPlacement, const EPopUpOrientation::Type Orientation, /*OUT*/ FVector2D* const CalculatedPopUpPosition) const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(false, TryCalculatePopupWindowPosition, InAnchor, InSize, ProposedPlacement, Orientation, CalculatedPopUpPosition)
	}

	void FPixelStreaming2ApplicationWrapper::GetInitialDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) const
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(GetInitialDisplayMetrics, OutDisplayMetrics)
	}

	EWindowTitleAlignment::Type FPixelStreaming2ApplicationWrapper::GetWindowTitleAlignment() const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(EWindowTitleAlignment::Left, GetWindowTitleAlignment)
	}

	EWindowTransparency FPixelStreaming2ApplicationWrapper::GetWindowTransparencySupport() const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(EWindowTransparency::None, GetWindowTransparencySupport)
	}

	void FPixelStreaming2ApplicationWrapper::DestroyApplication()
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(DestroyApplication)
	}

	IInputInterface* FPixelStreaming2ApplicationWrapper::GetInputInterface()
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(nullptr, GetInputInterface)
	}

	ITextInputMethodSystem* FPixelStreaming2ApplicationWrapper::GetTextInputMethodSystem()
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(nullptr, GetTextInputMethodSystem)
	}
	
	void FPixelStreaming2ApplicationWrapper::SendAnalytics(IAnalyticsProvider* Provider)
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(SendAnalytics, Provider)
	}

	bool FPixelStreaming2ApplicationWrapper::SupportsSystemHelp() const
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(false, SupportsSystemHelp)
	}

	void FPixelStreaming2ApplicationWrapper::ShowSystemHelp()
	{
		SAFE_CALL_WRAPPED_APPLICATION_VOID(ShowSystemHelp)
	}

	bool FPixelStreaming2ApplicationWrapper::ApplicationLicenseValid(FPlatformUserId PlatformUser)
	{
		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(false, ApplicationLicenseValid, PlatformUser)
	}

	bool FPixelStreaming2ApplicationWrapper::IsMouseAttached() const
	{
		return true;
	}

	bool FPixelStreaming2ApplicationWrapper::IsCursorDirectlyOverSlateWindow() const
	{
		return true;
	}

	TSharedPtr<FGenericWindow> FPixelStreaming2ApplicationWrapper::GetWindowUnderCursor()
	{
		TSharedPtr<SWindow> Window = TargetWindow.Pin();
		if (Window.IsValid())
		{
			FVector2D CursorPosition = Cursor->GetPosition();
			FGeometry WindowGeometry = Window->GetWindowGeometryInScreen();

			FVector2D WindowOffset = WindowGeometry.GetAbsolutePosition();
			FVector2D WindowSize = WindowGeometry.GetAbsoluteSize();

			FBox2D WindowRect(WindowOffset, WindowSize);
			if (WindowRect.IsInside(CursorPosition))
			{
				return Window->GetNativeWindow();
			}
		}

		SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL(nullptr, GetWindowUnderCursor)
	}

	void FPixelStreaming2ApplicationWrapper::SetTargetWindow(TWeakPtr<SWindow> InTargetWindow)
	{
		TargetWindow = InTargetWindow;
	}

	/** Initialize the list of possible modifier keys. */
	void FPixelStreaming2ApplicationWrapper::InitModifierKeys()
	{
		ModifierKeys[EModifierKey::LeftShift].AgnosticKey = &EKeys::LeftShift;
		ModifierKeys[EModifierKey::RightShift].AgnosticKey = &EKeys::RightShift;
		ModifierKeys[EModifierKey::LeftControl].AgnosticKey = &EKeys::LeftControl;
		ModifierKeys[EModifierKey::RightControl].AgnosticKey = &EKeys::RightControl;
		ModifierKeys[EModifierKey::LeftAlt].AgnosticKey = &EKeys::LeftAlt;
		ModifierKeys[EModifierKey::RightAlt].AgnosticKey = &EKeys::RightAlt;
		ModifierKeys[EModifierKey::CapsLock].AgnosticKey = &EKeys::CapsLock;
	}

	/**
	 * When the user presses or releases a modifier key then update its state to
	 * active or back to inactive.
	 * @param InAgnosticKey - The key the user is pressing.
	 * @param bInActive - Whether the key is pressed (active) or released (inactive).
	 */
	void FPixelStreaming2ApplicationWrapper::UpdateModifierKey(const FKey* InAgnosticKey, bool bInActive)
	{
		for (int KeyIndex = EModifierKey::LeftShift; KeyIndex < EModifierKey::Count; KeyIndex++)
		{
			FModifierKey& ModifierKey = ModifierKeys[KeyIndex];
			if (ModifierKey.AgnosticKey == InAgnosticKey)
			{
				ModifierKey.bActive = bInActive;
				break;
			}
		}
	}

	/**
	 * Return the current set of active modifier keys.
	 * @return The current set of active modifier keys.
	 */
	FModifierKeysState FPixelStreaming2ApplicationWrapper::GetModifierKeys() const
	{
		FModifierKeysState ModifierKeysState(
			/*bInIsLeftShiftDown*/ ModifierKeys[EModifierKey::LeftShift].bActive,
			/*bInIsRightShiftDown*/ ModifierKeys[EModifierKey::RightShift].bActive,
			/*bInIsLeftControlDown*/ ModifierKeys[EModifierKey::LeftControl].bActive,
			/*bInIsRightControlDown*/ ModifierKeys[EModifierKey::RightControl].bActive,
			/*bInIsLeftAltDown*/ ModifierKeys[EModifierKey::LeftAlt].bActive,
			/*bInIsRightAltDown*/ ModifierKeys[EModifierKey::RightAlt].bActive,
			/*bInIsLeftCommandDown*/ false,
			/*bInIsRightCommandDown*/ false,
			/*bInAreCapsLocked*/ ModifierKeys[EModifierKey::CapsLock].bActive);
		return ModifierKeysState;
	}
} // namespace UE::PixelStreaming2Input

#undef SAFE_CALL_WRAPPED_APPLICATION_VOID
#undef SAFE_CALL_WRAPPED_APPLICATION_WITH_RETVAL