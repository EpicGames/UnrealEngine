// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplication.h"
#include "Widgets/SWindow.h"

namespace UE::PixelStreaming2Input
{
	/**
	 * Wrap the GenericApplication layer so we can replace the cursor and override
	 * certain behavior.
	 */
	class FPixelStreaming2ApplicationWrapper : public GenericApplication
	{
	public:
		FPixelStreaming2ApplicationWrapper(TSharedPtr<GenericApplication> InWrappedApplication);
		~FPixelStreaming2ApplicationWrapper();

		void OnEnginePreExit();

		/**
		 * Functions passed directly to the wrapped application.
		 */
		virtual void						SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
		virtual void						PollGameDeviceState(const float TimeDelta) override;
		virtual void						PumpMessages(const float TimeDelta) override;
		virtual void						ProcessDeferredEvents(const float TimeDelta) override;
		virtual void						Tick(const float TimeDelta) override;
		virtual TSharedRef<FGenericWindow>	MakeWindow() override;
		virtual void						InitializeWindow(const TSharedRef<FGenericWindow>& Window, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FGenericWindow>& InParent, const bool bShowImmediately) override;
		virtual void						SetCapture(const TSharedPtr<FGenericWindow>& InWindow) override;
		virtual void*						GetCapture(void) const override;
		virtual void						SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow) override;
		virtual bool						IsUsingHighPrecisionMouseMode() const override;
		virtual bool						IsUsingTrackpad() const override;
		virtual bool						IsGamepadAttached() const override;
		virtual void						RegisterConsoleCommandListener(const FOnConsoleCommandListener& InListener) override;
		virtual void						AddPendingConsoleCommand(const FString& InCommand) override;
		virtual FPlatformRect				GetWorkArea(const FPlatformRect& CurrentWindow) const override;
		virtual bool						TryCalculatePopupWindowPosition(const FPlatformRect& InAnchor, const FVector2D& InSize, const FVector2D& ProposedPlacement, const EPopUpOrientation::Type Orientation, /*OUT*/ FVector2D* const CalculatedPopUpPosition) const override;
		virtual void						GetInitialDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) const override;
		virtual EWindowTitleAlignment::Type GetWindowTitleAlignment() const override;
		virtual EWindowTransparency			GetWindowTransparencySupport() const override;
		virtual void						DestroyApplication() override;
		virtual IInputInterface*			GetInputInterface() override;
		virtual ITextInputMethodSystem*		GetTextInputMethodSystem() override;
		virtual void						SendAnalytics(IAnalyticsProvider* Provider) override;
		virtual bool						SupportsSystemHelp() const override;
		virtual void						ShowSystemHelp() override;
		virtual bool						ApplicationLicenseValid(FPlatformUserId PlatformUser = PLATFORMUSERID_NONE) override;

		/**
		 * Functions with overridden behavior.
		 */
		virtual bool					   IsMouseAttached() const override;
		virtual bool					   IsCursorDirectlyOverSlateWindow() const override;
		virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override;

		/**
		 * Custom functions
		 */
		virtual void SetTargetWindow(TWeakPtr<SWindow> InTargetWindow);

		TSharedPtr<GenericApplication> WrappedApplication;
		TWeakPtr<SWindow>			   TargetWindow;

		struct EModifierKey
		{
			enum Type
			{
				LeftShift,
				RightShift,
				LeftControl,
				RightControl,
				LeftAlt,
				RightAlt,
				CapsLock,
				Count
			};
		};

		struct FModifierKey
		{
			bool		bActive = false;
			const FKey* AgnosticKey;
		};

		/** Whether a particular modifier key, such as Ctrl, is pressed. */
		FModifierKey ModifierKeys[EModifierKey::Count];

		/** Initialize the list of possible modifier keys. */
		void InitModifierKeys();

		/**
		 * When the user presses or releases a modifier key then update its state to
		 * active or back to inactive.
		 * @param InAgnosticKey - The key the user is pressing.
		 * @param bInActive - Whether the key is pressed (active) or released (inactive).
		 */
		void UpdateModifierKey(const FKey* InAgnosticKey, bool bInActive);

		/**
		 * Return the current set of active modifier keys.
		 * @return The current set of active modifier keys.
		 */
		virtual FModifierKeysState GetModifierKeys() const;

	private:
		FDelegateHandle OnEnginePreExitDelegateHandle;
	};

	/**
	 * When reading input from a browser then the cursor position will be sent
	 * across with mouse events. We want to use this position and avoid getting the
	 * cursor position from the operating system. This is not relevant to touch
	 * events.
	 */
	class FCursor : public ICursor
	{
	public:
		FCursor() {}
		virtual ~FCursor() = default;
		virtual FVector2D		   GetPosition() const override { return Position; }
		virtual void			   SetPosition(const int32 X, const int32 Y) override { Position = FVector2D(X, Y); };
		virtual void			   SetType(const EMouseCursor::Type InNewCursor) override {};
		virtual EMouseCursor::Type GetType() const override { return EMouseCursor::Type::Default; };
		virtual void			   GetSize(int32& Width, int32& Height) const override {};
		virtual void			   Show(bool bShow) override {};
		virtual void			   Lock(const RECT* const Bounds) override {};
		virtual void			   SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override {};

	private:
		/** The cursor position sent across with mouse events. */
		FVector2D Position;
	};
} // namespace UE::PixelStreaming2Input