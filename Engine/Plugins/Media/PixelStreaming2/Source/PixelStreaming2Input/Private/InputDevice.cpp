// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputDevice.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/InputDeviceRegistry.h"

TSharedPtr<UE::PixelStreaming2Input::FInputDevice> UE::PixelStreaming2Input::FInputDevice::InputDevice;

namespace UE::PixelStreaming2Input
{
	TSharedPtr<FInputDevice> FInputDevice::GetInputDevice()
	{
		if (InputDevice)
		{
			return InputDevice;
		}

		InputDevice = TSharedPtr<FInputDevice>(new FInputDevice());
		return InputDevice;
	}

	FInputDevice::FInputDevice()
	{
		// This is imperative for editor streaming as when a modal is open or we've hit a BP breakpoint, the engine tick loop will not run, so instead we rely on this delegate to tick for us
		FSlateApplication::Get().OnPreTick().AddRaw(this, &FInputDevice::Tick);
	}

	void FInputDevice::AddInputHandler(IPixelStreaming2InputHandler* InputHandler)
	{
		InputHandlers.Add(InputHandler);
	}

	void FInputDevice::RemoveInputHandler(IPixelStreaming2InputHandler* InputHandler)
	{
		InputHandlers.Remove(InputHandler);
	}

	void FInputDevice::Tick(float DeltaTime)
	{
		for (IPixelStreaming2InputHandler* Handler : InputHandlers)
		{
			Handler->Tick(DeltaTime);
		}
	}

	void FInputDevice::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler)
	{
		for (IPixelStreaming2InputHandler* Handler : InputHandlers)
		{
			Handler->SetMessageHandler(InTargetHandler);
		}
	}

	bool FInputDevice::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		bool bRetVal = true;
		for (IPixelStreaming2InputHandler* Handler : InputHandlers)
		{
			bRetVal &= Handler->Exec(InWorld, Cmd, Ar);
		}
		return bRetVal;
	}

	void FInputDevice::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
	{
		for (IPixelStreaming2InputHandler* Handler : InputHandlers)
		{
			Handler->SetChannelValue(ControllerId, ChannelType, Value);
		}
	}

	void FInputDevice::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
	{
		for (IPixelStreaming2InputHandler* Handler : InputHandlers)
		{
			Handler->SetChannelValues(ControllerId, Values);
		}
	}

	uint8 FInputDevice::OnControllerConnected()
	{
		TArray<uint8> InputDeviceKeys;
		InputDevices.GenerateKeyArray(InputDeviceKeys);

		// Find the next available controller id. eg [0, 2, 3] will return 1
		uint8 NextControllerId = 0;
		while (InputDeviceKeys.Contains(NextControllerId))
		{
			NextControllerId++;
		}

		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId				DeviceId = DeviceMapper.AllocateNewInputDeviceId();
		FPlatformUserId				UserId = DeviceMapper.GetPlatformUserForNewlyConnectedDevice();
		{
			static const FName PixelStreamingInputDeviceName(TEXT("PixelStreaming2InputDevice"));
			FInputDeviceDescriptor Desc;
			Desc.HardwareDeviceHandle     = DeviceId;
			Desc.InputDeviceName          = PixelStreamingInputDeviceName;
			Desc.HardwareDeviceIdentifier = FName(TEXT("PixelStreaming2_Controller"));
			FInputDeviceRegistry::RegisterDevice(Desc);
		}
		DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, UserId, EInputDeviceConnectionState::Connected);

		InputDevices.Add(NextControllerId, { DeviceId, UserId });

		return NextControllerId;
	}

	void FInputDevice::OnControllerDisconnected(uint8 DeleteControllerId)
	{
		FInputDeviceId	DeviceId = INPUTDEVICEID_NONE;
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		if (!GetPlatformUserAndDevice(DeleteControllerId, DeviceId, PlatformUserId))
		{
			return;
		}
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, DeviceMapper.GetUserForUnpairedInputDevices(), EInputDeviceConnectionState::Disconnected);

		InputDevices.Remove(DeleteControllerId);
	}

	bool FInputDevice::GetPlatformUserAndDevice(uint8 ControllerId, FInputDeviceId& OutDeviceId, FPlatformUserId& OutPlatformUser)
	{
		if (!InputDevices.Contains(ControllerId))
		{
			OutDeviceId = INPUTDEVICEID_NONE;
			OutPlatformUser = PLATFORMUSERID_NONE;
			return false;
		}

		TPair<FInputDeviceId, FPlatformUserId> DevicePair = InputDevices[ControllerId];
		OutDeviceId = DevicePair.Get<0>();
		OutPlatformUser = DevicePair.Get<1>();
		return true;
	}

	bool FInputDevice::GetControllerIdFromDeviceId(FInputDeviceId DeviceId, uint8& OutControllerId)
	{
		for (TPair<uint8, TPair<FInputDeviceId, FPlatformUserId>> InputDevicePair : InputDevices)
		{
			if (DeviceId == InputDevicePair.Value.Get<0>())
			{
				OutControllerId = InputDevicePair.Key;
				return true;
			}
		}
		return false;
	}
} // namespace UE::PixelStreaming2Input
