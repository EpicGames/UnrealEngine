// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDevice.h"
#include "IPixelStreaming2InputHandler.h"

namespace UE::PixelStreaming2Input
{
	/**
	 * @brief The input device used to interface the multiple streamers and the single input device created by the OS
	 *
	 */
	class FInputDevice : public IInputDevice
	{
	public:
		void Tick(float DeltaTime) override;
		/** Poll for controller state and send events if needed */
		virtual void SendControllerEvents() override {};

		/** Set which MessageHandler will route input  */
		virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler) override;

		/** Exec handler to allow console commands to be passed through for debugging */
		virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
		virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
		virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values) override;

		void  AddInputHandler(IPixelStreaming2InputHandler* InputHandler);
		void  RemoveInputHandler(IPixelStreaming2InputHandler* InputHandler);
		uint8 OnControllerConnected();
		void  OnControllerDisconnected(uint8 DeleteControllerId);
		bool  GetPlatformUserAndDevice(uint8 ControllerId, FInputDeviceId& OutDeviceId, FPlatformUserId& OutPlatformUser);
		bool  GetControllerIdFromDeviceId(FInputDeviceId DeviceId, uint8& OutControllerId);

		static TSharedPtr<FInputDevice> GetInputDevice();

	private:
		FInputDevice();

		/**
		 * A singleton pointer to the input device. We only want a single input device that has multiple input handlers.
		 * The reason for a single input device is that only one is created by the application, so make sure we always use that one
		 *
		 */
		static TSharedPtr<FInputDevice> InputDevice;

		/**
		 * The array of input handlers. Each input handler belongs to a single streamer
		 *
		 * NOTE: We store each input handler as a raw pointer as registering and deregistering is handled in the FPixelStreaming2InputHandler ctor and dtor
		 * NOTE: This pointer is non-owning. It is up to user code to keep the original IPixelStreaming2InputHandler alive to avoid deregistration
		 */
		TSet<IPixelStreaming2InputHandler*> InputHandlers;

		/**
		 * The map of connected controllers. As each handler can have multiple input devices, we want to make sure that each controller of each device is unique
		 * As such, a simple incrementer approach is not applicable and we must instead keep track of all the connected controllers
		 *
		 */
		TMap<uint8, TPair<FInputDeviceId, FPlatformUserId>> InputDevices;
	};
} // namespace UE::PixelStreaming2Input