// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputHandler.h"

#include "ApplicationWrapper.h"
#include "PixelStreaming2InputStructures.h"
#include "Framework/Application/SlateApplication.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Input/HittestGrid.h"
#include "IPixelStreaming2HMDModule.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "InputDevice.h"
#include "Logging.h"
#include "PixelStreaming2PluginSettings.h"

#if WITH_EDITOR
	#include "UObject/UObjectGlobals.h"
#endif

// TODO: Gesture recognition is moving to the browser, so add handlers for the gesture events.
// The gestures supported will be swipe, pinch,

namespace UE::PixelStreaming2Input
{
	FPixelStreaming2InputHandler::FPixelStreaming2InputHandler()
		: TargetViewport(nullptr)
		, NumActiveTouches(0)
		, bIsMouseActive(false)
		, MessageHandler(FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler())
		, PixelStreamerApplicationWrapper(MakeShared<FPixelStreaming2ApplicationWrapper>(FSlateApplication::Get().GetPlatformApplication()))
		, FocusedPos(UnfocusedPos)

	{
		// Manually call OnFilteredKeysChanged to initially populate the FilteredKeys list
		OnFilteredKeysChanged(UPixelStreaming2PluginSettings::CVarInputKeyFilter.AsVariable());

		// Register this input handler as an IMotionController. The module handles the registering as an IInputDevice
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			OnInputKeyFilterChangedHandle = Delegates->OnInputKeyFilterChanged.AddRaw(this, &FPixelStreaming2InputHandler::OnFilteredKeysChanged);
		}
	}

	FPixelStreaming2InputHandler::~FPixelStreaming2InputHandler()
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnInputKeyFilterChanged.Remove(OnInputKeyFilterChangedHandle);
		}
	}

	TSharedPtr<IPixelStreaming2DataProtocol> FPixelStreaming2InputHandler::GetToStreamerProtocol()
	{
		return ToStreamerProtocol;
	}

	TSharedPtr<IPixelStreaming2DataProtocol> FPixelStreaming2InputHandler::GetFromStreamerProtocol()
	{
		return FromStreamerProtocol;
	}

	void FPixelStreaming2InputHandler::RegisterMessageHandler(const FString& MessageType, const MessageHandlerFn& Handler)
	{
		TSharedPtr<IPixelStreaming2InputMessage> Message = ToStreamerProtocol->Find(MessageType);
		if (Message)
		{
			DispatchTable.Add(Message->GetID(), Handler);
		}
		else
		{
			UE_LOGF(LogPixelStreaming2Input, Error, "No message type called '%ls' was found in ToStreamer protocol", *MessageType);
		}
	}

	IPixelStreaming2InputHandler::MessageHandlerFn FPixelStreaming2InputHandler::FindMessageHandler(const FString& MessageType)
	{
		return DispatchTable.FindRef(ToStreamerProtocol->Find(MessageType)->GetID());
	}

	FName FPixelStreaming2InputHandler::GetMotionControllerDeviceTypeName() const
	{
		return FName(TEXT("PixelStreaming2XRController"));
	}

	bool FPixelStreaming2InputHandler::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const
	{
		if (IPixelStreaming2HMD* HMD = IPixelStreaming2HMDModule::Get().GetPixelStreaming2HMD(); (HMD == nullptr || ControllerIndex == INDEX_NONE))
		{
			return false;
		}

		EControllerHand DeviceHand;
		if (GetHandEnumForSourceName(MotionSource, DeviceHand))
		{
			FPixelStreaming2XRController Controller = XRControllers.FindRef(DeviceHand);
			OutOrientation = Controller.Transform.Rotator();
			OutPosition = Controller.Transform.GetTranslation();
			return true;
		}
		return false;
	}

	ETrackingStatus FPixelStreaming2InputHandler::GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const
	{
		EControllerHand DeviceHand;
		if (GetHandEnumForSourceName(MotionSource, DeviceHand))
		{
			const FPixelStreaming2XRController* Controller = XRControllers.Find(DeviceHand);
			return (Controller != nullptr) ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;
		}
		return ETrackingStatus::NotTracked;
	}

	void FPixelStreaming2InputHandler::EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const
	{
		SourcesOut.Add(FName(TEXT("AnyHand")));
		SourcesOut.Add(FName(TEXT("Left")));
		SourcesOut.Add(FName(TEXT("Right")));
		SourcesOut.Add(FName(TEXT("LeftGrip")));
		SourcesOut.Add(FName(TEXT("RightGrip")));
		SourcesOut.Add(FName(TEXT("LeftAim")));
		SourcesOut.Add(FName(TEXT("RightAim")));
	}

	void FPixelStreaming2InputHandler::Tick(const float InDeltaTime)
	{
#if WITH_EDITOR
		/* No routing input while saving ... this is relevant for auto-save and can cause an incredibly rare crash...
		 *
		 * The gist is that the auto-save system calls FSlateApplication::Tick(), which executes its OnPreTick() containing
		 * our FPixelStreaming2InputHandler::Tick. Routing any input executes Slate delegates. Again, the gist is that
		 * the delegates can do anything including calling StaticConstructObject(), which will crash the editor
		 * ("Illegal call to StaticConstructObject() while serializing object data!").
		 * An example of a StaticConstructObject call is a UMG widget calling CreateWidget in response to a button's OnClick (which we routed!).
		 *
		 * If you're curious why our Tick gets called by auto-save:
		 * The auto save starts in FPackageAutoSaver::AttemptAutoSave, which calls FEditorFileUtils::AutosaveMapEx.
		 * This causes the world package to be saved (UEditorEngine::SavePackage) with a FSlowTask.
		 * The slow task calls FFeedbackContextEditor::ProgressReported... which ticks slate so the progres bar modal window updates.
		 * Consult with FInputDevice::FInputDevice, which explicitly wants to tick when a modal window is open.
		 *
		 * TLDR: if we're auto-saving, we'll postbone routing input until the auto save is done.
		 */
		if (UE::IsSavingPackage())
		{
			return;
		}
#endif

		TouchIndicesProcessedThisFrame.Reset();

		FMessage Message;
		while (Messages.Dequeue(Message))
		{
			FMemoryReader Ar(Message.Data);
			(*Message.Handler)(Message.SourceId, Ar);
		}

		ProcessLatestAnalogInputFromThisTick();
		BroadcastActiveTouchMoveEvents();
	}

	void FPixelStreaming2InputHandler::OnMessage(FString SourceId, TArray<uint8> Buffer)
	{
		if (Buffer.Num() == 0)
		{
			return;
		}
		
		uint8 MessageType = Buffer[0];
		// Remove the message type. The remaining data in the buffer is now purely
		// the message data
		Buffer.RemoveAt(0);

		TFunction<void(FString, FMemoryReader)>* Handler = DispatchTable.Find(MessageType);
		if (Handler != nullptr)
		{
			FMessage Message = {
				SourceId, // Who sent this message
				Handler,  // The function to call
				Buffer	  // The message data
			};
			Messages.Enqueue(Message);
		}
		else
		{
			UE_LOGF(LogPixelStreaming2Input, Warning, "No handler registered for message with id %d", MessageType);
		}
	}

	void FPixelStreaming2InputHandler::SetTargetWindow(TWeakPtr<SWindow> InWindow)
	{
		TargetWindow = InWindow;
		PixelStreamerApplicationWrapper->SetTargetWindow(InWindow);
	}

	TWeakPtr<SWindow> FPixelStreaming2InputHandler::GetTargetWindow()
	{
		return TargetWindow;
	}

	void FPixelStreaming2InputHandler::SetTargetScreenRect(TWeakPtr<FIntRect> InScreenRect)
	{
		TargetScreenRect = InScreenRect;
	}

	TWeakPtr<FIntRect> FPixelStreaming2InputHandler::GetTargetScreenRect()
	{
		return TargetScreenRect;
	}

	void FPixelStreaming2InputHandler::SetTargetViewport(TWeakPtr<SViewport> InViewport)
	{
		TargetViewport = InViewport;
	}

	TWeakPtr<SViewport> FPixelStreaming2InputHandler::GetTargetViewport()
	{
		return TargetViewport;
	}

	void FPixelStreaming2InputHandler::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InTargetHandler)
	{
		MessageHandler = InTargetHandler;
	}

	bool FPixelStreaming2InputHandler::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		return GEngine->Exec(InWorld, Cmd, Ar);
	}

	void FPixelStreaming2InputHandler::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
	{
		// TODO: Implement FFB
	}

	void FPixelStreaming2InputHandler::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values)
	{
		// TODO: Implement FFB
	}

	bool FPixelStreaming2InputHandler::OnKeyChar(TCHAR Character)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "KEY_PRESSED: Character = '%c'", Character);
		// A key char event is never repeated, so set it to false. It's value
		// ultimately doesn't matter as this paramater isn't used later
		return MessageHandler->OnKeyChar(Character, false);
	}

	bool FPixelStreaming2InputHandler::OnKeyDown(FKey Key, bool bIsRepeat)
	{
		const uint32* KeyPtr;
		const uint32* CharacterPtr;
		FInputKeyManager::Get().GetCodesFromKey(Key, KeyPtr, CharacterPtr);
		uint32 KeyCode = KeyPtr ? *KeyPtr : 0;
		uint32 Character = CharacterPtr ? *CharacterPtr : 0;

		PixelStreamerApplicationWrapper->UpdateModifierKey(&Key, true);
		UE_LOGF(LogPixelStreaming2Input, Verbose, "KEY_DOWN: Key = %d; Character = %d; IsRepeat = %ls", KeyCode, Character, bIsRepeat ? TEXT("True") : TEXT("False"));
		return MessageHandler->OnKeyDown((int32)KeyCode, (int32)Character, bIsRepeat);
	}

	bool FPixelStreaming2InputHandler::OnKeyUp(FKey Key)
	{
		const uint32* KeyPtr;
		const uint32* CharacterPtr;
		FInputKeyManager::Get().GetCodesFromKey(Key, KeyPtr, CharacterPtr);
		uint32 KeyCode = KeyPtr ? *KeyPtr : 0;
		uint32 Character = CharacterPtr ? *CharacterPtr : 0;

		PixelStreamerApplicationWrapper->UpdateModifierKey(&Key, false);
		UE_LOGF(LogPixelStreaming2Input, Verbose, "KEY_UP: Key = %d; Character = %d", KeyCode, Character);
		return MessageHandler->OnKeyUp((int32)KeyCode, (int32)Character, false);
	}

	bool FPixelStreaming2InputHandler::OnTouchStarted(FIntPoint TouchLocation, int32 TouchIndex, float TouchForce)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "TOUCH_START: TouchIndex = %d; CursorPos = (%d, %d); Force = %.3f", TouchIndex, static_cast<int>(TouchLocation.X), static_cast<int>(TouchLocation.Y), TouchForce);

		bool bHandled = false;

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			// TouchLocation = TouchLocation - TargetViewport.Pin()->GetCachedGeometry().GetAbsolutePosition();
			FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				FPointerEvent		   PointerEvent(0, TouchIndex, TouchLocation, TouchLocation, TouchForce, true);
				bHandled = FSlateApplication::Get().RoutePointerDownEvent(WidgetPath, PointerEvent).IsEventHandled();
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			if (NumActiveTouches == 0 && !bIsMouseActive)
			{
				FSlateApplication::Get().OnCursorSet();
				// Make sure the application is active.
				FSlateApplication::Get().ProcessApplicationActivationEvent(true);
				// Default to the current touch location so that if the wrapped application is invalid, we just don't move the cursor
				FVector2D OldCursorLocation = FVector2D(TouchLocation.X, TouchLocation.Y);
				if (PixelStreamerApplicationWrapper->WrappedApplication.IsValid())
				{
					OldCursorLocation = PixelStreamerApplicationWrapper->WrappedApplication->Cursor->GetPosition();	
				} 
				else
				{
					UE_LOGF(LogPixelStreaming2Input, Warning, "Wrapped application is no longer valid");
				}
				PixelStreamerApplicationWrapper->Cursor->SetPosition(OldCursorLocation.X, OldCursorLocation.Y);
				FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper);
			}

			// We must update the user cursor position explicitly before updating the application cursor position
			// as if there's a delta between them, when the touch event is started it will trigger a move
			// resulting in a large 'drag' across the screen
			TSharedPtr<FSlateUser> User = FSlateApplication::Get().GetCursorUser();
			User->SetCursorPosition(TouchLocation);
			PixelStreamerApplicationWrapper->Cursor->SetPosition(TouchLocation.X, TouchLocation.Y);
			if (PixelStreamerApplicationWrapper->WrappedApplication.IsValid())
			{
				PixelStreamerApplicationWrapper->WrappedApplication->Cursor->SetPosition(TouchLocation.X, TouchLocation.Y);
			} 
			else
			{
				UE_LOGF(LogPixelStreaming2Input, Warning, "Wrapped application is no longer valid");
			}

			bHandled = MessageHandler->OnTouchStarted(PixelStreamerApplicationWrapper->GetWindowUnderCursor(), TouchLocation, TouchForce, TouchIndex, 0); // TODO: ControllerId?
		}

		NumActiveTouches++;

		FindFocusedWidget();

		return bHandled;
	}

	bool FPixelStreaming2InputHandler::OnTouchMoved(FIntPoint TouchLocation, int32 TouchIndex, float TouchForce)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "TOUCH_MOVE: TouchIndex = %d; CursorPos = (%d, %d); Force = %.3f", TouchIndex, static_cast<int>(TouchLocation.X), static_cast<int>(TouchLocation.Y), TouchForce);

		FCachedTouchEvent& TouchEvent = CachedTouchEvents.FindOrAdd(TouchIndex);
		TouchEvent.Force = TouchForce;
		TouchEvent.ControllerIndex = 0;

		bool bHandled = false;

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			// TouchLocation = TouchLocation - TargetViewport.Pin()->GetCachedGeometry().GetAbsolutePosition();
			TouchEvent.Location = TouchLocation;
			FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				FPointerEvent		   PointerEvent(0, TouchIndex, TouchLocation, LastTouchLocation, TouchForce, true);
				bHandled = FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, false);
			}

			LastTouchLocation = TouchLocation;
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			TouchEvent.Location = TouchLocation;
			bHandled = MessageHandler->OnTouchMoved(TouchEvent.Location, TouchEvent.Force, TouchIndex, TouchEvent.ControllerIndex); // TODO: ControllerId?
		}

		TouchIndicesProcessedThisFrame.Add(TouchIndex);

		return bHandled;
	}

	bool FPixelStreaming2InputHandler::OnTouchEnded(FIntPoint TouchLocation, int32 TouchIndex)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "TOUCH_END: TouchIndex = %d; CursorPos = (%d, %d)", TouchIndex, static_cast<int>(TouchLocation.X), static_cast<int>(TouchLocation.Y));

		bool bHandled = false;

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			// TouchLocation = TouchLocation - TargetViewport.Pin()->GetCachedGeometry().GetAbsolutePosition();
			FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				float				   TouchForce = 0.0f;
				FPointerEvent		   PointerEvent(0, TouchIndex, TouchLocation, TouchLocation, TouchForce, true);
				bHandled = FSlateApplication::Get().RoutePointerUpEvent(WidgetPath, PointerEvent).IsEventHandled();
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			bHandled = MessageHandler->OnTouchEnded(TouchLocation, TouchIndex, 0); // TODO: ControllerId?
		}

		CachedTouchEvents.Remove(TouchIndex);
		NumActiveTouches = (NumActiveTouches > 0) ? NumActiveTouches - 1 : NumActiveTouches;

		// If there's no remaining touches, and there is also no mouse over the player window
		// then set the platform application back to its default. We need to set it back to default
		// so that people using the editor (if editor streaming) can click on buttons outside the target window
		// and also have the correct cursor (pixel streaming forces default cursor)
		if (NumActiveTouches == 0 && !bIsMouseActive && InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			FVector2D OldCursorLocation = PixelStreamerApplicationWrapper->Cursor->GetPosition();
			if (PixelStreamerApplicationWrapper->WrappedApplication.IsValid())
			{
				PixelStreamerApplicationWrapper->WrappedApplication->Cursor->SetPosition(OldCursorLocation.X, OldCursorLocation.Y);
				FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper->WrappedApplication);
			} 
			else
			{
				UE_LOGF(LogPixelStreaming2Input, Warning, "Wrapped application is no longer valid");
			}
		}

		return bHandled;
	}

	uint8 FPixelStreaming2InputHandler::OnControllerConnected()
	{
		uint8 NextControllerId = FInputDevice::GetInputDevice()->OnControllerConnected();

		// When a virtual controller (from the browser) is "connected" into UE's input system,
		// it creates and id. That id is used to differentitate each controller used.
		// We must inform the browser of the id that was generated for the controller, so we send:
		// { "controllerId:" 1 // the id here }

		FString Descriptor = FString::Printf(TEXT("{ \"controllerId\": %d }"), NextControllerId);

		FBufferArchive Buffer;
		Buffer << Descriptor;
		TArray<uint8> Data(Buffer.GetData(), Buffer.Num());
		// Specific implementation for this method is handled per streamer
		OnSendMessage.Broadcast(EPixelStreaming2FromStreamerMessage::GamepadResponse, FMemoryReader(Data));

		UE_LOGF(LogPixelStreaming2Input, Verbose, "GAMEPAD_CONNECTED: ControllerId = %d", NextControllerId);

		return NextControllerId;
	}

	bool FPixelStreaming2InputHandler::OnControllerAnalog(uint8 ControllerIndex, FKey Key, double AxisValue)
	{
		FInputDeviceId	DeviceId = INPUTDEVICEID_NONE;
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		if (!FInputDevice::GetInputDevice()->GetPlatformUserAndDevice(ControllerIndex, DeviceId, PlatformUserId))
		{
			return false;
		}

		FAnalogValue AnalogValue;
		AnalogValue.Value = AxisValue;
		// Only send axes values continuously in the case of gamepad triggers
		AnalogValue.bKeepUnlessZero = (Key == EKeys::Gamepad_LeftTriggerAxis || Key == EKeys::Gamepad_RightTriggerAxis);

		// Overwrite the last data: every tick only process the latest
		AnalogEventsReceivedThisTick.FindOrAdd(DeviceId).FindOrAdd(Key) = AnalogValue;

		return true;
	}

	bool FPixelStreaming2InputHandler::OnControllerButtonPressed(uint8 ControllerIndex, FKey Key, bool bIsRepeat)
	{
		FInputDeviceId	DeviceId = INPUTDEVICEID_NONE;
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		if (!FInputDevice::GetInputDevice()->GetPlatformUserAndDevice(ControllerIndex, DeviceId, PlatformUserId))
		{
			return false;
		}

		UE_LOGF(LogPixelStreaming2Input, Verbose, "GAMEPAD_PRESSED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls;", ControllerIndex, *Key.ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));

		return MessageHandler->OnControllerButtonPressed(Key.GetFName(), PlatformUserId, DeviceId, bIsRepeat);
	}

	bool FPixelStreaming2InputHandler::OnControllerButtonReleased(uint8 ControllerIndex, FKey Key)
	{
		FInputDeviceId	DeviceId = INPUTDEVICEID_NONE;
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		if (!FInputDevice::GetInputDevice()->GetPlatformUserAndDevice(ControllerIndex, DeviceId, PlatformUserId))
		{
			return false;
		}

		UE_LOGF(LogPixelStreaming2Input, Verbose, "GAMEPAD_RELEASED: ControllerId = %d; KeyName = %ls;", ControllerIndex, *Key.ToString());

		return MessageHandler->OnControllerButtonReleased(Key.GetFName(), PlatformUserId, DeviceId, false);
	}

	bool FPixelStreaming2InputHandler::OnControllerDisconnected(uint8 ControllerIndex)
	{
		FInputDevice::GetInputDevice()->OnControllerDisconnected(ControllerIndex);

		UE_LOGF(LogPixelStreaming2Input, Verbose, "GAMEPAD_DISCONNECTED: ControllerId = %d", ControllerIndex);

		return true;
	}

	/**
	 * Mouse events
	 */
	bool FPixelStreaming2InputHandler::OnMouseEnter()
	{
		if (NumActiveTouches == 0 && !bIsMouseActive)
		{
			FSlateApplication::Get().OnCursorSet();
			FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper);
			// Make sure the application is active.
			FSlateApplication::Get().ProcessApplicationActivationEvent(true);
		}

		bIsMouseActive = true;
		UE_LOG(LogPixelStreaming2Input, Verbose, TEXT("MOUSE_ENTER"));
		return true;
	}

	bool FPixelStreaming2InputHandler::OnMouseLeave()
	{
		if (NumActiveTouches == 0)
		{
			if (PixelStreamerApplicationWrapper->WrappedApplication.IsValid())
			{
				// Restore normal application layer if there is no active touches and MouseEnter hasn't been triggered
				FSlateApplication::Get().OverridePlatformApplication(PixelStreamerApplicationWrapper->WrappedApplication);
			} 
			else
			{
				UE_LOGF(LogPixelStreaming2Input, Warning, "Wrapped application is no longer valid");
			}
		}
		bIsMouseActive = false;
		UE_LOGF(LogPixelStreaming2Input, Verbose, "MOUSE_LEAVE");
		return true;
	}

	bool FPixelStreaming2InputHandler::OnMouseUp(EMouseButtons::Type Button)
	{
		// Ensure we have wrapped the slate application at this point
		if (!bIsMouseActive)
		{
			OnMouseEnter();
		}

		UE_LOGF(LogPixelStreaming2Input, Verbose, "MOUSE_UP: Button = %d", Button);

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			FSlateApplication& SlateApplication = FSlateApplication::Get();
			FWidgetPath		   WidgetPath = FindRoutingMessageWidget(SlateApplication.GetCursorPos());

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				FKey				   Key = TranslateMouseButtonToKey(Button);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetLastCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					Key,
					0,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				return SlateApplication.RoutePointerUpEvent(WidgetPath, MouseEvent).IsEventHandled();
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			if (Button != EMouseButtons::Type::Invalid)
			{
				return MessageHandler->OnMouseUp(Button);
			}
		}

		return false;
	}

	bool FPixelStreaming2InputHandler::OnMouseDown(EMouseButtons::Type Button, FIntPoint ScreenLocation)
	{
		// Ensure we have wrapped the slate application at this point
		if (!bIsMouseActive)
		{
			OnMouseEnter();
		}

		UE_LOGF(LogPixelStreaming2Input, Verbose, "MOUSE_DOWN: Button = %d; Pos = (%d, %d)", Button, ScreenLocation.X, ScreenLocation.Y);
		// Set cursor pos on mouse down - we may not have moved if this is the very first click
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		SlateApplication.OnCursorSet();
		PixelStreamerApplicationWrapper->Cursor->SetPosition(ScreenLocation.X, ScreenLocation.Y);
		// Force window focus
		SlateApplication.ProcessApplicationActivationEvent(true);

		if (bSynthesizeMouseMoveForNextMouseDown)
		{
			SynthesizeMouseMove();
			bSynthesizeMouseMoveForNextMouseDown = false;
		}

		bool bHandled = false;
		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);

				FKey Key = TranslateMouseButtonToKey(Button);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					ScreenLocation,
					SlateApplication.GetLastCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					Key,
					0,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				bHandled = SlateApplication.RoutePointerDownEvent(WidgetPath, MouseEvent).IsEventHandled();
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			bHandled = MessageHandler->OnMouseDown(PixelStreamerApplicationWrapper->GetWindowUnderCursor(), Button, ScreenLocation);
		}

		// The browser may be faking a mouse when touching so it will send
		// over a mouse down event.
		FindFocusedWidget();

		return bHandled;
	}

	bool FPixelStreaming2InputHandler::OnMouseMove(FIntPoint ScreenLocation, FIntPoint Delta)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "MOUSE_MOVE: Pos = (%d, %d); Delta = (%d, %d)", ScreenLocation.X, ScreenLocation.Y, Delta.X, Delta.Y);
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		SlateApplication.OnCursorSet();
		PixelStreamerApplicationWrapper->Cursor->SetPosition(ScreenLocation.X, ScreenLocation.Y);

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetLastCursorPos(),
					FVector2D(Delta.X, Delta.Y),
					SlateApplication.GetPressedMouseButtons(),
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				return SlateApplication.RoutePointerMoveEvent(WidgetPath, MouseEvent, false);
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			return MessageHandler->OnRawMouseMove(Delta.X, Delta.Y);
		}

		return false;
	}

	bool FPixelStreaming2InputHandler::OnMouseWheel(FIntPoint ScreenLocation, float MouseWheelDelta)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "MOUSE_WHEEL: Delta = %.4f; Pos = (%d, %d)", MouseWheelDelta, ScreenLocation.X, ScreenLocation.Y);

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);

				FSlateApplication& SlateApplication = FSlateApplication::Get();

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					EKeys::Invalid,
					MouseWheelDelta,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				return SlateApplication.RouteMouseWheelOrGestureEvent(WidgetPath, MouseEvent, nullptr).IsEventHandled();
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			return MessageHandler->OnMouseWheel(MouseWheelDelta, ScreenLocation);
		}

		return false;
	}

	bool FPixelStreaming2InputHandler::OnMouseDoubleClick(EMouseButtons::Type Button, FIntPoint ScreenLocation)
	{
		UE_LOGF(LogPixelStreaming2Input, Verbose, "MOUSE_DOWN: Button = %d; Pos = (%d, %d)", Button, ScreenLocation.X, ScreenLocation.Y);
		// Force window focus
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		SlateApplication.ProcessApplicationActivationEvent(true);

		if (InputType == EPixelStreaming2InputType::RouteToWidget)
		{
			FWidgetPath WidgetPath = FindRoutingMessageWidget(ScreenLocation);

			if (WidgetPath.IsValid())
			{
				FScopedSwitchWorldHack SwitchWorld(WidgetPath);
				FKey				   Key = TranslateMouseButtonToKey(Button);

				FPointerEvent MouseEvent(
					SlateApplication.GetUserIndexForMouse(),
					FSlateApplicationBase::CursorPointerIndex,
					SlateApplication.GetCursorPos(),
					SlateApplication.GetLastCursorPos(),
					SlateApplication.GetPressedMouseButtons(),
					Key,
					0,
					SlateApplication.GetPlatformApplication()->GetModifierKeys());

				return SlateApplication.RoutePointerDoubleClickEvent(WidgetPath, MouseEvent).IsEventHandled();
			}
		}
		else if (InputType == EPixelStreaming2InputType::RouteToWindow)
		{
			return MessageHandler->OnMouseDoubleClick(PixelStreamerApplicationWrapper->GetWindowUnderCursor(), Button, ScreenLocation);
		}

		return false;
	}

	/**
	 * XR Handling
	 */
	bool FPixelStreaming2InputHandler::OnXREyeViews(FTransform LeftEyeTransform, FMatrix LeftEyeProjectionMatrix, FTransform RightEyeTransform, FMatrix RightEyeProjectionMatrix, FTransform HMDTransform)
	{
		if (IPixelStreaming2HMD* HMD = IPixelStreaming2HMDModule::Get().GetPixelStreaming2HMD(); HMD != nullptr)
		{
			HMD->SetEyeViews(LeftEyeTransform, LeftEyeProjectionMatrix, RightEyeTransform, RightEyeProjectionMatrix, HMDTransform);
			return true;
		}

		return false;
	}

	bool FPixelStreaming2InputHandler::OnXRHMDTransform(FTransform HMDTransform)
	{
		if (IPixelStreaming2HMD* HMD = IPixelStreaming2HMDModule::Get().GetPixelStreaming2HMD(); HMD != nullptr)
		{
			HMD->SetTransform(HMDTransform);
			return true;
		}

		return false;
	}

	bool FPixelStreaming2InputHandler::OnXRControllerTransform(FTransform ControllerTransform, EControllerHand Handedness)
	{
		FPixelStreaming2XRController Controller;
		Controller.Transform = ControllerTransform;
		Controller.Handedness = Handedness;
		XRControllers.Add(Handedness, Controller);

		return true;
	}

	bool FPixelStreaming2InputHandler::OnXRButtonTouched(EControllerHand Handedness, FKey Key, bool bIsRepeat)
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId				ControllerId = DeviceMapper.GetDefaultInputDevice();

		UE_LOGF(LogPixelStreaming2Input, Verbose, "XR_TOUCHED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls;", ControllerId.GetId(), *Key.ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));

		return MessageHandler->OnControllerButtonPressed(Key.GetFName(), PLATFORMUSERID_NONE /* Not used */, ControllerId, bIsRepeat);
	}

	bool FPixelStreaming2InputHandler::OnXRButtonTouchReleased(EControllerHand Handedness, FKey Key)
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId				ControllerId = DeviceMapper.GetDefaultInputDevice();

		UE_LOGF(LogPixelStreaming2Input, Verbose, "XR_TOUCH_RELEASED: ControllerId = %d; KeyName = %ls;", ControllerId.GetId(), *Key.ToString());

		return MessageHandler->OnControllerButtonReleased(Key.GetFName(), PLATFORMUSERID_NONE /* Not used */, ControllerId, false);
	}

	bool FPixelStreaming2InputHandler::OnXRButtonPressed(EControllerHand Handedness, FKey Key, bool bIsRepeat)
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId				ControllerId = DeviceMapper.GetDefaultInputDevice();

		UE_LOGF(LogPixelStreaming2Input, Verbose, "XR_PRESSED: ControllerId = %d; KeyName = %ls; IsRepeat = %ls", ControllerId.GetId(), *Key.ToString(), bIsRepeat ? TEXT("True") : TEXT("False"));

		return MessageHandler->OnControllerButtonPressed(Key.GetFName(), PLATFORMUSERID_NONE /* Not used */, ControllerId, bIsRepeat);
	}

	bool FPixelStreaming2InputHandler::OnXRButtonReleased(EControllerHand Handedness, FKey Key)
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId				ControllerId = DeviceMapper.GetDefaultInputDevice();

		UE_LOGF(LogPixelStreaming2Input, Verbose, "XR_RELEASED: ControllerId = %d; KeyName = %ls;", ControllerId.GetId(), *Key.ToString());

		return MessageHandler->OnControllerButtonReleased(Key.GetFName(), PLATFORMUSERID_NONE /* Not used */, ControllerId, false);
	}

	bool FPixelStreaming2InputHandler::OnXRAnalog(EControllerHand Handedness, FKey Key, double AnalogValue)
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FInputDeviceId				ControllerId = DeviceMapper.GetDefaultInputDevice();

		// This codepath is used for XR joysticks, which seems to be robust to temporary drops in input transmission
		// so we can safely set `bKeepUnlessZero` to false. However, if we use this for more than joysticks we will have to conditionally set this.
		FAnalogValue AnalogEvent;
		AnalogEvent.bKeepUnlessZero = false;
		AnalogEvent.Value = AnalogValue;
		AnalogEvent.bIsRepeat = false;
		AnalogEventsReceivedThisTick.FindOrAdd(ControllerId).FindOrAdd(Key) = AnalogEvent;

		UE_LOGF(LogPixelStreaming2Input, VeryVerbose, "XR_ANALOG: ControllerId = %d; KeyName = %ls; IsRepeat = False; AnalogValue = %.4f; [Queued for Tick()]", ControllerId.GetId(), *Key.ToString(), AnalogEvent.Value);

		return true;
	}

	bool FPixelStreaming2InputHandler::OnXRSystem(EPixelStreaming2XRSystem System)
	{
		IPixelStreaming2HMDModule::Get().SetActiveXRSystem(System);
		return true;
	}

	void FPixelStreaming2InputHandler::SetCommandHandler(const FString& CommandName, const CommandHandlerFn& Handler)
	{
		CommandHandlers.Add(CommandName, Handler);
	}

	void FPixelStreaming2InputHandler::SetElevatedCheck(const TFunction<bool(FString)>& CheckFn)
	{
		ElevatedCheck = CheckFn;
	}

	bool FPixelStreaming2InputHandler::IsElevated(const FString& Id)
	{
		return !ElevatedCheck || ElevatedCheck(Id);
	}

	FVector2D FPixelStreaming2InputHandler::ConvertToNormalizedScreenLocation(FVector2D Pos)
	{
		FVector2D NormalizedLocation = FVector2D::ZeroVector;

		TSharedPtr<SWindow> ApplicationWindow = TargetWindow.Pin();
		if (ApplicationWindow.IsValid())
		{
			FVector2D WindowOrigin = ApplicationWindow->GetPositionInScreen();
			if (TargetViewport.IsValid())
			{
				TSharedPtr<SViewport> ViewportWidget = TargetViewport.Pin();

				if (ViewportWidget.IsValid())
				{
					FGeometry InnerWindowGeometry = ApplicationWindow->GetWindowGeometryInWindow();

					// Find the widget path relative to the window
					FArrangedChildren JustWindow(EVisibility::Visible);
					JustWindow.AddWidget(FArrangedWidget(ApplicationWindow.ToSharedRef(), InnerWindowGeometry));

					FWidgetPath PathToWidget(ApplicationWindow.ToSharedRef(), JustWindow);
					if (PathToWidget.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
					{
						FArrangedWidget ArrangedWidget = PathToWidget.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());
						FVector2D		WindowClientSize = ArrangedWidget.Geometry.GetAbsoluteSize();

						NormalizedLocation = FVector2D(Pos / WindowClientSize);
					}
				}
			}
			else
			{
				FVector2D SizeInScreen = ApplicationWindow->GetSizeInScreen();
				NormalizedLocation = FVector2D(Pos / SizeInScreen);
			}
		}
		else if (TSharedPtr<FIntRect> ScreenRectPtr = TargetScreenRect.Pin())
		{
			FIntRect  ScreenRect = *ScreenRectPtr;
			FIntPoint SizeInScreen = ScreenRect.Max - ScreenRect.Min;
			NormalizedLocation = Pos / SizeInScreen;
		}

		NormalizedLocation *= uint16_MAX;
		return NormalizedLocation;
	}

	FIntPoint FPixelStreaming2InputHandler::ConvertFromNormalizedScreenLocation(const FVector2D& ScreenLocation, bool bIncludeOffset)
	{
		FIntPoint OutVector((int32)ScreenLocation.X, (int32)ScreenLocation.Y);

		if (TSharedPtr<SWindow> ApplicationWindow = TargetWindow.Pin())
		{
			FVector2D WindowOrigin = ApplicationWindow->GetPositionInScreen();
			if (TSharedPtr<SViewport> ViewportWidget = TargetViewport.Pin())
			{
				FGeometry InnerWindowGeometry = ApplicationWindow->GetWindowGeometryInWindow();

				// Find the widget path relative to the window
				FArrangedChildren JustWindow(EVisibility::Visible);
				JustWindow.AddWidget(FArrangedWidget(ApplicationWindow.ToSharedRef(), InnerWindowGeometry));

				FWidgetPath PathToWidget(ApplicationWindow.ToSharedRef(), JustWindow);
				if (PathToWidget.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
				{
					FArrangedWidget ArrangedWidget = PathToWidget.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());

					FVector2D WindowClientOffset = ArrangedWidget.Geometry.GetAbsolutePosition();
					FVector2D WindowClientSize = ArrangedWidget.Geometry.GetAbsoluteSize();

					FVector2D OutTemp = bIncludeOffset
						? (ScreenLocation * WindowClientSize) + WindowOrigin + WindowClientOffset
						: (ScreenLocation * WindowClientSize);
					UE_LOGF(LogPixelStreaming2Input, Verbose, "%.4f, %.4f", ScreenLocation.X, ScreenLocation.Y);
					OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
				}
			}
			else
			{
				FVector2D SizeInScreen = ApplicationWindow->GetSizeInScreen();
				FVector2D OutTemp = bIncludeOffset
					? (SizeInScreen * ScreenLocation) + ApplicationWindow->GetPositionInScreen()
					: (SizeInScreen * ScreenLocation);
				OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
			}
		}
		else if (TSharedPtr<FIntRect> ScreenRectPtr = TargetScreenRect.Pin())
		{
			FIntRect  ScreenRect = *ScreenRectPtr;
			FIntPoint SizeInScreen = ScreenRect.Max - ScreenRect.Min;
			FVector2D OutTemp = FVector2D(SizeInScreen.X, SizeInScreen.Y) * ScreenLocation + (bIncludeOffset ? FVector2D(ScreenRect.Min.X, ScreenRect.Min.Y) : FVector2D(0, 0));
			OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
		}
		else if (TSharedPtr<FIntPoint> ScreenSize = TargetScreenSize.Pin())
		{
			UE_LOGF(LogPixelStreaming2Input, Warning, "You're using deprecated functionality by setting a target screen size. This functionality will be removed in later versions. Please use SetTargetScreenRect instead!");
			FIntPoint SizeInScreen = *ScreenSize;
			FVector2D OutTemp = FVector2D(SizeInScreen) * ScreenLocation;
			OutVector = FIntPoint((int32)OutTemp.X, (int32)OutTemp.Y);
		}

		return OutVector;
	}

	void FPixelStreaming2InputHandler::OnFilteredKeysChanged(IConsoleVariable* Var)
	{
		FString			CommaList = Var->GetString();
		TArray<FString> KeyStringArray;
		CommaList.ParseIntoArray(KeyStringArray, TEXT(","), true);
		FilteredKeys.Empty();
		for (auto&& KeyString : KeyStringArray)
		{
			FilteredKeys.Add(FKey(*KeyString));
		}
	}

	bool FPixelStreaming2InputHandler::FilterKey(const FKey& Key)
	{
		for (auto&& FilteredKey : FilteredKeys)
		{
			if (FilteredKey == Key)
				return false;
		}
		return true;
	}

	void FPixelStreaming2InputHandler::ProcessLatestAnalogInputFromThisTick()
	{
		for (auto AnalogInputIt = AnalogEventsReceivedThisTick.CreateIterator(); AnalogInputIt; ++AnalogInputIt)
		{
			for (auto FKeyIt = AnalogInputIt->Value.CreateIterator(); FKeyIt; ++FKeyIt)
			{
				const FInputDeviceId& ControllerId = AnalogInputIt->Key;
				FKey				  Key = FKeyIt->Key;
				FAnalogValue		  AnalogValue = FKeyIt->Value;
				bool				  bIsRepeat = AnalogValue.bIsRepeat;

				// Check if this gamepad event is specific to a normal gamepad and not an xr gamepad
				if (Key == EKeys::Gamepad_LeftX ||			 //
					Key == EKeys::Gamepad_LeftY ||			 //
					Key == EKeys::Gamepad_RightX ||			 //
					Key == EKeys::Gamepad_RightY ||			 //
					Key == EKeys::Gamepad_LeftTriggerAxis || //
					Key == EKeys::Gamepad_RightTriggerAxis)
				{
					// This is a gamepad key. We need to check that the gamepad hasn't been disconnected for we've been able to process this analog event
					uint8 ControllerIndex;
					if (!FInputDevice::GetInputDevice()->GetControllerIdFromDeviceId(ControllerId, ControllerIndex))
					{
						// We're unable to get a platform user and device for this controller id. That means the controllers been disconnected
						FKeyIt.RemoveCurrent();
						continue;
					}
				}

				// Pass an analog input along the engine's input processing system
				bool bHandled = MessageHandler->OnControllerAnalog(Key.GetFName(), PLATFORMUSERID_NONE /* Not used */, ControllerId, AnalogValue.Value);
				UE_LOGF(LogPixelStreaming2Input, VeryVerbose, "TICKED ANALOG Input: ControllerId = %d; KeyName = %ls; IsRepeat = %ls; AnalogValue = %.4f; Handled = %ls; [Queued for Tick()]", ControllerId.GetId(), *Key.ToString(), bIsRepeat ? TEXT("True") : TEXT("False"), AnalogValue.Value, bHandled ? TEXT("True") : TEXT("False"));

				// Remove current analog key unless it has the special `bKeepUnlessZero` flag set.
				// This flag is used to continuously apply input values across ticks because
				// Pixel Streaming may not have transmitted an axis value in time for the next tick.
				// But in all ordinary cases where this flag is not set, the stored analog value should
				// be dropped from the map so the input for the axis (e.g. joystick) is only applied the frame
				// it is received. The `bKeepUnlessZero` is used for trigger axes, where a temporary drop in
				// input triggers UE into thinking a full press/release should occur.
				if (!AnalogValue.bKeepUnlessZero)
				{
					FKeyIt.RemoveCurrent();
				}
				else if (AnalogValue.bKeepUnlessZero && AnalogValue.Value == 0.0)
				{
					// HACK: If we have zero, send it again next frame to ensure we trigger a release internally
					// Without this release does not seem to get processed for axes inputs
					FKeyIt->Value.bIsRepeat = true;
					FKeyIt->Value.bKeepUnlessZero = false;
				}
				else
				{
					// We are resending the same input, signal this is the case on UE side
					FKeyIt->Value.bIsRepeat = true;
				}
			}
		}
	}

	void FPixelStreaming2InputHandler::BroadcastActiveTouchMoveEvents()
	{
		if (!ensure(MessageHandler))
		{
			return;
		}

		for (TPair<int32, FCachedTouchEvent> CachedTouchEvent : CachedTouchEvents)
		{
			const int32&			 TouchIndex = CachedTouchEvent.Key;
			const FCachedTouchEvent& TouchEvent = CachedTouchEvent.Value;

			// Only broadcast events that haven't already been fired this frame
			if (!TouchIndicesProcessedThisFrame.Contains(TouchIndex))
			{
				if (InputType == EPixelStreaming2InputType::RouteToWidget)
				{
					FWidgetPath WidgetPath = FindRoutingMessageWidget(TouchEvent.Location);

					if (WidgetPath.IsValid())
					{
						FScopedSwitchWorldHack SwitchWorld(WidgetPath);
						FPointerEvent		   PointerEvent(0, TouchIndex, TouchEvent.Location, LastTouchLocation, TouchEvent.Force, true);
						FSlateApplication::Get().RoutePointerMoveEvent(WidgetPath, PointerEvent, false);
					}
				}
				else if (InputType == EPixelStreaming2InputType::RouteToWindow)
				{
					MessageHandler->OnTouchMoved(TouchEvent.Location, TouchEvent.Force, TouchIndex, TouchEvent.ControllerIndex);
				}
			}
		}
	}

	void FPixelStreaming2InputHandler::SynthesizeMouseMove() const
	{
		// Move the mouse back and forth so the net result does not result in
		// moving the cursor.
		MessageHandler->OnRawMouseMove(1, 0);
		MessageHandler->OnRawMouseMove(-1, 0);
	}

	FKey FPixelStreaming2InputHandler::TranslateMouseButtonToKey(const EMouseButtons::Type Button)
	{
		FKey Key = EKeys::Invalid;

		switch (Button)
		{
			case EMouseButtons::Left:
				Key = EKeys::LeftMouseButton;
				break;
			case EMouseButtons::Middle:
				Key = EKeys::MiddleMouseButton;
				break;
			case EMouseButtons::Right:
				Key = EKeys::RightMouseButton;
				break;
			case EMouseButtons::Thumb01:
				Key = EKeys::ThumbMouseButton;
				break;
			case EMouseButtons::Thumb02:
				Key = EKeys::ThumbMouseButton2;
				break;
		}

		return Key;
	}

	void FPixelStreaming2InputHandler::FindFocusedWidget()
	{
		FSlateApplication::Get().ForEachUser([this](FSlateUser& User) {
			TSharedPtr<SWidget> FocusedWidget = User.GetFocusedWidget();

			if(!FocusedWidget)
			{
				return;
			}

			static FName SEditableTextType(TEXT("SEditableText"));
			static FName SMultiLineEditableTextType(TEXT("SMultiLineEditableText"));
			bool		 bEditable = FocusedWidget && (FocusedWidget->GetType() == SEditableTextType || FocusedWidget->GetType() == SMultiLineEditableTextType);

			if (bEditable)
			{
				if (FocusedWidget->GetType() == TEXT("SEditableText"))
				{
					SEditableText* TextBox = static_cast<SEditableText*>(FocusedWidget.Get());
					bEditable = !TextBox->IsTextReadOnly();
				}
				else if (FocusedWidget->GetType() == TEXT("SMultiLineEditableText"))
				{
					SMultiLineEditableText* TextBox = static_cast<SMultiLineEditableText*>(FocusedWidget.Get());
					bEditable = !TextBox->IsTextReadOnly();
				}
			}

			FVector2D Pos = UnfocusedPos;
			if (bEditable)
			{
				Pos = FocusedWidget->GetCachedGeometry().GetAbsolutePosition();

				TSharedPtr<SWindow> ApplicationWindow = TargetWindow.Pin();
				if (ApplicationWindow.IsValid())
				{
					FVector2D WindowOrigin = ApplicationWindow->GetPositionInScreen();
					if (TargetViewport.IsValid())
					{
						TSharedPtr<SViewport> ViewportWidget = TargetViewport.Pin();

						if (ViewportWidget.IsValid())
						{
							FGeometry InnerWindowGeometry = ApplicationWindow->GetWindowGeometryInWindow();

							// Find the widget path relative to the window
							FArrangedChildren JustWindow(EVisibility::Visible);
							JustWindow.AddWidget(FArrangedWidget(ApplicationWindow.ToSharedRef(), InnerWindowGeometry));

							FWidgetPath PathToWidget(ApplicationWindow.ToSharedRef(), JustWindow);
							if (PathToWidget.ExtendPathTo(FWidgetMatcher(ViewportWidget.ToSharedRef()), EVisibility::Visible))
							{
								FArrangedWidget ArrangedWidget = PathToWidget.FindArrangedWidget(ViewportWidget.ToSharedRef()).Get(FArrangedWidget::GetNullWidget());

								FVector2D WindowClientOffset = ArrangedWidget.Geometry.GetAbsolutePosition();
								FVector2D WindowClientSize = ArrangedWidget.Geometry.GetAbsoluteSize();

								Pos = Pos - WindowClientOffset;
							}
						}
					}
				}
			}

			if (Pos != FocusedPos)
			{
				FocusedPos = Pos;

				// Tell the browser that the focus has changed.
				TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
				JsonObject->SetStringField(TEXT("command"), TEXT("onScreenKeyboard"));
				JsonObject->SetBoolField(TEXT("showOnScreenKeyboard"), bEditable);

				if (bEditable)
				{
					FVector2D NormalizedLocation = ConvertToNormalizedScreenLocation(Pos);

					JsonObject->SetNumberField(TEXT("x"), static_cast<uint16>(NormalizedLocation.X));
					JsonObject->SetNumberField(TEXT("y"), static_cast<uint16>(NormalizedLocation.Y));

					FText TextboxContents;
					if (FocusedWidget->GetType() == TEXT("SEditableText"))
					{
						SEditableText* TextBox = static_cast<SEditableText*>(FocusedWidget.Get());
						TextboxContents = TextBox->GetText();
					}
					else if (FocusedWidget->GetType() == TEXT("SMultiLineEditableText"))
					{
						SMultiLineEditableText* TextBox = static_cast<SMultiLineEditableText*>(FocusedWidget.Get());
						TextboxContents = TextBox->GetText();
					}

					JsonObject->SetStringField(TEXT("contents"), TextboxContents.ToString());
				}

				FString															 Descriptor;
				TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Descriptor);
				FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);

				FBufferArchive Buffer;
				Buffer << Descriptor;
				TArray<uint8> Data(Buffer.GetData(), Buffer.Num());

				/**
				 * Send the following JSON string in a "Command" message to all players
				 * {
				 *	"command": "onScreenKeyboard",
				 *	"showOnScreenKeyboard": "true", //or false
				 *   "x": 1, //some uint16
				 *   "y": 1, //some uint16
				 *   "contents": "text box contents" // whatever text the textbox has in it
				 * }
				 */

				// Specific implementation for this method is handled per streamer
				OnSendMessage.Broadcast(EPixelStreaming2FromStreamerMessage::Command, FMemoryReader(Data));
			}
		});
	}

	FWidgetPath FPixelStreaming2InputHandler::FindRoutingMessageWidget(const FVector2D& Location) const
	{
		if (TSharedPtr<SWindow> PlaybackWindowPinned = TargetWindow.Pin())
		{
			if (PlaybackWindowPinned->AcceptsInput())
			{
				bool					  bIgnoreEnabledStatus = false;
				TArray<FWidgetAndPointer> WidgetsAndCursors = PlaybackWindowPinned->GetHittestGrid().GetBubblePath(Location, FSlateApplication::Get().GetCursorRadius(), bIgnoreEnabledStatus);
				return FWidgetPath(MoveTemp(WidgetsAndCursors));
			}
		}
		return FWidgetPath();
	}
} // namespace UE::PixelStreaming2Input
