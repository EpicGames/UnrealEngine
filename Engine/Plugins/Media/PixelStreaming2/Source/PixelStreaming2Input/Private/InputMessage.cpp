// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputMessage.h"
#include "PixelStreaming2InputEnums.h"

namespace UE::PixelStreaming2Input
{

	FInputMessage::FInputMessage(uint8 InId)
		: Id(InId)
		, Structure({})
	{
	}

	FInputMessage::FInputMessage(uint8 InId, TArray<EPixelStreaming2MessageTypes> InStructure)
		: Id(InId)
		, Structure(InStructure)
	{
	}

} // namespace UE::PixelStreaming2Input