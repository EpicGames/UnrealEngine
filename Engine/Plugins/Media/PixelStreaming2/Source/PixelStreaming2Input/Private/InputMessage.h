// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreaming2InputEnums.h"
#include "IPixelStreaming2InputMessage.h"

#include "Containers/Array.h"
#include "HAL/Platform.h"

namespace UE::PixelStreaming2Input
{

	class FInputMessage : public IPixelStreaming2InputMessage
	{
	public:
		// Internal use constructor taking an ID. Sets Structure to []
		FInputMessage(uint8 InId);

		// Constructor taking an ID and a Structure
		FInputMessage(uint8 InId, TArray<EPixelStreaming2MessageTypes> InStructure);

		virtual ~FInputMessage() = default;

		uint8									   GetID() const override { return Id; }
		TArray<EPixelStreaming2MessageTypes> GetStructure() const override { return Structure; }

	private:
		uint8									   Id;
		TArray<EPixelStreaming2MessageTypes> Structure;
	};

} // namespace UE::PixelStreaming2Input