// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputProtocolMap.h"
#include "InputMessage.h"

namespace UE::PixelStreaming2Input
{

	TSharedPtr<IPixelStreaming2InputMessage> FInputProtocolMap::AddMessageInternal(FString Key, uint8 Id, TArray<EPixelStreaming2MessageTypes> InStructure)
	{
		if (Ids.Contains(Id))
		{
			checkf(false, TEXT("Cannot add message to the protocol because the id used, id=%d, is already in use."), Id);
			return TSharedPtr<IPixelStreaming2InputMessage>();
		}
		if (InnerMap.Contains(Key))
		{
			checkf(false, TEXT("Cannot add message to the protocol because the key used, key=%s, is already in use."), *Key);
			return TSharedPtr<IPixelStreaming2InputMessage>();
		}
		TSharedPtr<FInputMessage> NewMessage = TSharedPtr<FInputMessage>(new FInputMessage(Id, InStructure));
		Ids.Add(Id);
		return InnerMap.Add(Key, NewMessage);
	}

	TSharedPtr<IPixelStreaming2InputMessage> FInputProtocolMap::Add(FString StringKey)
	{
		// User calls this to add custom message type with no message body
		return Add(StringKey, {});
	}

	TSharedPtr<IPixelStreaming2InputMessage> FInputProtocolMap::Add(FString StringKey, TArray<EPixelStreaming2MessageTypes> InStructure)
	{
		// User call this to add a custom message type with message body as per the structure
		uint8										   MessageId = UserMessageId++;
		TSharedPtr<IPixelStreaming2InputMessage> Res = AddMessageInternal(StringKey, MessageId, {});
		if (Res)
		{
			OnProtocolUpdatedDelegate.Broadcast();
		}
		return Res;
	}

	bool FInputProtocolMap::AddInternal(FString StringKey, uint8 Id)
	{
		return AddInternal(StringKey, Id, {});
	}

	bool FInputProtocolMap::AddInternal(FString StringKey, uint8 Id, TArray<EPixelStreaming2MessageTypes> InStructure)
	{
		checkf(Id < 200 || Id > 250, TEXT("Internal Pixel Streaming ids must not use the user reserved range [200-250], this message used %d"), Id);
		TSharedPtr<IPixelStreaming2InputMessage> Res = AddMessageInternal(StringKey, Id, InStructure);
		bool										   bAdded = Res ? true : false;
		checkf(bAdded, TEXT("FInputProtocolMap::AddInternal failed to add key=%s id=%d. This key may already be in use."), *StringKey, Id);
		if (bAdded)
		{
			OnProtocolUpdatedDelegate.Broadcast();
		}
		return bAdded;
	}

	int FInputProtocolMap::Remove(FString Key)
	{
		if (TSharedPtr<IPixelStreaming2InputMessage> Message = Find(Key))
		{
			Ids.Remove(Message->GetID());
		}

		int NRemoved = InnerMap.Remove(Key);
		if (NRemoved > 0)
		{
			OnProtocolUpdatedDelegate.Broadcast();
		}
		return NRemoved;
	}

	TSharedPtr<IPixelStreaming2InputMessage> FInputProtocolMap::Find(FString Key)
	{
		TSharedPtr<IPixelStreaming2InputMessage>* MaybeValue = InnerMap.Find(Key);
		if (MaybeValue)
		{
			return *MaybeValue;
		}
		return TSharedPtr<IPixelStreaming2InputMessage>();
	}

	const TSharedPtr<IPixelStreaming2InputMessage> FInputProtocolMap::Find(FString Key) const
	{
		const TSharedPtr<IPixelStreaming2InputMessage>* MaybeValue = InnerMap.Find(Key);
		if (MaybeValue)
		{
			return *MaybeValue;
		}
		return TSharedPtr<IPixelStreaming2InputMessage>();
	}

	void FInputProtocolMap::Clear()
	{
		Ids.Empty();
		InnerMap.Empty();
		OnProtocolUpdatedDelegate.Broadcast();
	}

	bool FInputProtocolMap::IsEmpty() const
	{
		return InnerMap.IsEmpty();
	}

	void FInputProtocolMap::Apply(const TFunction<void(FString, TSharedPtr<IPixelStreaming2InputMessage>)>& Visitor)
	{
		for (auto&& [Key, Value] : InnerMap)
		{
			Visitor(Key, Value);
		}
	}

	TSharedPtr<FJsonObject> FInputProtocolMap::ToJson()
	{
		TSharedPtr<FJsonObject> ProtocolJson = MakeShareable(new FJsonObject());

		ProtocolJson->SetField(TEXT("Direction"), MakeShared<FJsonValueNumber>(static_cast<uint8>(Direction)));
		Apply([ProtocolJson](FString Key, TSharedPtr<IPixelStreaming2InputMessage> Value) {
			TSharedPtr<FJsonObject> MessageJson = MakeShareable(new FJsonObject());

			MessageJson->SetField(TEXT("id"), MakeShared<FJsonValueNumber>(Value->GetID()));

			TArray<EPixelStreaming2MessageTypes> Structure = Value->GetStructure();
			TArray<TSharedPtr<FJsonValue>>			   StructureJson;
			for (auto It = Structure.CreateIterator(); It; ++It)
			{
				FString Text;
				switch (*It)
				{
					case EPixelStreaming2MessageTypes::Uint8:
						Text = "uint8";
						break;
					case EPixelStreaming2MessageTypes::Uint16:
						Text = "uint16";
						break;
					case EPixelStreaming2MessageTypes::Int16:
						Text = "int16";
						break;
					case EPixelStreaming2MessageTypes::Float:
						Text = "float";
						break;
					case EPixelStreaming2MessageTypes::Double:
						Text = "double";
						break;
					case EPixelStreaming2MessageTypes::String:
						Text = "string";
						break;
					case EPixelStreaming2MessageTypes::Undefined:
					default:
						Text = "";
				}
				TSharedRef<FJsonValueString> JsonValue = MakeShareable(new FJsonValueString(*Text));
				StructureJson.Add(JsonValue);
			}
			MessageJson->SetArrayField(TEXT("structure"), StructureJson);

			ProtocolJson->SetField(*Key, MakeShared<FJsonValueObject>(MessageJson));
		});

		return ProtocolJson;
	}

}