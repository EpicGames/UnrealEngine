// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2DefaultDataProtocol.h"

#include "InputProtocolMap.h"
#include "IPixelStreaming2InputMessage.h"

namespace UE::PixelStreaming2Input
{

	typedef EPixelStreaming2MessageTypes EType;

	TSharedPtr<IPixelStreaming2DataProtocol> GetDefaultToStreamerProtocol()
	{
		TSharedPtr<FInputProtocolMap> ToStreamerProtocol = TSharedPtr<FInputProtocolMap>(new FInputProtocolMap(EPixelStreaming2MessageDirection::ToStreamer));

		/**
		 * Control Messages.
		 */
		// Simple commands with no payload
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::IFrameRequest, 0);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::RequestQualityControl, 1);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::FpsRequest, 2);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::AverageBitrateRequest, 3);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::StartStreaming, 4);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::StopStreaming, 5);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::LatencyTest, 6, { EType::String });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::RequestInitialSettings, 7);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::TestEcho, 8, { EType::String });

		/**
		 * Input Messages.
		 */
		// Generic Input Messages.
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::UIInteraction, 50, { EType::String });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::Command, 51, { EType::String });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::TextboxEntry, 52, { EType::String });

		// Keyboard Input Message.
		// Complex command with payload, therefore we specify the length of the payload (bytes) as well as the structure of the payload
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::KeyDown, 60, { EType::Uint8, EType::Uint8 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::KeyUp, 61, { EType::Uint8 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::KeyPress, 62, { EType::Uint16 });

		// Mouse Input Messages.
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseEnter, 70);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseLeave, 71);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseDown, 72, { EType::Uint8, EType::Uint16, EType::Uint16 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseUp, 73, { EType::Uint8, EType::Uint16, EType::Uint16 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseMove, 74, { EType::Uint16, EType::Uint16, EType::Uint16, EType::Uint16 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseWheel, 75, { EType::Int16, EType::Uint16, EType::Uint16 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::MouseDouble, 76, { EType::Uint8, EType::Uint16, EType::Uint16 });

		// Touch Input Messages.
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::TouchStart, 80, { EType::Uint8, EType::Uint16, EType::Uint16, EType::Uint8, EType::Uint8, EType::Uint8 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::TouchEnd, 81, { EType::Uint8, EType::Uint16, EType::Uint16, EType::Uint8, EType::Uint8, EType::Uint8 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::TouchMove, 82, { EType::Uint8, EType::Uint16, EType::Uint16, EType::Uint8, EType::Uint8, EType::Uint8 });

		// Gamepad Input Messages.
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::GamepadButtonPressed, 90, { EType::Uint8, EType::Uint8, EType::Uint8 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::GamepadButtonReleased, 91, { EType::Uint8, EType::Uint8, EType::Uint8 });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::GamepadAnalog, 92, { EType::Uint8, EType::Uint8, EType::Double });
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::GamepadConnected, 93);
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::GamepadDisconnected, 94, { EType::Uint8 });

		// XR Input Messages.
		// clang-format off
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XREyeViews, 109, {
															// Left eye transform (4x4 matrix)
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															// Left eye perspective projection (4x4 matrix)
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															// Right eye transform (4x4 matrix)
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															// Right eye perspective projection (4x4 matrix)
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															// HMD 4x4 Transform
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
														});
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRHMDTransform, 110, {	// 4x4 Transform
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRControllerTransform, 111, {// 4x4 Transform
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															EType::Float, EType::Float, EType::Float, EType::Float,
															// Handedness (L, R, Any)
															EType::Uint8
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRButtonPressed, 112, {// Handedness,  ButtonIdx,    IsRepeat,     PressedAmount
															EType::Uint8, EType::Uint8, EType::Uint8, EType::Double
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRButtonTouched, 113, {// Handedness,  ButtonIdx,    IsRepeat
															EType::Uint8, EType::Uint8, EType::Uint8
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRButtonReleased, 114, {// Handedness, ButtonIdx,    IsRepeat
															EType::Uint8, EType::Uint8, EType::Uint8
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRAnalog, 115, {// Handedness, ButtonIdx, AxisValue
															EType::Uint8, EType::Uint8, EType::Double
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRSystem, 116, {// Type of the XR system
															EType::Uint8
														});

		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::XRButtonTouchReleased, 117, {// Handedness, ButtonIdx, IsRepeat
															EType::Uint8, EType::Uint8, EType::Uint8
														});
		
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::ChannelRelayStatus, 198); // id, 2 byte string length, string, uint8 flag
		
		ToStreamerProtocol->AddInternal(EPixelStreaming2ToStreamerMessage::Multiplexed, 199); // id, 2 byte string length, string, original message

		// clang-format on
		return ToStreamerProtocol;
	}

	TSharedPtr<IPixelStreaming2DataProtocol> GetDefaultFromStreamerProtocol()
	{
		TSharedPtr<FInputProtocolMap> FromStreamerProtocol = TSharedPtr<FInputProtocolMap>(new FInputProtocolMap(EPixelStreaming2MessageDirection::FromStreamer));

		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::QualityControlOwnership, 0);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::Response, 1);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::Command, 2);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::FreezeFrame, 3);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::UnfreezeFrame, 4);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::VideoEncoderAvgQP, 5);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::LatencyTest, 6);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::InitialSettings, 7);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::FileExtension, 8);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::FileMimeType, 9);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::FileContents, 10);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::TestEcho, 11);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::InputControlOwnership, 12);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::GamepadResponse, 13);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::Protocol, 255);
		FromStreamerProtocol->AddInternal(EPixelStreaming2FromStreamerMessage::Multiplexed, 199);

		return FromStreamerProtocol;
	}

} // namespace UE::PixelStreaming2Input