// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2InputEnums.h"

namespace EPixelStreaming2ToStreamerMessage
{
	const FString IFrameRequest = FString(TEXT("IFrameRequest"));
	const FString RequestQualityControl = FString(TEXT("RequestQualityControl"));
	const FString FpsRequest = FString(TEXT("FpsRequest"));
	const FString AverageBitrateRequest = FString(TEXT("AverageBitrateRequest"));
	const FString StartStreaming = FString(TEXT("StartStreaming"));
	const FString StopStreaming = FString(TEXT("StopStreaming"));
	const FString LatencyTest = FString(TEXT("LatencyTest"));
	const FString RequestInitialSettings = FString(TEXT("RequestInitialSettings"));
	const FString TestEcho = FString(TEXT("TestEcho"));
	const FString UIInteraction = FString(TEXT("UIInteraction"));
	const FString Command = FString(TEXT("Command"));
	const FString TextboxEntry = FString(TEXT("TextboxEntry"));
	const FString KeyDown = FString(TEXT("KeyDown"));
	const FString KeyUp = FString(TEXT("KeyUp"));
	const FString KeyPress = FString(TEXT("KeyPress"));
	const FString MouseEnter = FString(TEXT("MouseEnter"));
	const FString MouseLeave = FString(TEXT("MouseLeave"));
	const FString MouseDown = FString(TEXT("MouseDown"));
	const FString MouseUp = FString(TEXT("MouseUp"));
	const FString MouseMove = FString(TEXT("MouseMove"));
	const FString MouseWheel = FString(TEXT("MouseWheel"));
	const FString MouseDouble = FString(TEXT("MouseDouble"));
	const FString TouchStart = FString(TEXT("TouchStart"));
	const FString TouchEnd = FString(TEXT("TouchEnd"));
	const FString TouchMove = FString(TEXT("TouchMove"));
	const FString GamepadButtonPressed = FString(TEXT("GamepadButtonPressed"));
	const FString GamepadButtonReleased = FString(TEXT("GamepadButtonReleased"));
	const FString GamepadAnalog = FString(TEXT("GamepadAnalog"));
	const FString GamepadConnected = FString(TEXT("GamepadConnected"));
	const FString GamepadDisconnected = FString(TEXT("GamepadDisconnected"));
	const FString XREyeViews = FString(TEXT("XREyeViews"));
	const FString XRHMDTransform = FString(TEXT("XRHMDTransform"));
	const FString XRControllerTransform = FString(TEXT("XRControllerTransform"));
	const FString XRButtonPressed = FString(TEXT("XRButtonPressed"));
	const FString XRButtonTouched = FString(TEXT("XRButtonTouched"));
	const FString XRButtonReleased = FString(TEXT("XRButtonReleased"));
	const FString XRAnalog = FString(TEXT("XRAnalog"));
	const FString XRSystem = FString(TEXT("XRSystem"));
	const FString XRButtonTouchReleased = FString(TEXT("XRButtonTouchReleased"));
	const FString Multiplexed = FString(TEXT("Multiplexed"));
	const FString ChannelRelayStatus = FString(TEXT("ChannelRelayStatus"));
} // namespace EPixelStreaming2ToStreamerMessage

namespace EPixelStreaming2FromStreamerMessage
{
	const FString QualityControlOwnership = FString(TEXT("QualityControlOwnership"));
	const FString Response = FString(TEXT("Response"));
	const FString Command = FString(TEXT("Command"));
	const FString FreezeFrame = FString(TEXT("FreezeFrame"));
	const FString UnfreezeFrame = FString(TEXT("UnfreezeFrame"));
	const FString VideoEncoderAvgQP = FString(TEXT("VideoEncoderAvgQP"));
	const FString LatencyTest = FString(TEXT("LatencyTest"));
	const FString InitialSettings = FString(TEXT("InitialSettings"));
	const FString FileExtension = FString(TEXT("FileExtension"));
	const FString FileMimeType = FString(TEXT("FileMimeType"));
	const FString FileContents = FString(TEXT("FileContents"));
	const FString TestEcho = FString(TEXT("TestEcho"));
	const FString InputControlOwnership = FString(TEXT("InputControlOwnership"));
	const FString GamepadResponse = FString(TEXT("GamepadResponse"));
	const FString Multiplexed = FString(TEXT("Multiplexed"));
	const FString Protocol = FString(TEXT("Protocol"));
} // namespace EPixelStreaming2FromStreamerMessage
