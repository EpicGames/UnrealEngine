// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPixelStreaming2InputHandler.h"

#include "InputDevice.h"

IPixelStreaming2InputHandler::IPixelStreaming2InputHandler()
{
    // Register this input handler with the module's input device so that it's ticked
    UE::PixelStreaming2Input::FInputDevice::GetInputDevice()->AddInputHandler(this);
}

IPixelStreaming2InputHandler::~IPixelStreaming2InputHandler()
{
    UE::PixelStreaming2Input::FInputDevice::GetInputDevice()->RemoveInputHandler(this);
}