// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2InputModule.h"

#include "Framework/Application/SlateApplication.h"
#include "InputHandler.h"
#include "IPixelStreaming2HMDModule.h"

namespace UE::PixelStreaming2Input
{
	void FPixelStreaming2InputModule::StartupModule()
	{
		if (!FSlateApplication::IsInitialized())
		{
			return;
		}

		InputDevice = FInputDevice::GetInputDevice();

		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	void FPixelStreaming2InputModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	}

	TSharedPtr<IPixelStreaming2InputHandler> FPixelStreaming2InputModule::CreateInputHandler()
	{
		return MakeShared<FPixelStreaming2InputHandler>();
	}

	TSharedPtr<IInputDevice> FPixelStreaming2InputModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	{
		return InputDevice;
	}
} // namespace UE::PixelStreaming2Input

IMPLEMENT_MODULE(UE::PixelStreaming2Input::FPixelStreaming2InputModule, PixelStreaming2Input)