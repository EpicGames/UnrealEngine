// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2InputModule.h"
#include "InputDevice.h"

namespace UE::PixelStreaming2Input
{
	class FPixelStreaming2InputModule : public IPixelStreaming2InputModule
	{
	public:
		virtual TSharedPtr<IPixelStreaming2InputHandler> CreateInputHandler() override;

	private:
		/** IModuleInterface implementation */
		void StartupModule() override;
		void ShutdownModule() override;
		/** End IModuleInterface implementation */

		/** IInputDeviceModule implementation */
		virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
		/** End IInputDeviceModule implementation */

		TSharedPtr<FInputDevice> InputDevice;
	};
} // namespace UE::PixelStreaming2Input