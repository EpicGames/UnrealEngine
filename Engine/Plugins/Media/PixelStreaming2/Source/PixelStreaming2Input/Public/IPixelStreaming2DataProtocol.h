// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2InputMessage.h"
#include "PixelStreaming2InputEnums.h"
#include "Dom/JsonObject.h"

/**
 * Interface for a datachannel protocol.
 * The user is not meant to extend this through polymorphism, but rather request the appropriate protocol from the InputHandler and extend it using Add().
 */
class IPixelStreaming2DataProtocol
{
public:
	IPixelStreaming2DataProtocol() = default;
	virtual ~IPixelStreaming2DataProtocol() = default;

	/**
	 * Adds a custom message type, with no message body, to the protocol.
	 * @param Key The string identifier used to uniquely identify and query this message inside the protocol.
	 * @return True if the new message was added to the protocol (fails if key is already present).
	 */
	virtual TSharedPtr<IPixelStreaming2InputMessage> Add(FString Key) = 0;

	/**
	 * Adds a custom message type, with associated message structured, to the protocol.
	 * @param Key The string identifier used to uniquely identify and query this message inside the protocol.
	 * @return True if the new message was added to the protocol (fails if key is already present).
	 */
	virtual TSharedPtr<IPixelStreaming2InputMessage> Add(FString Key, TArray<EPixelStreaming2MessageTypes> InStructure) = 0;

	/**
	 * Looks for the supplied message type in this protocol.
	 * This call should primarily be made from user code when the user has supplied a custom message type to the protocol.
	 * Note: If you are looking for a default message type consider using the Get() methods and passing one of the From/ToStreamer message enums to avoid typos.
	 * @param Key The string representation of the message type we are looking for.
	 * @return The message type associated with the `FromStreamer` message type passed.
	 */
	virtual TSharedPtr<IPixelStreaming2InputMessage> Find(FString Key) = 0;

	/**
	 * @return A JSON schema representing the data protocol.
	 */
	virtual TSharedPtr<FJsonObject> ToJson() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnProtocolUpdated);
	/**
	 * Called any time to protocol has a new data type added to it.
	 * Internally this will force Pixel Streaming to resend the entire data protocol.
	 */
	virtual FOnProtocolUpdated& OnProtocolUpdated() = 0;
};
