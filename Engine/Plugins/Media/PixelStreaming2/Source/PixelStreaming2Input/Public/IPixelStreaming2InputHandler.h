// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "GenericPlatform/GenericPlatformMisc.h"
#include "IInputDevice.h"
#include "IPixelStreaming2DataProtocol.h"
#include "PixelStreaming2HMDEnums.h"
#include "Widgets/SViewport.h"

#define UE_API PIXELSTREAMING2INPUT_API

/**
 * The IPixelStreaming2InputHandler, used to handle input from a remote peer and pass it to UE accordingly. Setting the target viewport allows for
 * scaling of input from browser to application, and setting the target window ensure that if windows are tiled (eg editor)
 * that the streamed input only affect the target window.
 */
class IPixelStreaming2InputHandler : public IInputDevice
{
public:
	UE_API IPixelStreaming2InputHandler();
	UE_API virtual ~IPixelStreaming2InputHandler();

	/**
	 * @brief Handle the message from the WebRTC data channel.
	 * @param SourceId A source ID for this message
	 * @param Buffer The data channel message
	 */
	virtual void OnMessage(FString SourceId, TArray<uint8> Buffer) = 0;

	/**
	 * @brief Set the viewport this input device is associated with.
	 * @param InTargetViewport The viewport to set
	 */
	virtual void SetTargetViewport(TWeakPtr<SViewport> InTargetViewport) = 0;

	/**
	 * @brief Get the viewport this input device is associated with.
	 * @return The viewport this input device is associated with
	 */
	virtual TWeakPtr<SViewport> GetTargetViewport() = 0;

	/**
	 * @brief Set the viewport this input device is associated with.
	 * @param InTargetWindow The viewport to set
	 */
	virtual void SetTargetWindow(TWeakPtr<SWindow> InTargetWindow) = 0;

	/**
	 * @brief Get the viewport this input device is associated with.
	 * @return The viewport this input device is associated with
	 */
	virtual TWeakPtr<SWindow> GetTargetWindow() = 0;

	/**
	 * @brief Set the target screen rectangle for this streamer. This is used to when the streamer doesn't have a singular target window / viewport
	 * and as such we just use the manual scale
	 * @param InTargetScreenRect The target screen rectangle
	 */
	virtual void SetTargetScreenRect(TWeakPtr<FIntRect> InTargetScreenRect) = 0;

	/**
	 * @brief Get the target screen rectangle for this streamer
	 * @return The target screen rectangle
	 */
	virtual TWeakPtr<FIntRect> GetTargetScreenRect() = 0;

	/**
	 * @brief Set whether the input devices is faking touch events using keyboard and mouse this can be useful for debugging.
	 * @return true
	 * @return false
	 */
	virtual bool IsFakingTouchEvents() const = 0;

	/**
	 * @brief The callback signature for handling a command sent to the data channel.
	 *
	 * @param SourceId The source id of the sender of this message.
	 * @param Message The full message in the form of a FMemoryReader.
	 */
	using MessageHandlerFn = TFunction<void(FString SourceId, FMemoryReader Message)>;

	/**
	 * @brief Register a function to be called whenever the specified message type is received.
	 *
	 * @param MessageType The human readable identifier for the message
	 * @param Handler The function called when this message type is received. This handler must take a single parameter (an FMemoryReader) and have a return type of void
	 */
	virtual void RegisterMessageHandler(const FString& MessageType, const MessageHandlerFn& Handler) = 0;

	/**
	 * @brief The callback signature for handling a command sent to the data channel.
	 *
	 * @param SourceId The source id of the sender of this message.
	 * @param Descriptor The full descriptor of the commaand.
	 * @param CommandString The relevant string parameters for the command.
	 */
	using CommandHandlerFn = TFunction<void(FString SourceId, FString Descriptor, FString CommandString)>;

	/**
	 * @brief Register a custom function to execute when command JSON is received over the data channel: "{ type: "Command", YourCommand: YourCommandValue }".
	 * Note: You can also override the default Pixel Streaming command handlers by setting handlers with the same name as those already used, e.g. "Stat.FPS".
	 * @param CommandName The name of the command to handle. This corresponds to the key in the JSON message and is used to identify the command.
	 * @param Handler The function that will be executed when the command is received.
	 */
	virtual void SetCommandHandler(const FString& CommandName, const CommandHandlerFn& Handler) = 0;

	/**
	 * @brief Some behaviours might want to be limited to a specific source or group of sources. This method sets a check function to test of a given source id is "elevated".
	 *
	 * @param CheckFn A callback that takes a SourceId and returns true if the source id is an elevated user.
	 */
	virtual void SetElevatedCheck(const TFunction<bool(FString)>& CheckFn) = 0;

	/**
	 * @brief Checks whether the given id has elevated priviledges.
	 *
	 * @return True if id is elevated and false is not elevated.
	 */
	virtual bool IsElevated(const FString& Id) = 0;

	/**
	 * @brief Find the function to be called whenever the specified message type is received.
	 *
	 * @param MessageType The human readable identifier for the message
	 * @return TFunction<void(FString, FMemoryReader)> The function called when this message type is received.
	 */
	virtual MessageHandlerFn FindMessageHandler(const FString& MessageType) = 0;

	/**
	 * @return The "ToStreamer" data protocol. This can be used to modify the protocol with custom data channel messages.
	 */
	virtual TSharedPtr<IPixelStreaming2DataProtocol> GetToStreamerProtocol() = 0;

	/**
	 * @return The "FromStreamer" data protocol. This can be used to modify the protocol with custom data channel messages.
	 */
	virtual TSharedPtr<IPixelStreaming2DataProtocol> GetFromStreamerProtocol() = 0;

	/**
	 * @brief Set the input handlers type. This controls whether input is routed to widgets or windows.
	 * @param InputType The input routing type.
	 */
	virtual void SetInputType(EPixelStreaming2InputType InputType) = 0;

	DECLARE_EVENT_TwoParams(IPixelStreaming2InputHandler, FOnSendMessage, FString, FMemoryReader);
	/*
	 * An event that is only fired internally of the InputHandler when it wants to send a message to all connected players.
	 * Examples include when a virtual gamepad controller is "connected" and given a unique id.
	 */
	FOnSendMessage OnSendMessage;

	/**
	 * @brief Notify the input handler of a character event.
	 *
	 * @param Character The character
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnKeyChar(TCHAR Character) = 0;

	/**
	 * @brief Notify the input handler of a key down event.
	 *
	 * @param Key The key
	 * @param bIsRepeat Is this a repeat of a previous Key down (ie has the key been held down)
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnKeyDown(FKey Key, bool bIsRepeat) = 0;

	/**
	 * @brief Notify the input handler of a key up event.
	 *
	 * @param Key The key
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnKeyUp(FKey Key) = 0;

	/**
	 * @brief Notify the input handler of the mouse entering the application (typically the browser window).
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseEnter() = 0;

	/**
	 * @brief Notify the input handler of the mouse leaving the application (typically the browser window).
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseLeave() = 0;

	/**
	 * @brief Notify the input handler of a mouse down event.
	 *
	 * @param Button The mouse button
	 * @param ScreenPosition The position of the mouse event in UE screen space
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseDown(EMouseButtons::Type Button, FIntPoint ScreenPosition) = 0;

	/**
	 * @brief Notify the input handler of a mouse up event.
	 *
	 * @param Button The mouse button
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseUp(EMouseButtons::Type Button) = 0;

	/**
	 * @brief Notify the input handler of a mouse move event.
	 *
	 * @param ScreenPosition The starting position of the mouse event in UE screen space
	 * @param Delta The amount the mouse moves in this event in UE screen space
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseMove(FIntPoint ScreenPosition, FIntPoint Delta) = 0;

	/**
	 * @brief Notify the input handler of a mouse wheel event.
	 *
	 * @param ScreenPosition The position of the mouse event in UE screen space
	 * @param MouseWheelDelta The amount the mouse wheel moved
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseWheel(FIntPoint ScreenPosition, float MouseWheelDelta) = 0;

	/**
	 * @brief Notify the input handler of a mouse double click event.
	 *
	 * @param Button The mouse button
	 * @param ScreenPosition The position of the mouse event in UE screen space
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnMouseDoubleClick(EMouseButtons::Type Button, FIntPoint ScreenPosition) = 0;

	/**
	 * @brief Notify the input handler of a touch start event.
	 *
	 * @param TouchPosition The position of the touch event in UE screen space
	 * @param TouchIndex The index of the finger that started touching
	 * @param Force The force of the touch in the range 0 (no force) to 1 (maximum force)
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnTouchStarted(FIntPoint TouchPosition, int32 TouchIndex, float Force) = 0;

	/**
	 * @brief Notify the input handler of a touch moved event.
	 *
	 * @param TouchPosition The position of the touch event after the move has completed in UE screen space
	 * @param TouchIndex The index of the finger that moved
	 * @param Force The force of the touch in the range 0 (no force) to 1 (maximum force)
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnTouchMoved(FIntPoint TouchPosition, int32 TouchIndex, float Force) = 0;

	/**
	 * @brief Notify the input handler of a touch ended event.
	 *
	 * @param TouchPosition The position of the touch event in UE screen space
	 * @param TouchIndex The index of the finger that stopped touching
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnTouchEnded(FIntPoint TouchPosition, int32 TouchIndex) = 0;

	/**
	 * @brief Notify the input handler of a controller connecting.
	 *
	 * @return the index to use for this new controller
	 */
	virtual uint8 OnControllerConnected() = 0;

	/**
	 * @brief Notify the input handler of a controller analog event.
	 *
	 * @param ControlledIndex The index of the controller received from OnControllerConnected
	 * @param Axis The axis FKey
	 * @param AxisValue The value of the axis in the range -1 to 1
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnControllerAnalog(uint8 ControllerIndex, FKey Axis, double AxisValue) = 0;

	/**
	 * @brief Notify the input handler of a controller button press event.
	 *
	 * @param ControlledIndex The index of the controller received from OnControllerConnected
	 * @param Key The button FKey
	 * @param bIsRepeat Is this a repeat of a previous button press (ie has the button been held down)
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnControllerButtonPressed(uint8 ControllerIndex, FKey Key, bool bIsRepeat) = 0;

	/**
	 * @brief Notify the input handler of a controller button release event.
	 *
	 * @param ControlledIndex The index of the controller received from OnControllerConnected
	 * @param Key The button FKey
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnControllerButtonReleased(uint8 ControllerIndex, FKey Key) = 0;

	/**
	 * @brief Notify the input handler of a controller disconnecting.
	 *
	 * @param ControlledIndex The index of the controller received from OnControllerConnected
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnControllerDisconnected(uint8 ControllerIndex) = 0;

	/**
	 * @brief Notify the input handler of receiving the Eye Views to use with XR streaming (usually received oncne at the start of a stream)
	 *
	 * @param LeftEyeTransform The transform of the left eye
	 * @param LeftEyeProjectionMatrix The projection matrix of the left eye
	 * @param RightEyeTransform The transform of the right eye
	 * @param RightEyeProjectionMatrix The projection matrix of the right eye
	 * @param HMDTransform The transform of the HMD itself
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXREyeViews(FTransform LeftEyeTransform, FMatrix LeftEyeProjectionMatrix, FTransform RightEyeTransform, FMatrix RightEyeProjectionMatrix, FTransform HMDTransform) = 0;

	/**
	 * @brief Notify the input handler of receiving the transform of the HMD to use with XR streaming (received once per frame that the HMD displays).
	 *
	 * @param HMDTransform The transform of the HMD itself
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRHMDTransform(FTransform HMDTransform) = 0;

	/**
	 * @brief Notify the input handler of receiving the transform of an XR controller to use with XR streaming (received once per frame that the HMD displays).
	 *
	 * @param ControllerTransform The transform of the controller
	 * @param Handedness Specifies the hand this transform applies to
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRControllerTransform(FTransform ControllerTransform, EControllerHand Handedness) = 0;

	/**
	 * @brief Notify the input handler of receiving a XR controller button touch event to use with XR streaming
	 *
	 * @param Handedness Specifies the hand this button touch event applies to
	 * @param Key The button FKey
	 * @param bIsRepeat Is this a repeat of a previous button press (ie has the button been held down)
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRButtonTouched(EControllerHand Handedness, FKey Key, bool bIsRepeat) = 0;

	/**
	 * @brief Notify the input handler of receiving a XR controller button touch release event to use with XR streaming
	 *
	 * @param Handedness Specifies the hand this button touch event applies to
	 * @param Key The button FKey
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRButtonTouchReleased(EControllerHand Handedness, FKey Key) = 0;

	/**
	 * @brief Notify the input handler of receiving a XR controller button press event to use with XR streaming
	 *
	 * @param Handedness Specifies the hand this button press event applies to
	 * @param Key The button FKey
	 * @param bIsRepeat Is this a repeat of a previous button press (ie has the button been held down)
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRButtonPressed(EControllerHand Handedness, FKey Key, bool bIsRepeat) = 0;

	/**
	 * @brief Notify the input handler of receiving a XR controller button release event to use with XR streaming
	 *
	 * @param Handedness Specifies the hand this button release event applies to
	 * @param Key The button FKey
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRButtonReleased(EControllerHand Handedness, FKey Key) = 0;

	/**
	 * @brief Notify the input handler of receiving a XR controller button release event to use with XR streaming
	 *
	 * @param Handedness Specifies the hand this button release event applies to
	 * @param Key The axis FKey
	 * @param AnalogValue The value of the axis in the range -1 to 1
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRAnalog(EControllerHand Handedness, FKey Key, double AnalogValue) = 0;

	/**
	 * @brief Notify the input handler of the headset system connected to the frontend to use with XR streaming
	 *
	 * @param System The system
	 *
	 * @return true if the event was handled. false otherwise
	 */
	virtual bool OnXRSystem(EPixelStreaming2XRSystem System) = 0;
};

#undef UE_API
