// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreaming2InputEnums.h"

#include "Containers/Array.h"

/**
 * A message that is sent over the Pixel Streaming data channel.
 */
class IPixelStreaming2InputMessage
{
public:
	/**
	 * Gets the id of the message.
	 * Note: All messages required a unique id.
	 * @return The id of the message.
	 */
	virtual uint8 GetID() const = 0;

	/**
	 * Get the message structure, e.g. the array of types that make up this message.
	 * Note: The types are packed tightly and sent in a byte buffer.
	 * @return The structure of the message.
	 */
	virtual TArray<EPixelStreaming2MessageTypes> GetStructure() const = 0;
};
