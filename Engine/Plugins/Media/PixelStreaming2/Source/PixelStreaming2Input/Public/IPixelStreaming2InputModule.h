// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"
#include "IPixelStreaming2InputHandler.h"

/**
 * Provides mechanism for managing input handlers and devices in Pixel Streaming.
 */
class IPixelStreaming2InputModule : public IInputDeviceModule
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed.
	 */
	static inline IPixelStreaming2InputModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreaming2InputModule>("PixelStreaming2Input");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable() { return FModuleManager::Get().IsModuleLoaded("PixelStreaming2Input"); }

	/**
	 * @brief Create a Input Handler object.
	 *
	 * @return TSharedPtr<IPixelStreaming2InputHandler> the input handler for this streamer.
	 */
	virtual TSharedPtr<IPixelStreaming2InputHandler> CreateInputHandler() = 0;

	/**
	 * Attempts to create a new input device interface
	 * @param InMessageHandler A reference to the message handler that will process input events from the device.
	 * @return	Interface to the new input device, if we were able to successfully create one.
	 */
	virtual TSharedPtr<class IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override = 0;
};
