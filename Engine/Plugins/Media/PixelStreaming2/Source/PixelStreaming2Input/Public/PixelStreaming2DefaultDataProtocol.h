// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "IPixelStreaming2DataProtocol.h"

namespace UE::PixelStreaming2Input
{

	/**
	 * @return The default "ToStreamer" data protocol.
	 */
	PIXELSTREAMING2INPUT_API TSharedPtr<IPixelStreaming2DataProtocol> GetDefaultToStreamerProtocol();

	/**
	 * @return The default "FromStreamer" data protocol.
	 */
	PIXELSTREAMING2INPUT_API TSharedPtr<IPixelStreaming2DataProtocol> GetDefaultFromStreamerProtocol();

} // namespace UE::PixelStreaming2Input