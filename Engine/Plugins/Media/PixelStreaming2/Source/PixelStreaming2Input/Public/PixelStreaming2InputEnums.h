// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * @brief The data types supported when sending messages across the data channel to/from peers.
 */
enum class EPixelStreaming2MessageTypes
{
	Uint8 = 0,
	Uint16 = 1,
	Int16 = 2,
	Float = 3,
	Double = 4,
	String = 5,
	Undefined = 6
};

/**
 * @brief The message directions
 */
enum class EPixelStreaming2MessageDirection : uint8
{
	ToStreamer = 0,
	FromStreamer = 1
};

/**
 * @brief The possible actions pixel streaming supports. These actions help differentiate input received from the browser.
 */
enum class EPixelStreaming2InputAction : uint8
{
	X = 0,
	Y = 1,
	Axis = 2,
	Click = 3,
	Touch = 4,
	None = 5,
};

/**
 * @brief The possible control schemes pixel streaming supports. RouteToWindow routes input at an application level. RouteToWidget routes input to a specific widget, ignoring the rest of the application.
 */
enum class EPixelStreaming2InputType : uint8
{
	RouteToWindow = 0,
	RouteToWidget = 1
};

/**
 * @brief Known message types for the default `ToStreamer` message protocol of Pixel Streaming.
 */
namespace EPixelStreaming2ToStreamerMessage
{
	PIXELSTREAMING2INPUT_API extern const FString IFrameRequest;
	PIXELSTREAMING2INPUT_API extern const FString RequestQualityControl;
	PIXELSTREAMING2INPUT_API extern const FString FpsRequest;
	PIXELSTREAMING2INPUT_API extern const FString AverageBitrateRequest;
	PIXELSTREAMING2INPUT_API extern const FString StartStreaming;
	PIXELSTREAMING2INPUT_API extern const FString StopStreaming;
	PIXELSTREAMING2INPUT_API extern const FString LatencyTest;
	PIXELSTREAMING2INPUT_API extern const FString RequestInitialSettings;
	PIXELSTREAMING2INPUT_API extern const FString TestEcho;
	PIXELSTREAMING2INPUT_API extern const FString UIInteraction;
	PIXELSTREAMING2INPUT_API extern const FString Command;
	PIXELSTREAMING2INPUT_API extern const FString TextboxEntry;
	PIXELSTREAMING2INPUT_API extern const FString KeyDown;
	PIXELSTREAMING2INPUT_API extern const FString KeyUp;
	PIXELSTREAMING2INPUT_API extern const FString KeyPress;
	PIXELSTREAMING2INPUT_API extern const FString MouseEnter;
	PIXELSTREAMING2INPUT_API extern const FString MouseLeave;
	PIXELSTREAMING2INPUT_API extern const FString MouseDown;
	PIXELSTREAMING2INPUT_API extern const FString MouseUp;
	PIXELSTREAMING2INPUT_API extern const FString MouseMove;
	PIXELSTREAMING2INPUT_API extern const FString MouseWheel;
	PIXELSTREAMING2INPUT_API extern const FString MouseDouble;
	PIXELSTREAMING2INPUT_API extern const FString TouchStart;
	PIXELSTREAMING2INPUT_API extern const FString TouchEnd;
	PIXELSTREAMING2INPUT_API extern const FString TouchMove;
	PIXELSTREAMING2INPUT_API extern const FString GamepadButtonPressed;
	PIXELSTREAMING2INPUT_API extern const FString GamepadButtonReleased;
	PIXELSTREAMING2INPUT_API extern const FString GamepadAnalog;;
	PIXELSTREAMING2INPUT_API extern const FString GamepadConnected;
	PIXELSTREAMING2INPUT_API extern const FString GamepadDisconnected;
	PIXELSTREAMING2INPUT_API extern const FString XREyeViews;
	PIXELSTREAMING2INPUT_API extern const FString XRHMDTransform;
	PIXELSTREAMING2INPUT_API extern const FString XRControllerTransform;
	PIXELSTREAMING2INPUT_API extern const FString XRButtonPressed;
	PIXELSTREAMING2INPUT_API extern const FString XRButtonTouched;
	PIXELSTREAMING2INPUT_API extern const FString XRButtonReleased;
	PIXELSTREAMING2INPUT_API extern const FString XRAnalog;
	PIXELSTREAMING2INPUT_API extern const FString XRSystem;
	PIXELSTREAMING2INPUT_API extern const FString XRButtonTouchReleased;
	PIXELSTREAMING2INPUT_API extern const FString Multiplexed;
	PIXELSTREAMING2INPUT_API extern const FString ChannelRelayStatus;
} // namespace EPixelStreaming2ToStreamerMessage

/**
 * @brief Known message types for the default `FromStreamer` message protocol of Pixel Streaming.
 */
namespace EPixelStreaming2FromStreamerMessage
{
	PIXELSTREAMING2INPUT_API extern const FString QualityControlOwnership;
	PIXELSTREAMING2INPUT_API extern const FString Response;
	PIXELSTREAMING2INPUT_API extern const FString Command;
	PIXELSTREAMING2INPUT_API extern const FString FreezeFrame;
	PIXELSTREAMING2INPUT_API extern const FString UnfreezeFrame;
	PIXELSTREAMING2INPUT_API extern const FString VideoEncoderAvgQP;
	PIXELSTREAMING2INPUT_API extern const FString LatencyTest;
	PIXELSTREAMING2INPUT_API extern const FString InitialSettings;
	PIXELSTREAMING2INPUT_API extern const FString FileExtension;
	PIXELSTREAMING2INPUT_API extern const FString FileMimeType;
	PIXELSTREAMING2INPUT_API extern const FString FileContents;
	PIXELSTREAMING2INPUT_API extern const FString TestEcho;
	PIXELSTREAMING2INPUT_API extern const FString InputControlOwnership;
	PIXELSTREAMING2INPUT_API extern const FString GamepadResponse;
	PIXELSTREAMING2INPUT_API extern const FString Multiplexed;
	PIXELSTREAMING2INPUT_API extern const FString Protocol;
} // namespace EPixelStreaming2FromStreamerMessage