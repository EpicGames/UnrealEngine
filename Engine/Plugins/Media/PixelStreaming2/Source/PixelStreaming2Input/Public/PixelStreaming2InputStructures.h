// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"

namespace UE::PixelStreaming2Input
{
	template <typename IntegerSequence>
	struct TPayloadParser;

	template <uint32... Indices>
	struct TPayloadParser<TIntegerSequence<uint32, Indices...>>
	{
		template <typename TupleType>
		static void Parse(TupleType&& Tuple, FArchive& Ar)
		{
			([](auto& Arg, FArchive& Ar) { Ar << Arg; }(Forward<TupleType>(Tuple).template Get<Indices>(), Ar), ...);
		}
	};

	template <typename... Types>
	struct TPayload : public TTuple<std::decay_t<Types>...>
	{
		TPayload(FArchive& Ar)
		{
			TPayloadParser<TMakeIntegerSequence<uint32, sizeof...(Types)>>::Parse(*this, Ar);
		}
	};
} // namespace UE::PixelStreaming2Input
