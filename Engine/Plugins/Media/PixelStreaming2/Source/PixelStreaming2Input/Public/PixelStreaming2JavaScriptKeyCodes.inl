// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"

namespace UE::PixelStreaming2Input 
{

/**
 * An array which will convert JavaScript key codes into FKeys. The JavaScript
 * key code should be used as an index into the array to perform the lookup.
 * https://www.cambiaresearch.com/articles/15/javascript-char-codes-key-codes
 * http://keycode.info/
 * Most key codes use the standard values given in the links above, but some
 * have been choosen specially for 'right' versions of the same key.
 */

	static const FKey* JavaScriptKeyCodeToFKey[256] =
	{
		/*   0 */ &EKeys::Invalid,
		/*   1 */ &EKeys::Invalid,
		/*   2 */ &EKeys::Invalid,
		/*   3 */ &EKeys::Invalid,
		/*   4 */ &EKeys::Invalid,
		/*   5 */ &EKeys::Invalid,
		/*   6 */ &EKeys::Invalid,
		/*   7 */ &EKeys::Invalid,
		/*   8 */ &EKeys::BackSpace,
		/*   9 */ &EKeys::Tab,
		/*  10 */ &EKeys::Invalid,
		/*  11 */ &EKeys::Invalid,
		/*  12 */ /*&EKeys::Clear*/ &EKeys::Invalid,
		/*  13 */ &EKeys::Enter,
		/*  14 */ &EKeys::Invalid,
		/*  15 */ &EKeys::Invalid,
		/*  16 */ &EKeys::LeftShift,
		/*  17 */ &EKeys::LeftControl,
		/*  18 */ &EKeys::LeftAlt,
		/*  19 */ &EKeys::Pause,
		/*  20 */ &EKeys::CapsLock,
		/*  21 */ &EKeys::Invalid,
		/*  22 */ &EKeys::Invalid,
		/*  23 */ &EKeys::Invalid,
		/*  24 */ &EKeys::Invalid,
		/*  25 */ &EKeys::Invalid,
		/*  26 */ &EKeys::Invalid,
		/*  27 */ &EKeys::Escape,
		/*  28 */ &EKeys::Invalid,
		/*  29 */ &EKeys::Invalid,
		/*  30 */ &EKeys::Invalid,
		/*  31 */ &EKeys::Invalid,
		/*  32 */ &EKeys::SpaceBar,
		/*  33 */ &EKeys::PageUp,
		/*  34 */ &EKeys::PageDown,
		/*  35 */ &EKeys::End,
		/*  36 */ &EKeys::Home,
		/*  37 */ &EKeys::Left,
		/*  38 */ &EKeys::Up,
		/*  39 */ &EKeys::Right,
		/*  40 */ &EKeys::Down,
		/*  41 */ &EKeys::Invalid,
		/*  42 */ &EKeys::Invalid,
		/*  43 */ &EKeys::Invalid,
		/*  44 */ /*&EKeys::PrintScreen*/ &EKeys::Invalid,
		/*  45 */ &EKeys::Insert,
		/*  46 */ &EKeys::Delete,
		/*  47 */ &EKeys::Invalid,
		/*  48 */ &EKeys::Zero,
		/*  49 */ &EKeys::One,
		/*  50 */ &EKeys::Two,
		/*  51 */ &EKeys::Three,
		/*  52 */ &EKeys::Four,
		/*  53 */ &EKeys::Five,
		/*  54 */ &EKeys::Six,
		/*  55 */ &EKeys::Seven,
		/*  56 */ &EKeys::Eight,
		/*  57 */ &EKeys::Nine,
		/*  58 */ &EKeys::Invalid,
		/*  59 */ &EKeys::Invalid,
		/*  60 */ &EKeys::Invalid,
		/*  61 */ &EKeys::Invalid,
		/*  62 */ &EKeys::Invalid,
		/*  63 */ &EKeys::Invalid,
		/*  64 */ &EKeys::Invalid,
		/*  65 */ &EKeys::A,
		/*  66 */ &EKeys::B,
		/*  67 */ &EKeys::C,
		/*  68 */ &EKeys::D,
		/*  69 */ &EKeys::E,
		/*  70 */ &EKeys::F,
		/*  71 */ &EKeys::G,
		/*  72 */ &EKeys::H,
		/*  73 */ &EKeys::I,
		/*  74 */ &EKeys::J,
		/*  75 */ &EKeys::K,
		/*  76 */ &EKeys::L,
		/*  77 */ &EKeys::M,
		/*  78 */ &EKeys::N,
		/*  79 */ &EKeys::O,
		/*  80 */ &EKeys::P,
		/*  81 */ &EKeys::Q,
		/*  82 */ &EKeys::R,
		/*  83 */ &EKeys::S,
		/*  84 */ &EKeys::T,
		/*  85 */ &EKeys::U,
		/*  86 */ &EKeys::V,
		/*  87 */ &EKeys::W,
		/*  88 */ &EKeys::X,
		/*  89 */ &EKeys::Y,
		/*  90 */ &EKeys::Z,
		/*  91 */ /*&EKeys::LeftWindowKey*/ &EKeys::Invalid,
		/*  92 */ /*&EKeys::RightWindowKey*/ &EKeys::Invalid,
		/*  93 */ /*&EKeys::SelectKey*/ &EKeys::Invalid,
		/*  94 */ &EKeys::Invalid,
		/*  95 */ &EKeys::Invalid,
		/*  96 */ &EKeys::NumPadZero,
		/*  97 */ &EKeys::NumPadOne,
		/*  98 */ &EKeys::NumPadTwo,
		/*  99 */ &EKeys::NumPadThree,
		/* 100 */ &EKeys::NumPadFour,
		/* 101 */ &EKeys::NumPadFive,
		/* 102 */ &EKeys::NumPadSix,
		/* 103 */ &EKeys::NumPadSeven,
		/* 104 */ &EKeys::NumPadEight,
		/* 105 */ &EKeys::NumPadNine,
		/* 106 */ &EKeys::Multiply,
		/* 107 */ &EKeys::Add,
		/* 108 */ &EKeys::Invalid,
		/* 109 */ &EKeys::Subtract,
		/* 110 */ &EKeys::Decimal,
		/* 111 */ &EKeys::Divide,
		/* 112 */ &EKeys::F1,
		/* 113 */ &EKeys::F2,
		/* 114 */ &EKeys::F3,
		/* 115 */ &EKeys::F4,
		/* 116 */ &EKeys::F5,
		/* 117 */ &EKeys::F6,
		/* 118 */ &EKeys::F7,
		/* 119 */ &EKeys::F8,
		/* 120 */ &EKeys::F9,
		/* 121 */ &EKeys::F10,
		/* 122 */ &EKeys::F11,
		/* 123 */ &EKeys::F12,
		/* 124 */ &EKeys::Invalid,
		/* 125 */ &EKeys::Invalid,
		/* 126 */ &EKeys::Invalid,
		/* 127 */ &EKeys::Invalid,
		/* 128 */ &EKeys::Invalid,
		/* 129 */ &EKeys::Invalid,
		/* 130 */ &EKeys::Invalid,
		/* 131 */ &EKeys::Invalid,
		/* 132 */ &EKeys::Invalid,
		/* 133 */ &EKeys::Invalid,
		/* 134 */ &EKeys::Invalid,
		/* 135 */ &EKeys::Invalid,
		/* 136 */ &EKeys::Invalid,
		/* 137 */ &EKeys::Invalid,
		/* 138 */ &EKeys::Invalid,
		/* 139 */ &EKeys::Invalid,
		/* 140 */ &EKeys::Invalid,
		/* 141 */ &EKeys::Invalid,
		/* 142 */ &EKeys::Invalid,
		/* 143 */ &EKeys::Invalid,
		/* 144 */ &EKeys::NumLock,
		/* 145 */ &EKeys::ScrollLock,
		/* 146 */ &EKeys::Invalid,
		/* 147 */ &EKeys::Invalid,
		/* 148 */ &EKeys::Invalid,
		/* 149 */ &EKeys::Invalid,
		/* 150 */ &EKeys::Invalid,
		/* 151 */ &EKeys::Invalid,
		/* 152 */ &EKeys::Invalid,
		/* 153 */ &EKeys::Invalid,
		/* 154 */ &EKeys::Invalid,
		/* 155 */ &EKeys::Invalid,
		/* 156 */ &EKeys::Invalid,
		/* 157 */ &EKeys::Invalid,
		/* 158 */ &EKeys::Invalid,
		/* 159 */ &EKeys::Invalid,
		/* 160 */ &EKeys::Invalid,
		/* 161 */ &EKeys::Invalid,
		/* 162 */ &EKeys::Invalid,
		/* 163 */ &EKeys::Invalid,
		/* 164 */ &EKeys::Invalid,
		/* 165 */ &EKeys::Invalid,
		/* 166 */ &EKeys::Invalid,
		/* 167 */ &EKeys::Invalid,
		/* 168 */ &EKeys::Invalid,
		/* 169 */ &EKeys::Invalid,
		/* 170 */ &EKeys::Invalid,
		/* 171 */ &EKeys::Invalid,
		/* 172 */ &EKeys::Invalid,
		/* 173 */ &EKeys::Invalid,
		/* 174 */ &EKeys::Invalid,
		/* 175 */ &EKeys::Invalid,
		/* 176 */ &EKeys::Invalid,
		/* 177 */ &EKeys::Invalid,
		/* 178 */ &EKeys::Invalid,
		/* 179 */ &EKeys::Invalid,
		/* 180 */ &EKeys::Invalid,
		/* 181 */ &EKeys::Invalid,
		/* 182 */ &EKeys::Invalid,
		/* 183 */ &EKeys::Invalid,
		/* 184 */ &EKeys::Invalid,
		/* 185 */ &EKeys::Invalid,
		/* 186 */ &EKeys::Semicolon,
		/* 187 */ &EKeys::Equals,
		/* 188 */ &EKeys::Comma,
		/* 189 */ &EKeys::Hyphen,
		/* 190 */ &EKeys::Period,
		/* 191 */ &EKeys::Slash,
		/* 192 */ &EKeys::Tilde,
		/* 193 */ &EKeys::Invalid,
		/* 194 */ &EKeys::Invalid,
		/* 195 */ &EKeys::Invalid,
		/* 196 */ &EKeys::Invalid,
		/* 197 */ &EKeys::Invalid,
		/* 198 */ &EKeys::Invalid,
		/* 199 */ &EKeys::Invalid,
		/* 200 */ &EKeys::Invalid,
		/* 201 */ &EKeys::Invalid,
		/* 202 */ &EKeys::Invalid,
		/* 203 */ &EKeys::Invalid,
		/* 204 */ &EKeys::Invalid,
		/* 205 */ &EKeys::Invalid,
		/* 206 */ &EKeys::Invalid,
		/* 207 */ &EKeys::Invalid,
		/* 208 */ &EKeys::Invalid,
		/* 209 */ &EKeys::Invalid,
		/* 210 */ &EKeys::Invalid,
		/* 211 */ &EKeys::Invalid,
		/* 212 */ &EKeys::Invalid,
		/* 213 */ &EKeys::Invalid,
		/* 214 */ &EKeys::Invalid,
		/* 215 */ &EKeys::Invalid,
		/* 216 */ &EKeys::Invalid,
		/* 217 */ &EKeys::Invalid,
		/* 218 */ &EKeys::Invalid,
		/* 219 */ &EKeys::LeftBracket,
		/* 220 */ &EKeys::Backslash,
		/* 221 */ &EKeys::RightBracket,
		/* 222 */ &EKeys::Apostrophe,
		/* 223 */ &EKeys::Tilde,
		/* 224 */ &EKeys::Invalid,
		/* 225 */ &EKeys::Invalid,
		/* 226 */ &EKeys::Invalid,
		/* 227 */ &EKeys::Invalid,
		/* 228 */ &EKeys::Invalid,
		/* 229 */ &EKeys::Invalid,
		/* 230 */ &EKeys::Invalid,
		/* 231 */ &EKeys::Invalid,
		/* 232 */ &EKeys::Invalid,
		/* 233 */ &EKeys::Invalid,
		/* 234 */ &EKeys::Invalid,
		/* 235 */ &EKeys::Invalid,
		/* 236 */ &EKeys::Invalid,
		/* 237 */ &EKeys::Invalid,
		/* 238 */ &EKeys::Invalid,
		/* 239 */ &EKeys::Invalid,
		/* 240 */ &EKeys::Invalid,
		/* 241 */ &EKeys::Invalid,
		/* 242 */ &EKeys::Invalid,
		/* 243 */ &EKeys::Invalid,
		/* 244 */ &EKeys::Invalid,
		/* 245 */ &EKeys::Invalid,
		/* 246 */ &EKeys::Invalid,
		/* 247 */ &EKeys::Invalid,
		/* 248 */ &EKeys::Invalid,
		/* 249 */ &EKeys::Invalid,
		/* 250 */ &EKeys::Invalid,
		/* 251 */ &EKeys::Invalid,
		/* 252 */ &EKeys::Invalid,

		// Special key codes.
		/* 253 */ &EKeys::RightShift,
		/* 254 */ &EKeys::RightControl,
		/* 255 */ &EKeys::RightAlt
	};
} // namespace UE::PixelStreaming2Input