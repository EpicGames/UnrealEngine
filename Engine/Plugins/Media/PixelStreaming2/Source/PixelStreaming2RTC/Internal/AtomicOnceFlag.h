// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <atomic>

namespace UE::PixelStreaming2
{
	class FAtomicOnceFlag
	{
	public:
		// Sets flag to true and returns if the flag had fired before
		inline bool Trigger() { return bFlag.exchange(true); };
	private:
		std::atomic<bool> bFlag = false;
	};
} // namespace UE::PixelStreaming2
