// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/UnrealMemory.h"

#include "epic_rtc/common/memory.h"

namespace UE::PixelStreaming2
{
	class FEpicRtcAllocator : public EpicRtcMemoryInterface
	{
	public:
		FEpicRtcAllocator() = default;
		virtual ~FEpicRtcAllocator() = default;

		[[nodiscard]] virtual void* Allocate(uint64_t Size, uint64_t Alignment, const char* Tag) override
		{
			return FMemory::Malloc(Size, Alignment);
		}

		[[nodiscard]] virtual void* Reallocate(void* Pointer, uint64_t Size, uint64_t Alignment, const char* Tag) override
		{
			return FMemory::Realloc(Pointer, Size, Alignment);
		}

		virtual void Free(void* Pointer) override
		{
			return FMemory::Free(Pointer);
		}
	};
} // namespace UE::PixelStreaming2
