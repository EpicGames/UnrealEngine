// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCapturer.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	/**
	 * FEpicRtcAudioCapturer overrides the default PushAudio behaviour of the FAudioCapturer in order to
	 * break up the pushed audio into 10ms chunks
	 */
	class FEpicRtcAudioCapturer : public FAudioCapturer
	{
	public:
		virtual ~FEpicRtcAudioCapturer() = default;

		// Override the push audio method as EpicRtc needs the broadcasted audio to be in 10ms chunks
		UE_API virtual void PushAudio(const float* AudioData, int32 NumSamples, int32 NumChannels, int32 SampleRate) override;

	protected:
		FEpicRtcAudioCapturer(const int SampleRate = 48000, const int NumChannels = 2, const float SampleSizeInSeconds = 0.5f)
			: FAudioCapturer(SampleRate, NumChannels, SampleSizeInSeconds) 
		{
		}

	private:
		TArray<int16_t> RecordingBuffer;

		// Needed so FAudioCapturer::Create can access the constructor
		friend FAudioCapturer;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
