// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSink.h"
#include "EpicRtcTrack.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	// Collects audio coming in from EpicRtc and passes into into UE's audio system.
	class FEpicRtcAudioSink : public FAudioSink, public TEpicRtcTrack<EpicRtcAudioTrackInterface>
	{
	public:
		static UE_API TSharedPtr<FEpicRtcAudioSink> Create(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack);
		virtual ~FEpicRtcAudioSink() = default;
	private:
		UE_API FEpicRtcAudioSink(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack);
	};
} // namespace UE::PixelStreaming2

#undef UE_API
