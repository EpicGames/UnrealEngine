// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSource.h"
#include "EpicRtcTrack.h"
#include "Templates/SharedPointer.h"

#include "epic_rtc/core/audio/audio_track.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	class FEpicRtcAudioSource : public FAudioSource, public TEpicRtcTrack<EpicRtcAudioTrackInterface>
	{
	public:
		static UE_API TSharedPtr<FEpicRtcAudioSource> Create(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack, TSharedPtr<class FEpicRtcAudioCapturer> InCapturer);

		UE_API virtual void OnAudioBuffer(const int16_t* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate);
	private:
		UE_API FEpicRtcAudioSource(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack);
	};
} // namespace UE::PixelStreaming2

#undef UE_API
