// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/audio/audio_track_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#include "EpicRtcAudioTrackObserver.generated.h"

#define UE_API PIXELSTREAMING2RTC_API

UINTERFACE(MinimalAPI)
class UPixelStreaming2AudioTrackObserver : public UInterface
{
	GENERATED_BODY()
};

class IPixelStreaming2AudioTrackObserver
{
	GENERATED_BODY()

public:
	virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) = 0;
	virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) = 0;
	virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) = 0;
};

namespace UE::PixelStreaming2
{
	class FEpicRtcAudioTrackObserver : public EpicRtcAudioTrackObserverInterface
	{
	public:
		UE_API FEpicRtcAudioTrackObserver(TObserverVariant<IPixelStreaming2AudioTrackObserver> UserObserver);
		virtual ~FEpicRtcAudioTrackObserver() = default;

	private:
		// Begin EpicRtcAudioTrackObserverInterface
		UE_API virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) override;
		UE_API virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) override;
		UE_API virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) override;
		// End EpicRtcAudioTrackObserverInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2AudioTrackObserver> UserObserver;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
