// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/audio/audio_track_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

class IPixelStreaming2AudioTrackObserver;

namespace UE::PixelStreaming2
{		
	class FEpicRtcAudioTrackObserverFactory : public EpicRtcAudioTrackObserverFactoryInterface
	{
	public:
		UE_API FEpicRtcAudioTrackObserverFactory(TObserverVariant<IPixelStreaming2AudioTrackObserver> UserObserver);
		virtual ~FEpicRtcAudioTrackObserverFactory() = default;

	public:
		// Begin EpicRtcAudioTrackObserverFactoryInterface
		UE_API virtual EpicRtcErrorCode CreateAudioTrackObserver(const EpicRtcStringView ParticipantId, const EpicRtcStringView AudioTrackId, EpicRtcAudioTrackObserverInterface** OutAudioTrackObserver) override;
		// End EpicRtcAudioTrackObserverFactoryInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2AudioTrackObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
