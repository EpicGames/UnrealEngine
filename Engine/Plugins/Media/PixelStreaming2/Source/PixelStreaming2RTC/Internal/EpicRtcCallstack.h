// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcTrace.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/UnrealMemory.h"
#include "HAL/RunnableThread.h"
#include "ProfilingDebugging/TagTrace.h"

#include "epic_rtc/core/callstack_api.h"

UE_TRACE_CHANNEL_EXTERN(EpicRtcProfilingChannel);

#define ENABLE_EPICRTC_TRACING (ENABLE_LOW_LEVEL_MEM_TRACKER && UE_MEMORY_TAGS_TRACE_ENABLED && UE_TRACE_ENABLED)

namespace UE::PixelStreaming2
{
#if ENABLE_EPICRTC_TRACING
	class FEpicRtcCallstack : public EpicRtcCallstackInterface
	{
		static constexpr bool USE_EPICRTC_PREFIX = false;
		static constexpr bool USE_EPICRTC_CHANNEL = true;

		virtual EpicRtcCallstackType SupportsEntryExit() const override 
		{ 
			return EpicRtcCallstackType::MEMORY | EpicRtcCallstackType::CPUPROFILE; 
		}

		class LocalLLMScope
		{
		public:
			LocalLLMScope(FName& ScopeName) 
                : LLMScope(ScopeName, false, ELLMTagSet::None, ELLMTracker::Default)
                , MemScope(ScopeName)  
            {
            }

			~LocalLLMScope() = default;
			
			// UE Insights handles scope with multiple objects, so it's easier to wrap them into this class
			FLLMScope LLMScope;
			FMemScope MemScope;
		};
		static_assert(sizeof(LocalLLMScope) <= EpicRtcCallstackInterface::LOCALSTACK_BYTES, "Insufficient stack bytes provided by EpicRtcCallstack");

		static FString GetScopeString(const char* InScopeName)
		{
			FString Name = InScopeName;
			// UE doesn't work with spaces, underscores, slashes or other special chars
			Name.ReplaceInline(TEXT(" "), TEXT(""));
			Name.ReplaceInline(TEXT("_"), TEXT(""));
			Name.ReplaceInline(TEXT("/"), TEXT(""));
			return Name;
		}

		virtual void OnEntry(const char* InScopeName, EpicRtcCallstackType InType, void* volatile* InLocalStatic, uint8_t InLocalStackStorage[EpicRtcCallstackInterface::LOCALSTACK_BYTES]) override
		{
			// Currently only supporting either MEMORY or CPUPROFILE, not both
			if (InType == EpicRtcCallstackType::MEMORY)
			{
				if (!*InLocalStatic)
				{
					// Create an FName for the scope name
					FName* ScopeFName = new FName("EpicRTC/" + GetScopeString(InScopeName));
					if (FPlatformAtomics::InterlockedCompareExchangePointer(InLocalStatic, (void*)ScopeFName, nullptr) != nullptr)
					{
						// Scope was created simultaneously on another thread, use the other one
						delete ScopeFName;
						ScopeFName = nullptr;
					}
				}
				// Construct in place using provided stack memory
				(void)new(InLocalStackStorage) LocalLLMScope(*(FName*)*InLocalStatic);
			}
			else if (InType == EpicRtcCallstackType::CPUPROFILE)
			{
				uint32 SpecId = 0;
				if (!*InLocalStatic)
				{
					if (USE_EPICRTC_PREFIX)
					{
						// Prefix with epicrtc as we do with memory so that all epicrt's functions are easily filtered
						FName ScopeFName(FString("EpicRTC/") + InScopeName);
						FCpuProfilerTrace::GetOrCreateSpecId(SpecId, ScopeFName, nullptr, 0);
					}
					else
					{
						FCpuProfilerTrace::GetOrCreateSpecId(SpecId, InScopeName, nullptr, 0);
					}
					*InLocalStatic = (void*)(size_t)SpecId; // Ok to overwrite more than once
				}
				else
				{
					SpecId = (uint32)(size_t)*InLocalStatic;
				}
				if (USE_EPICRTC_CHANNEL)
				{
					*InLocalStackStorage = CpuChannel | EpicRtcChannel;
					if (*InLocalStackStorage)
					{
						FCpuProfilerTrace::OutputBeginEvent(SpecId);
					}
				}
				else
				{
					FCpuProfilerTrace::OutputBeginEvent(SpecId);
				}
			}
		}
        
		virtual void OnExit(EpicRtcCallstackType InType, uint8_t InLocalStackStorage[EpicRtcCallstackInterface::LOCALSTACK_BYTES]) override
		{
			if (InType == EpicRtcCallstackType::MEMORY)
			{
				// Destruct in-place allocated class
				((LocalLLMScope*)InLocalStackStorage)->~LocalLLMScope();
			}
			else if (InType == EpicRtcCallstackType::CPUPROFILE)
			{
				if (!USE_EPICRTC_CHANNEL || *InLocalStackStorage)
				{
					FCpuProfilerTrace::OutputEndEvent();
				}
			}
		}
	};
#endif
}