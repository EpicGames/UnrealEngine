// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2Stats.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2Trace.h"
#include "TickableTask.h"

#include "epic_rtc/core/conference.h"

namespace UE::PixelStreaming2
{
	class FEpicRtcTickConferenceTask : public FPixelStreamingTickableTask
	{
	public:
		FEpicRtcTickConferenceTask(TRefCountPtr<EpicRtcConferenceInterface>& EpicRtcConference, const FString& TaskName = TEXT("EpicRtcTickConferenceTask"))
			: EpicRtcConference(EpicRtcConference)
			, TaskName(TaskName)
		{
		}

		virtual ~FEpicRtcTickConferenceTask() override
		{
			// We may get a call to destroy the task before we've had a chance to tick again.
			// So to be safe, we tick the conference a final time
			if (EpicRtcConference)
			{

				while (EpicRtcConference->NeedsTick())
				{
					EpicRtcConference->Tick();
				}
			}
		}

		// Begin FPixelStreamingTickableTask
		virtual void Tick(float DeltaMs) override
		{
			if (EpicRtcConference)
			{
				// Tick conference normally. This handles things like data channel message
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FEpicRtcTickConferenceTask::Tick GraphValue", PixelStreaming2Channel);
					IPixelStreaming2Stats::Get().GraphValue(TEXT("ConferenceTickInterval"), DeltaMs, 1, 0.f, 1.f);
				}

				while (EpicRtcConference->NeedsTick())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("EpicRtcConferenceInterface::Tick", PixelStreaming2Channel);
					EpicRtcConference->Tick();
				}
			}
		}

		virtual const FString& GetName() const override
		{
			return TaskName;
		}
		// End FPixelStreamingTickableTask

	private:
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
		FString									 TaskName;
	};

} // namespace UE::PixelStreaming2
