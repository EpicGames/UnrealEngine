// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "Video/GenericFrameInfo.h"
#include "Video/VideoEncoder.h"

#include "epic_rtc/core/video/video_buffer.h"
#include "epic_rtc/core/video/video_codec_info.h"
#include "epic_rtc/plugins/signalling/join_options.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

FORCEINLINE bool operator==(const EpicRtcVideoResolution& Lhs, const EpicRtcVideoResolution& Rhs)
{
	return Lhs._width == Rhs._width && Lhs._height == Rhs._height;
}

namespace UE::PixelStreaming2
{
	class FEpicRtcString : public EpicRtcStringInterface
	{
	public:
		FEpicRtcString() = default;
		FEpicRtcString(const FString& String)
			: String(String)
		{
		}

		virtual const char* Get() const override
		{
			return (const char*)*String;
		}

		virtual uint64_t Length() const override
		{
			return String.Len();
		}

	private:
		FUtf8String String;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcEncodedVideoBuffer : public EpicRtcEncodedVideoBufferInterface
	{
	public:
		FEpicRtcEncodedVideoBuffer() = default;
		FEpicRtcEncodedVideoBuffer(uint8_t* InData, uint64_t InSize)
			: Data(InData, InSize)
		{
		}
		virtual ~FEpicRtcEncodedVideoBuffer() = default;

		virtual const uint8_t* GetData() const override { return Data.GetData(); };
		virtual uint64_t	   GetSize() const override { return Data.Num(); };

	private:
		TArray<uint8> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcParameterPair : public EpicRtcParameterPairInterface
	{
	public:
		FEpicRtcParameterPair(EpicRtcStringInterface* Key, EpicRtcStringInterface* Value)
			: Key(Key)
			, Value(Value)
		{
		}

		virtual EpicRtcStringInterface* GetKey() override
		{
			return Key;
		}

		virtual EpicRtcStringInterface* GetValue() override
		{
			return Value;
		}

	private:
		TRefCountPtr<EpicRtcStringInterface> Key;
		TRefCountPtr<EpicRtcStringInterface> Value;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcParameterPairArray : public EpicRtcParameterPairArrayInterface
	{
	public:
		FEpicRtcParameterPairArray() = default;

		virtual ~FEpicRtcParameterPairArray()
		{
			for (auto& ParameterPair : Data)
			{
				if (ParameterPair != nullptr)
				{
					ParameterPair->Release();
				}
			}
		}

		FEpicRtcParameterPairArray(const TArray<TRefCountPtr<EpicRtcParameterPairInterface>>& ParameterPairs)
		{
			for (auto& ParameterPair : ParameterPairs)
			{
				Data.Add(ParameterPair.GetReference());
				if (ParameterPair.GetReference() != nullptr)
				{
					ParameterPair->AddRef();
				}
			}
		}

		FEpicRtcParameterPairArray(const TArray<EpicRtcParameterPairInterface*>& ParameterPairs)
		{
			for (auto& ParameterPair : ParameterPairs)
			{
				Data.Add(ParameterPair);
				if (ParameterPair != nullptr)
				{
					ParameterPair->AddRef();
				}
			}
		}

		FEpicRtcParameterPairArray(std::initializer_list<EpicRtcParameterPairInterface*> ParameterPairs)
		{
			for (auto& ParameterPair : ParameterPairs)
			{
				Data.Add(ParameterPair);
				if (ParameterPair != nullptr)
				{
					ParameterPair->AddRef();
				}
			}
		}

		virtual EpicRtcParameterPairInterface* const* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcParameterPairInterface** Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

		void Append(std::initializer_list<EpicRtcParameterPairInterface*> ParameterPairs)
		{
			for (auto& ParameterPair : ParameterPairs)
			{
				Data.Add(ParameterPair);
				if (ParameterPair != nullptr)
				{
					ParameterPair->AddRef();
				}
			}
		}

	private:
		TArray<EpicRtcParameterPairInterface*> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcScalabilityModeArray : public EpicRtcVideoScalabilityModeArrayInterface
	{
	public:
		FEpicRtcScalabilityModeArray() = default;
		virtual ~FEpicRtcScalabilityModeArray() = default;

		FEpicRtcScalabilityModeArray(const TArray<EpicRtcVideoScalabilityMode>& ScalabilityModes)
			: Data(ScalabilityModes)
		{
		}

		FEpicRtcScalabilityModeArray(std::initializer_list<EpicRtcVideoScalabilityMode> ScalabilityModes)
			: Data(ScalabilityModes)
		{
		}

		FEpicRtcScalabilityModeArray(const TArray<EScalabilityMode>& ScalabilityModes)
		{
			for (EScalabilityMode ScalabilityMode : ScalabilityModes)
			{
				Data.Add(static_cast<EpicRtcVideoScalabilityMode>(ScalabilityMode)); // HACK if the Enums become un-aligned
			}
		}

		virtual const EpicRtcVideoScalabilityMode* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcVideoScalabilityMode* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

		void Append(std::initializer_list<EpicRtcVideoScalabilityMode> ScalabilityModes)
		{
			Data.Append(ScalabilityModes);
		}

	private:
		TArray<EpicRtcVideoScalabilityMode> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcVideoCodecInfo : public EpicRtcVideoCodecInfoInterface
	{
	public:
		FEpicRtcVideoCodecInfo(EpicRtcVideoCodec Codec, EpicRtcParameterPairArrayInterface* Parameters = new FEpicRtcParameterPairArray(), EpicRtcVideoScalabilityModeArrayInterface* ScalabilityModes = new FEpicRtcScalabilityModeArray())
			: Codec(Codec)
			, Parameters(Parameters)
			, ScalabilityModes(ScalabilityModes)
		{
		}

		virtual ~FEpicRtcVideoCodecInfo() = default;

		EpicRtcVideoCodec GetCodec() override
		{
			return Codec;
		}

		virtual EpicRtcParameterPairArrayInterface* GetParameters() override
		{
			return Parameters;
		}

		EpicRtcVideoScalabilityModeArrayInterface* GetScalabilityModes() override
		{
			return ScalabilityModes;
		}

	private:
		EpicRtcVideoCodec										Codec;
		TRefCountPtr<EpicRtcParameterPairArrayInterface>		Parameters;
		TRefCountPtr<EpicRtcVideoScalabilityModeArrayInterface> ScalabilityModes;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FVideoCodecInfoArray : public EpicRtcVideoCodecInfoArrayInterface
	{
	public:
		FVideoCodecInfoArray() = default;

		virtual ~FVideoCodecInfoArray()
		{
			for (auto& Codec : Data)
			{
				if (Codec != nullptr)
				{
					Codec->Release();
				}
			}
		}

		FVideoCodecInfoArray(const TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>& Codecs)
		{
			for (auto& Codec : Codecs)
			{
				Data.Add(Codec.GetReference());
				if (Codec.GetReference() != nullptr)
				{
					Codec->AddRef();
				}
			}
		}

		FVideoCodecInfoArray(const TArray<EpicRtcVideoCodecInfoInterface*>& Codecs)
		{
			for (auto& Codec : Codecs)
			{
				Data.Add(Codec);
				if (Codec != nullptr)
				{
					Codec->AddRef();
				}
			}
		}

		FVideoCodecInfoArray(std::initializer_list<EpicRtcVideoCodecInfoInterface*> Codecs)
		{
			for (auto& Codec : Codecs)
			{
				Data.Add(Codec);
				if (Codec != nullptr)
				{
					Codec->AddRef();
				}
			}
		}

		virtual EpicRtcVideoCodecInfoInterface* const* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcVideoCodecInfoInterface** Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcVideoCodecInfoInterface*> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcVideoResolutionBitrateLimitsArray : public EpicRtcVideoResolutionBitrateLimitsArrayInterface
	{
	public:
		FEpicRtcVideoResolutionBitrateLimitsArray() = default;
		virtual ~FEpicRtcVideoResolutionBitrateLimitsArray() = default;

		FEpicRtcVideoResolutionBitrateLimitsArray(const TArray<EpicRtcVideoResolutionBitrateLimits>& BitrateLimits)
			: Data(BitrateLimits)
		{
		}

		FEpicRtcVideoResolutionBitrateLimitsArray(std::initializer_list<EpicRtcVideoResolutionBitrateLimits> BitrateLimits)
			: Data(BitrateLimits)
		{
		}

		virtual const EpicRtcVideoResolutionBitrateLimits* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcVideoResolutionBitrateLimits* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcVideoResolutionBitrateLimits> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcPixelFormatArray : public EpicRtcPixelFormatArrayInterface
	{
	public:
		FEpicRtcPixelFormatArray() = default;
		virtual ~FEpicRtcPixelFormatArray() = default;

		FEpicRtcPixelFormatArray(const TArray<EpicRtcPixelFormat>& PixelFormats)
			: Data(PixelFormats)
		{
		}

		FEpicRtcPixelFormatArray(std::initializer_list<EpicRtcPixelFormat> PixelFormats)
			: Data(PixelFormats)
		{
		}

		virtual const EpicRtcPixelFormat* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcPixelFormat* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcPixelFormat> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcVideoFrameTypeArray : public EpicRtcVideoFrameTypeArrayInterface
	{
	public:
		FEpicRtcVideoFrameTypeArray() = default;
		virtual ~FEpicRtcVideoFrameTypeArray() = default;

		FEpicRtcVideoFrameTypeArray(const TArray<EpicRtcVideoFrameType>& FrameTypes)
			: Data(FrameTypes)
		{
		}

		FEpicRtcVideoFrameTypeArray(std::initializer_list<EpicRtcVideoFrameType> FrameTypes)
			: Data(FrameTypes)
		{
		}

		virtual const EpicRtcVideoFrameType* Get() const override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcVideoFrameType> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcInt32Array : public EpicRtcInt32ArrayInterface
	{
	public:
		FEpicRtcInt32Array() = default;
		virtual ~FEpicRtcInt32Array() = default;

		FEpicRtcInt32Array(const TArray<int32_t>& Ints)
			: Data(Ints)
		{
		}

		FEpicRtcInt32Array(std::initializer_list<int32_t> Ints)
			: Data(Ints)
		{
		}

		virtual const int32_t* Get() const override
		{
			return Data.GetData();
		}

		virtual int32_t* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

		void Append(std::initializer_list<int32_t> Ints)
		{
			Data.Append(Ints);
		}

	private:
		TArray<int32_t> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcBoolArray : public EpicRtcBoolArrayInterface
	{
	public:
		FEpicRtcBoolArray() = default;
		virtual ~FEpicRtcBoolArray() = default;

		FEpicRtcBoolArray(const TArray<EpicRtcBool>& Bools)
			: Data(Bools)
		{
		}

		FEpicRtcBoolArray(const TArray<bool>& Bools)
		{
			Data.SetNum(Bools.Num());
			for (size_t i = 0; i < Bools.Num(); i++)
			{
				Data[i] = Bools[i];
			}
		}

		FEpicRtcBoolArray(std::initializer_list<EpicRtcBool> Bools)
			: Data(Bools)
		{
		}

		FEpicRtcBoolArray(std::initializer_list<bool> Bools)
		{
			for (auto& Bool : Bools)
			{
				Data.Add(Bool);
			}
		}

		virtual const EpicRtcBool* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcBool* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcBool> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcDecodeTargetIndicationArray : public EpicRtcDecodeTargetIndicationArrayInterface
	{
	public:
		FEpicRtcDecodeTargetIndicationArray() = default;
		virtual ~FEpicRtcDecodeTargetIndicationArray() = default;

		FEpicRtcDecodeTargetIndicationArray(const TArray<EpicRtcDecodeTargetIndication>& DecodeTargetIndications)
			: Data(DecodeTargetIndications)
		{
		}

		FEpicRtcDecodeTargetIndicationArray(std::initializer_list<EpicRtcDecodeTargetIndication> DecodeTargetIndications)
			: Data(DecodeTargetIndications)
		{
		}

		// Helper method for converting array AVCodecs' EDecodeTargetIndication to array of EpicRtc's EpicRtcDecodeTargetIndication
		FEpicRtcDecodeTargetIndicationArray(const TArray<EDecodeTargetIndication>& DecodeTargetIndications)
		{
			Data.SetNum(DecodeTargetIndications.Num());

			for (size_t i = 0; i < DecodeTargetIndications.Num(); i++)
			{
				switch (DecodeTargetIndications[i])
				{
					case EDecodeTargetIndication::NotPresent:
						Data[i] = EpicRtcDecodeTargetIndication::NotPresent;
						break;
					case EDecodeTargetIndication::Discardable:
						Data[i] = EpicRtcDecodeTargetIndication::Discardable;
						break;
					case EDecodeTargetIndication::Switch:
						Data[i] = EpicRtcDecodeTargetIndication::Switch;
						break;
					case EDecodeTargetIndication::Required:
						Data[i] = EpicRtcDecodeTargetIndication::Required;
						break;
					default:
						checkNoEntry();
				}
			}
		}

		virtual const EpicRtcDecodeTargetIndication* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcDecodeTargetIndication* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcDecodeTargetIndication> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcCodecBufferUsageArray : public EpicRtcCodecBufferUsageArrayInterface
	{
	public:
		FEpicRtcCodecBufferUsageArray() = default;
		virtual ~FEpicRtcCodecBufferUsageArray() = default;

		FEpicRtcCodecBufferUsageArray(const TArray<EpicRtcCodecBufferUsage>& CodecBufferUsages)
			: Data(CodecBufferUsages)
		{
		}

		FEpicRtcCodecBufferUsageArray(std::initializer_list<EpicRtcCodecBufferUsage> CodecBufferUsages)
			: Data(CodecBufferUsages)
		{
		}

		// Helper method for converting array AVCodecs' FCodecBufferUsage to array of EpicRtc's EpicRtcCodecBufferUsage
		FEpicRtcCodecBufferUsageArray(const TArray<FCodecBufferUsage>& CodecBufferUsages)
		{
			Data.SetNum(CodecBufferUsages.Num());
			for (size_t i = 0; i < CodecBufferUsages.Num(); i++)
			{
				const FCodecBufferUsage& CodecBufferUsage = CodecBufferUsages[i];
				Data[i] = EpicRtcCodecBufferUsage{
					._id = CodecBufferUsage.Id,
					._referenced = CodecBufferUsage.bReferenced,
					._updated = CodecBufferUsage.bUpdated
				};
			}
		}

		virtual const EpicRtcCodecBufferUsage* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcCodecBufferUsage* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcCodecBufferUsage> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcVideoResolutionArray : public EpicRtcVideoResolutionArrayInterface
	{
	public:
		FEpicRtcVideoResolutionArray() = default;
		virtual ~FEpicRtcVideoResolutionArray() = default;

		FEpicRtcVideoResolutionArray(const TArray<EpicRtcVideoResolution>& Resolutions)
			: Data(Resolutions)
		{
		}

		FEpicRtcVideoResolutionArray(std::initializer_list<EpicRtcVideoResolution> Resolutions)
			: Data(Resolutions)
		{
		}

		// Helper method for converting array AVCodecs' FResolution to array of EpicRtc's EpicRtcVideoResolution
		FEpicRtcVideoResolutionArray(const TArray<FIntPoint>& Resolutions)
		{
			Data.SetNum(Resolutions.Num());
			for (size_t i = 0; i < Resolutions.Num(); i++)
			{
				const FIntPoint& Resolution = Resolutions[i];
				Data[i] = EpicRtcVideoResolution{
					._width = Resolution.X,
					._height = Resolution.Y
				};
			}
		}

		virtual const EpicRtcVideoResolution* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcVideoResolution* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcVideoResolution> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcGenericFrameInfoArray : public EpicRtcGenericFrameInfoArrayInterface
	{
	public:
		FEpicRtcGenericFrameInfoArray() = default;

		virtual ~FEpicRtcGenericFrameInfoArray()
		{
			for (auto& GenericFrameInfo : Data)
			{
				if (GenericFrameInfo != nullptr)
				{
					GenericFrameInfo->Release();
				}
			}
		}

		FEpicRtcGenericFrameInfoArray(const TArray<TRefCountPtr<EpicRtcGenericFrameInfoInterface>>& GenericFrameInfos)
		{
			for (auto& GenericFrameInfo : GenericFrameInfos)
			{
				Data.Add(GenericFrameInfo.GetReference());
				if (GenericFrameInfo.GetReference() != nullptr)
				{
					GenericFrameInfo->AddRef();
				}
			}
		}

		FEpicRtcGenericFrameInfoArray(const TArray<EpicRtcGenericFrameInfoInterface*>& GenericFrameInfos)
		{
			for (auto& GenericFrameInfo : GenericFrameInfos)
			{
				Data.Add(GenericFrameInfo);
				if (GenericFrameInfo != nullptr)
				{
					GenericFrameInfo->AddRef();
				}
			}
		}

		FEpicRtcGenericFrameInfoArray(std::initializer_list<EpicRtcGenericFrameInfoInterface*> GenericFrameInfos)
		{
			for (auto& GenericFrameInfo : GenericFrameInfos)
			{
				Data.Add(GenericFrameInfo);
				if (GenericFrameInfo != nullptr)
				{
					GenericFrameInfo->AddRef();
				}
			}
		}

		virtual EpicRtcGenericFrameInfoInterface* const* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcGenericFrameInfoInterface** Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcGenericFrameInfoInterface*> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcGenericFrameInfo : public EpicRtcGenericFrameInfoInterface
	{
	public:
		FEpicRtcGenericFrameInfo(const FGenericFrameInfo& GenericFrameInfo)
			: SpatialId(GenericFrameInfo.SpatialId)
			, TemporalId(GenericFrameInfo.TemporalId)
			, DecodeTargetIndications(MakeRefCount<FEpicRtcDecodeTargetIndicationArray>(GenericFrameInfo.DecodeTargetIndications))
			, FrameDiffs(MakeRefCount<FEpicRtcInt32Array>(GenericFrameInfo.FrameDiffs))
			, ChainDiffs(MakeRefCount<FEpicRtcInt32Array>(GenericFrameInfo.ChainDiffs))
			, EncoderBuffers(MakeRefCount<FEpicRtcCodecBufferUsageArray>(GenericFrameInfo.EncoderBuffers))
			, PartOfChain(MakeRefCount<FEpicRtcBoolArray>(GenericFrameInfo.PartOfChain))
			, ActiveDecodeTargets(MakeRefCount<FEpicRtcBoolArray>(GenericFrameInfo.ActiveDecodeTargets))
		{
		}

		virtual ~FEpicRtcGenericFrameInfo() = default;

		virtual int32_t										 GetSpatialLayerId() override { return SpatialId; }
		virtual int32_t										 GetTemporalLayerId() override { return TemporalId; }
		virtual EpicRtcDecodeTargetIndicationArrayInterface* GetDecodeTargetIndications() override { return DecodeTargetIndications; }
		virtual EpicRtcInt32ArrayInterface*					 GetFrameDiffs() override { return FrameDiffs; }
		virtual EpicRtcInt32ArrayInterface*					 GetChainDiffs() override { return ChainDiffs; }
		virtual EpicRtcCodecBufferUsageArrayInterface*		 GetEncoderBufferUsages() override { return EncoderBuffers; }
		virtual EpicRtcBoolArrayInterface*					 GetPartOfChain() override { return PartOfChain; }
		virtual EpicRtcBoolArrayInterface*					 GetActiveDecodeTargets() override { return ActiveDecodeTargets; }

	private:
		int32_t											  SpatialId;
		int32_t											  TemporalId;
		TRefCountPtr<FEpicRtcDecodeTargetIndicationArray> DecodeTargetIndications;
		TRefCountPtr<FEpicRtcInt32Array>				  FrameDiffs;
		TRefCountPtr<FEpicRtcInt32Array>				  ChainDiffs;
		TRefCountPtr<FEpicRtcCodecBufferUsageArray>		  EncoderBuffers;
		TRefCountPtr<FEpicRtcBoolArray>					  PartOfChain;
		TRefCountPtr<FEpicRtcBoolArray>					  ActiveDecodeTargets;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcFrameDependencyStructure : public EpicRtcFrameDependencyStructure
	{
	public:
		FEpicRtcFrameDependencyStructure(const FFrameDependencyStructure& FrameDependencyStructure)
			: StructureId(FrameDependencyStructure.StructureId)
			, NumDecodeTargets(FrameDependencyStructure.NumDecodeTargets)
			, NumChains(FrameDependencyStructure.NumChains)
			, DecodeTargetProtectedByChain(MakeRefCount<FEpicRtcInt32Array>(FrameDependencyStructure.DecodeTargetProtectedByChain))
			, Resolutions(MakeRefCount<FEpicRtcVideoResolutionArray>(FrameDependencyStructure.Resolutions))
		{
			TArray<EpicRtcGenericFrameInfoInterface*> GenericFrameInfoArray;
			GenericFrameInfoArray.SetNum(FrameDependencyStructure.Templates.Num());

			for (size_t i = 0; i < FrameDependencyStructure.Templates.Num(); i++)
			{
				FGenericFrameInfo GenericFrameInfo;
				GenericFrameInfo.SpatialId = FrameDependencyStructure.Templates[i].SpatialId;
				GenericFrameInfo.TemporalId = FrameDependencyStructure.Templates[i].TemporalId;
				GenericFrameInfo.DecodeTargetIndications = FrameDependencyStructure.Templates[i].DecodeTargetIndications;
				GenericFrameInfo.FrameDiffs = FrameDependencyStructure.Templates[i].FrameDiffs;
				GenericFrameInfo.ChainDiffs = FrameDependencyStructure.Templates[i].ChainDiffs;

				GenericFrameInfoArray[i] = new FEpicRtcGenericFrameInfo(GenericFrameInfo);
			}

			Templates = MakeRefCount<FEpicRtcGenericFrameInfoArray>(GenericFrameInfoArray);
		}

		virtual ~FEpicRtcFrameDependencyStructure() = default;

		virtual int32_t								   GetStructureId() override { return StructureId; }
		virtual int32_t								   GetNumDecodeTargets() override { return NumDecodeTargets; }
		virtual int32_t								   GetNumChains() override { return NumChains; }
		virtual EpicRtcInt32ArrayInterface*			   GetDecodeTargetProtectedByChain() override { return DecodeTargetProtectedByChain; }
		virtual EpicRtcVideoResolutionArrayInterface*  GetResolutions() override { return Resolutions; }
		virtual EpicRtcGenericFrameInfoArrayInterface* GetTemplates() override { return Templates; }

		friend bool operator==(FEpicRtcFrameDependencyStructure& Lhs, FEpicRtcFrameDependencyStructure& Rhs)
		{
			TArray<int32_t> LhsDecodeTargetProtectedByChain(Lhs.GetDecodeTargetProtectedByChain()->Get(), Lhs.GetDecodeTargetProtectedByChain()->Size());
			TArray<int32_t> RhsDecodeTargetProtectedByChain(Rhs.GetDecodeTargetProtectedByChain()->Get(), Rhs.GetDecodeTargetProtectedByChain()->Size());

			TArray<EpicRtcVideoResolution> LhsResolutions(Lhs.GetResolutions()->Get(), Lhs.GetResolutions()->Size());
			TArray<EpicRtcVideoResolution> RhsResolutions(Rhs.GetResolutions()->Get(), Rhs.GetResolutions()->Size());

			TArray<EpicRtcGenericFrameInfoInterface*> LhsTemplates(Lhs.GetTemplates()->Get(), Lhs.GetTemplates()->Size());
			TArray<EpicRtcGenericFrameInfoInterface*> RhsTemplates(Rhs.GetTemplates()->Get(), Rhs.GetTemplates()->Size());

			return Lhs.NumDecodeTargets == Rhs.NumDecodeTargets
				&& Lhs.NumChains == Rhs.NumChains
				&& LhsDecodeTargetProtectedByChain == RhsDecodeTargetProtectedByChain
				&& LhsResolutions == RhsResolutions
				&& LhsTemplates == RhsTemplates;
		}

	private:
		int											StructureId;
		int											NumDecodeTargets;
		int											NumChains;
		TRefCountPtr<FEpicRtcInt32Array>			DecodeTargetProtectedByChain;
		TRefCountPtr<FEpicRtcVideoResolutionArray>	Resolutions;
		TRefCountPtr<FEpicRtcGenericFrameInfoArray> Templates;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

	class FEpicRtcSignallingJoinOptionsArray : public EpicRtcSignallingJoinOptionsArrayInterface
	{
	public:
		FEpicRtcSignallingJoinOptionsArray() = default;
		virtual ~FEpicRtcSignallingJoinOptionsArray() = default;

		FEpicRtcSignallingJoinOptionsArray(const TArray<EpicRtcSignallingJoinOptions>& SignallingJoinOptions)
			: Data(SignallingJoinOptions)
		{
		}

		FEpicRtcSignallingJoinOptionsArray(std::initializer_list<EpicRtcSignallingJoinOptions> SignallingJoinOptions)
			: Data(SignallingJoinOptions)
		{
		}

		virtual const EpicRtcSignallingJoinOptions* Get() const override
		{
			return Data.GetData();
		}

		virtual EpicRtcSignallingJoinOptions* Get() override
		{
			return Data.GetData();
		}

		virtual uint64_t Size() const override
		{
			return Data.Num();
		}

	private:
		TArray<EpicRtcSignallingJoinOptions> Data;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};
} // namespace UE::PixelStreaming2
