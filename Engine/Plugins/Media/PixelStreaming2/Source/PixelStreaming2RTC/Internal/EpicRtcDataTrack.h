// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcTrack.h"
#include "IPixelStreaming2DataProtocol.h"
#include "Templates/SharedPointer.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	template <typename T>
	inline size_t GetByteSizeOf(const T&& Value)
	{
		// Requires being r-value reference because the string and array specialization
		// is not always detected and forwarding allows the right specialization to be used.
		static_assert(std::is_rvalue_reference<decltype(Value)>::value);
		return sizeof(Value);
	}

	template <typename T>
	inline const void* GetDataPointer(const T&& Value)
	{
		// Requires being r-value reference because the string and array specialization
		// is not always detected and forwarding allows the right specialization to be used.
		static_assert(std::is_rvalue_reference<decltype(Value)>::value);
		return &Value;
	}

	inline size_t GetByteSizeOf(const FString&& Value)
	{
		// Requires being r-value reference because the string and array specialization
		// is not always detected and forwarding allows the right specialization to be used.
		static_assert(std::is_rvalue_reference<decltype(Value)>::value);
		return Value.Len() * sizeof(TCHAR);
	}

	inline const void* GetDataPointer(const FString&& Value)
	{
		// Requires being r-value reference because the string and array specialization
		// is not always detected and forwarding allows the right specialization to be used.
		static_assert(std::is_rvalue_reference<decltype(Value)>::value);
		return *Value;
	}

	template <typename T>
	inline size_t GetByteSizeOf(const TArray<T>&& Value)
	{
		// Requires being r-value reference because the string and array specialization
		// is not always detected and forwarding allows the right specialization to be used.
		static_assert(std::is_rvalue_reference<decltype(Value)>::value);
		return Value.Num() * sizeof(T);
	}

	template <typename T>
	inline const void* GetDataPointer(const TArray<T>&& Value)
	{
		// Requires being r-value reference because the string and array specialization
		// is not always detected and forwarding allows the right specialization to be used.
		static_assert(std::is_rvalue_reference<decltype(Value)>::value);
		return Value.GetData();
	}

	struct FBufferBuilder
	{
		TArray<uint8> Buffer;
		size_t		  Pos;

		FBufferBuilder(size_t size)
			: Pos(0)
		{
			Buffer.SetNum(size);
		}

		size_t Serialize(const void* Data, size_t DataSize)
		{
			check(Pos + DataSize <= Buffer.Num());
			FMemory::Memcpy(Buffer.GetData() + Pos, Data, DataSize);
			return Pos + DataSize;
		}

		template <typename T>
		void Insert(const T&& Value)
		{
			// Requires being r-value reference because the string and array specialization
			// is not always detected and forwarding allows the right specialization to be used.
			static_assert(std::is_rvalue_reference<decltype(Value)>::value);

			const size_t VSize = GetByteSizeOf(Forward<const T>(Value));
			const void*	 VLoc = GetDataPointer(Forward<const T>(Value));
			check(Pos + VSize <= Buffer.Num());
			FMemory::Memcpy(Buffer.GetData() + Pos, VLoc, VSize);
			Pos += VSize;
		}
	};

	class FEpicRtcDataTrack : public TEpicRtcTrack<EpicRtcDataTrackInterface>, public TSharedFromThis<FEpicRtcDataTrack>
	{
	public:
		static UE_API TSharedPtr<FEpicRtcDataTrack> Create(TRefCountPtr<EpicRtcDataTrackInterface> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol);

		virtual ~FEpicRtcDataTrack() = default;

		void SetDataProtocol(TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol)
		{
			WeakDataProtocol = InDataProtocol;
		}

		/**
		 * Sends a series of arguments to the data channel with the given type.
		 * @param MessageType The name of the message you want to send. This message name must be registered IPixelStreaming2InputHandler::GetFromStreamerProtocol().
		 * @returns True of the message was successfully sent.
		 */
		template <typename... Args>
		bool SendMessage(FString MessageType, Args... VarArgs) const
		{
			if (!IsActive())
			{
				return false;
			}

			uint8 MessageId;
			if (!GetMessageId(MessageType, MessageId))
			{
				return false;
			}

			FBufferBuilder Builder = EncodeMessage(MessageId, Forward<Args>(VarArgs)...);

			return Send(Builder.Buffer);
		}

		/**
		 * Sends a large buffer of data to the data track, will chunk into multiple data frames if frames greater than 16KB.
		 * @param MessageType The name of the message, it must be registered in IPixelStreaming2InputHandler::GetTo/FromStreamerProtocol()
		 * @param DataBytes The raw byte buffer to send.
		 * @returns True of the message was successfully sent.
		 */
		UE_API bool SendArbitraryData(const FString& MessageType, const TArray64<uint8>& DataBytes) const;

		/**
		 * @return The state of the underlying EpicRtc data track.
		 */
		EpicRtcTrackState GetState() const { return Track->GetState(); }

		void SetSendTrack(TRefCountPtr<EpicRtcDataTrackInterface> InSendTrack) { SendTrack = InSendTrack; }

	protected:
		UE_API FEpicRtcDataTrack(TRefCountPtr<EpicRtcDataTrackInterface> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol);
		UE_API FEpicRtcDataTrack(TSharedPtr<FEpicRtcDataTrack> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol);

		virtual void PrependData(FBufferBuilder& Builder) const {};

		UE_API bool IsActive() const;

		UE_API bool GetMessageId(const FString& MessageType, uint8& OutMessageId) const;

	private:
		/**
		 * Track that is used for sending data with Consumer/Producer architecture.
		 */
		TRefCountPtr<EpicRtcDataTrackInterface> SendTrack;

		TWeakPtr<IPixelStreaming2DataProtocol> WeakDataProtocol;

		UE_API bool Send(TArray<uint8>& Buffer) const;

		template <typename... Args>
		FBufferBuilder EncodeMessage(uint8 Type, Args... VarArgs) const
		{

			FBufferBuilder Builder(sizeof(Type) + (0 + ... + GetByteSizeOf(Forward<Args>(VarArgs))));
			PrependData(Builder);
			Builder.Insert(Forward<uint8>(Type));
			(Builder.Insert(Forward<Args>(VarArgs)), ...);

			return MoveTemp(Builder);
		}
	};

	class FEpicRtcMutliplexDataTrack : public FEpicRtcDataTrack
	{
	public:
		static UE_API TSharedPtr<FEpicRtcMutliplexDataTrack> Create(TSharedPtr<FEpicRtcDataTrack> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol, const FString& InPlayerId);

	protected:
		UE_API FEpicRtcMutliplexDataTrack(TSharedPtr<FEpicRtcDataTrack> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol, const FString& InPlayerId);

		UE_API virtual void PrependData(FBufferBuilder& Builder) const override;

	private:
		const FString PlayerId;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
