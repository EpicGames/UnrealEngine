// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/data_track_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#include "EpicRtcDataTrackObserver.generated.h"

#define UE_API PIXELSTREAMING2RTC_API

UINTERFACE(MinimalAPI)
class UPixelStreaming2DataTrackObserver : public UInterface
{
	GENERATED_BODY()
};

class IPixelStreaming2DataTrackObserver
{
	GENERATED_BODY()

public:
	virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) = 0;
	virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) = 0;
	virtual void OnDataTrackError(EpicRtcDataTrackInterface*, const EpicRtcErrorCode) = 0;
};

namespace UE::PixelStreaming2
{
	class FEpicRtcDataTrackObserver : public EpicRtcDataTrackObserverInterface
	{
	public:
		UE_API FEpicRtcDataTrackObserver(TObserverVariant<IPixelStreaming2DataTrackObserver> UserObserver);
		virtual ~FEpicRtcDataTrackObserver() = default;

	private:
		// Begin EpicRtcDataTrackObserverInterface
		UE_API virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) override;
		UE_API virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) override;
		UE_API virtual void OnDataTrackError(EpicRtcDataTrackInterface*, const EpicRtcErrorCode) override;
		// End EpicRtcDataTrackObserverInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2DataTrackObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
