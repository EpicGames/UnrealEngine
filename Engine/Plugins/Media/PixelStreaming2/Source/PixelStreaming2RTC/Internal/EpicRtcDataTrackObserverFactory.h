// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/data_track_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

class IPixelStreaming2DataTrackObserver;

namespace UE::PixelStreaming2
{
	class FEpicRtcDataTrackObserverFactory : public EpicRtcDataTrackObserverFactoryInterface
	{
	public:
		UE_API FEpicRtcDataTrackObserverFactory(TObserverVariant<IPixelStreaming2DataTrackObserver> UserObserver);
		virtual ~FEpicRtcDataTrackObserverFactory() = default;

	public:
		// Begin EpicRtcDataTrackObserverFactoryInterface
		UE_API virtual EpicRtcErrorCode CreateDataTrackObserver(const EpicRtcStringView ParticipantId, const EpicRtcStringView DataTrackId, EpicRtcDataTrackObserverInterface** OutDataTrackObserver) override;
		// End EpicRtcDataTrackObserverFactoryInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2DataTrackObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
