// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Internationalization/Regex.h"
#include "Logging/LogMacros.h"

#include "epic_rtc/common/logging.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

PIXELSTREAMING2RTC_API DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreaming2EpicRtc, Log, All);
PIXELSTREAMING2RTC_API DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreaming2WebRtc, Fatal, All);

namespace UE::PixelStreaming2
{
	constexpr EpicRtcLogLevel UnrealLogToEpicRtcCategoryMap[] = {
		EpicRtcLogLevel::Off,
		EpicRtcLogLevel::Critical,
		EpicRtcLogLevel::Error,
		EpicRtcLogLevel::Warning,
		EpicRtcLogLevel::Info,
		EpicRtcLogLevel::Info,
		EpicRtcLogLevel::Debug,
		EpicRtcLogLevel::Trace,
		EpicRtcLogLevel::Trace
	};

	static_assert(EpicRtcLogLevel::Off == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::NoLogging]);
	static_assert(EpicRtcLogLevel::Critical == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::Fatal]);
	static_assert(EpicRtcLogLevel::Error == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::Error]);
	static_assert(EpicRtcLogLevel::Warning == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::Warning]);
	static_assert(EpicRtcLogLevel::Info == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::Display]);
	static_assert(EpicRtcLogLevel::Info == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::Log]);
	static_assert(EpicRtcLogLevel::Debug == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::Verbose]);
	static_assert(EpicRtcLogLevel::Trace == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::VeryVerbose]);
	static_assert(EpicRtcLogLevel::Trace == UnrealLogToEpicRtcCategoryMap[ELogVerbosity::Type::All]);

	/**
	 * An interface providing functionality for manipulating EpicRtc logs.
	 *
	 * This includes completely removing log messages by checking IsFiltered()
	 * or by redacting sensitive information using Censor()
	 */
	class ILogManipulator
	{
	public:
		virtual ~ILogManipulator() = default;

		/**
		 * Checks if a log message should or shouldn't be logged.
		 *
		 * @return true is the log message is filtered and shouldn't be displayed
		 */
		virtual bool IsFiltered(ELogVerbosity::Type LogVerbosity, const FString& LogString) = 0;

		virtual FString Censor(ELogVerbosity::Type LogVerbosity, const FString& LogString) = 0;
	};

	class FEpicRtcLogFilter : public ILogManipulator
	{
	public:
		// Default constructor. Populates the contents of RegexPatterns with values from CVarEpicRtcLogFilter
		UE_API FEpicRtcLogFilter();
		// Alternate constructor that takes in a pre-populated array. Used for internal testing
		UE_API FEpicRtcLogFilter(const TArray<FRegexPattern>& RegexPatterns);

		UE_API virtual ~FEpicRtcLogFilter();

		UE_API virtual bool IsFiltered(ELogVerbosity::Type LogVerbosity, const FString& LogString) override;

		virtual FString Censor(ELogVerbosity::Type LogVerbosity, const FString& LogString) override { return LogString; }

	private:
		UE_API void OnEpicRtcLogFilterChanged(IConsoleVariable* Var);
		UE_API void ParseFilterString(const FString& LogFilterString);

	private:
		TArray<FRegexPattern> RegexPatterns;

		FDelegateHandle EpicRtcLogFilterChangedHandle;
	};

	class FEpicRtcLogsRedirector : public EpicRtcLoggerInterface
	{
	public:
		UE_API FEpicRtcLogsRedirector(TSharedPtr<ILogManipulator> LogManipulator = nullptr);

		UE_API virtual void Log(const EpicRtcLogMessage& Message) override;

	private:
		TSharedPtr<ILogManipulator> LogManipulator;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};
} // namespace UE::PixelStreaming2

#undef UE_API
