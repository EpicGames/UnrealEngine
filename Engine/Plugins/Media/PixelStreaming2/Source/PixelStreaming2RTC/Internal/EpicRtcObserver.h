// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "UObject/Interface.h"
#include "UObject/WeakInterfacePtr.h"

#include <type_traits>

namespace UE::PixelStreaming2
{
	template <typename T>
	class TObserverVariant
	{
	public:
		TObserverVariant() = default;

		TObserverVariant(TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>> ObserverVariant)
			: ObserverVariant(ObserverVariant)
		{
		}

		T* operator->() const
		{
			switch (ObserverVariant.GetIndex())
			{
				case TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>>::template IndexOfType<TWeakPtr<T>>():
				{
					if (TSharedPtr<T> PinnedUserObserver = ObserverVariant.template Get<TWeakPtr<T>>().Pin())
					{
						return PinnedUserObserver.Get();
					}
					break;
				}
				case TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>>::template IndexOfType<TWeakInterfacePtr<T>>():
				{
					TWeakInterfacePtr<T> WeakObserver = ObserverVariant.template Get<TWeakInterfacePtr<T>>();
					if (WeakObserver.IsValid())
					{
						return WeakObserver.Get();
					}
					break;
				}
				default:
					checkNoEntry();
			}

			return nullptr;
		}

		operator bool() const
		{
			switch (ObserverVariant.GetIndex())
			{
				case TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>>::template IndexOfType<TWeakPtr<T>>():
				{
					return ObserverVariant.template Get<TWeakPtr<T>>().IsValid();
				}
				case TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>>::template IndexOfType<TWeakInterfacePtr<T>>():
				{
					return ObserverVariant.template Get<TWeakInterfacePtr<T>>().IsValid();
				}
				default:
					checkNoEntry();
			}
			return false;
		}

	private:
		TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>> ObserverVariant;
	};

	template <typename T>
	inline TObserverVariant<T> TObserver(TWeakPtr<T> WeakObserver)
	{
		return TObserverVariant<T>(TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>>(TInPlaceType<TWeakPtr<T>>(), WeakObserver));
	}

	template <typename T>
	inline TObserverVariant<T> TObserver(TWeakInterfacePtr<T> WeakObserver)
	{
		return TObserverVariant<T>(TVariant<TYPE_OF_NULLPTR, TWeakPtr<T>, TWeakInterfacePtr<T>>(TInPlaceType<TWeakInterfacePtr<T>>(), WeakObserver));
	}
} // namespace UE::PixelStreaming2