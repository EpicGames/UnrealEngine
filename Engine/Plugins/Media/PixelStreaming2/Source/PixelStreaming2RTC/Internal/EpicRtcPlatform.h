// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "epic_rtc/core/platform.h"

#if PLATFORM_ANDROID
	
    #include "Android/AndroidApplication.h"
    
	#if USE_ANDROID_JNI
		#include "Android/AndroidJNI.h"
	#endif // USE_ANDROID_JNI

    namespace UE::PixelStreaming2
    {
        class FEpicRtcAndroidPlatform : public EpicRtcAndroidPlatformInterface
        {
            PIXELSTREAMING2RTC_API virtual JavaVM* GetJavaVM() override;
            
            virtual ~FEpicRtcAndroidPlatform() = default;
        };
    }
    
#endif // PLATFORM_ANDROID
