// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/room_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#include "EpicRtcRoomObserver.generated.h"

#define UE_API PIXELSTREAMING2RTC_API

UINTERFACE(MinimalAPI)
class UPixelStreaming2RoomObserver : public UInterface
{
	GENERATED_BODY()
};

class IPixelStreaming2RoomObserver
{
	GENERATED_BODY()

public:
	virtual void							   OnRoomStateUpdate(const EpicRtcRoomState State) = 0;
	virtual void							   OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) = 0;
	virtual void							   OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) = 0;
	virtual void							   OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) = 0;
	virtual void							   OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) = 0;
	virtual void							   OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) = 0;
	[[nodiscard]] virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) = 0;
	[[nodiscard]] virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) = 0;
	virtual void							   OnRoomErrorUpdate(const EpicRtcErrorCode Error) = 0;
};

namespace UE::PixelStreaming2
{
	class FEpicRtcRoomObserver : public EpicRtcRoomObserverInterface
	{
	public:
		UE_API FEpicRtcRoomObserver(TObserverVariant<IPixelStreaming2RoomObserver> UserObserver);
		virtual ~FEpicRtcRoomObserver() = default;

	private:
		// Begin EpicRtcRoomObserver
		UE_API virtual void							   OnRoomStateUpdate(const EpicRtcRoomState State) override;
		UE_API virtual void							   OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) override;
		UE_API virtual void							   OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) override;
		UE_API virtual void							   OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) override;
		UE_API virtual void							   OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) override;
		UE_API virtual void							   OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) override;
		[[nodiscard]] UE_API virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		[[nodiscard]] UE_API virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		UE_API virtual void							   OnRoomErrorUpdate(const EpicRtcErrorCode Error) override;
		// Begin EpicRtcRoomObserver

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2RoomObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
