// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/session_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#include "EpicRtcSessionObserver.generated.h"

#define UE_API PIXELSTREAMING2RTC_API

UINTERFACE(MinimalAPI)
class UPixelStreaming2SessionObserver : public UInterface
{
	GENERATED_BODY()
};

class IPixelStreaming2SessionObserver
{
	GENERATED_BODY()

public:
	virtual void OnSessionStateUpdate(const EpicRtcSessionState State) = 0;
	virtual void OnSessionErrorUpdate(const EpicRtcErrorCode Error) = 0;
	virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) = 0;
};

namespace UE::PixelStreaming2
{
	class FEpicRtcSessionObserver : public EpicRtcSessionObserverInterface
	{
	public:
		UE_API FEpicRtcSessionObserver(TObserverVariant<IPixelStreaming2SessionObserver> UserObserver);
		virtual ~FEpicRtcSessionObserver() = default;

	private:
		// Begin EpicRtcSessionObserver
		UE_API virtual void OnSessionStateUpdate(const EpicRtcSessionState State) override;
		UE_API virtual void OnSessionErrorUpdate(const EpicRtcErrorCode Error) override;
		UE_API virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) override;
		// End EpicRtcSessionObserver
	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	private:
		TObserverVariant<IPixelStreaming2SessionObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
