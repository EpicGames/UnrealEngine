// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"

#include "epic_rtc/core/stats.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	class FEpicRtcStatsCollector : public EpicRtcStatsCollectorCallbackInterface
	{
	public:
		FEpicRtcStatsCollector() = default;
		~FEpicRtcStatsCollector() = default;

		// Begin EpicRtcStatsCollectorCallbackInterface interface
		UE_API void			 OnStatsDelivered(const EpicRtcStatsReport& InReport) override;
		// End EpicRtcStatsCollectorCallbackInterface interface

		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcStatsCollectorCallbackInterface interface
	public:

		DECLARE_EVENT_TwoParams(FEpicRtcStatsCollector, FOnStatsReady, const FString&, const EpicRtcConnectionStats&);
		FOnStatsReady OnStatsReady;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
