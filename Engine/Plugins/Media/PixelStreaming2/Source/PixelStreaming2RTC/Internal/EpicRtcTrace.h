// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Trace.h"

UE_TRACE_CHANNEL_EXTERN(EpicRtcChannel, PIXELSTREAMING2RTC_API);