// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Templates/RefCounting.h"

#include "epic_rtc/core/audio/audio_track.h"
#include "epic_rtc/core/data_track.h"
#include "epic_rtc/core/video/video_track.h"

namespace UE::PixelStreaming2
{
	template <typename TrackInterface UE_REQUIRES(std::is_same_v<TrackInterface, EpicRtcAudioTrackInterface> || std::is_same_v<TrackInterface, EpicRtcDataTrackInterface> || std::is_same_v<TrackInterface, EpicRtcVideoTrackInterface>)>
	class TEpicRtcTrack
	{
	public:
		TEpicRtcTrack(TrackInterface* Track)
			: Track(Track)
		{
		}

		virtual ~TEpicRtcTrack() = default;

		/**
		 * @return The id of the underlying EpicRtc track.
		 */
		EpicRtcStringView GetTrackId() const
		{
			return Track ? Track->GetId() : EpicRtcStringView{ ._ptr = nullptr, ._length = 0 };
		}

	protected:
		TRefCountPtr<TrackInterface> Track;
	};
} // namespace UE::PixelStreaming2