// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoCapturer.h"

#include "epic_rtc/core/video/video_buffer.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	class FEpicRtcVideoCapturer : public FVideoCapturer
	{
	public:
		static UE_API TSharedPtr<FEpicRtcVideoCapturer> Create(bool bIsSRGB = false);
		virtual ~FEpicRtcVideoCapturer() = default;

		UE_API TRefCountPtr<EpicRtcVideoBufferInterface> GetFrameBuffer();

	private:
		UE_API FEpicRtcVideoCapturer(bool bIsSRGB);
	};
} // namespace UE::PixelStreaming2

#undef UE_API
