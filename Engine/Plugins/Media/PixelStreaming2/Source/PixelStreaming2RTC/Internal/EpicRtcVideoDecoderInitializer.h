// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcContainers.h"
#include "Video/VideoConfig.h"

#include "epic_rtc/core/video/video_decoder.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{

	class FEpicRtcVideoDecoderInitializer : public EpicRtcVideoDecoderInitializerInterface
	{
	public:
		FEpicRtcVideoDecoderInitializer() = default;
		virtual ~FEpicRtcVideoDecoderInitializer() = default;

		// Begin EpicRtcVideoDecoderInitializerInterface
		UE_API virtual void								 CreateDecoder(EpicRtcVideoCodecInfoInterface* CodecInfo, EpicRtcVideoDecoderInterface** OutDecoder) override;
		UE_API virtual EpicRtcStringView					 GetName() override;
		UE_API virtual EpicRtcVideoCodecInfoArrayInterface* GetSupportedCodecs() override;
		// End EpicRtcVideoDecoderInitializerInterface

	private:
		UE_API TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> CreateSupportedDecoderMap();

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

} // namespace UE::PixelStreaming2

#undef UE_API
