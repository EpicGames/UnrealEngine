// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcContainers.h"
#include "Video/VideoConfig.h"

#include "epic_rtc/core/video/video_encoder.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{

	class FEpicRtcVideoEncoderInitializer : public EpicRtcVideoEncoderInitializerInterface
	{
	public:
		FEpicRtcVideoEncoderInitializer() = default;
		virtual ~FEpicRtcVideoEncoderInitializer() = default;

		// Begin EpicRtcVideoEncoderInitializerInterface
		UE_API virtual void								 CreateEncoder(EpicRtcVideoCodecInfoInterface* CodecInfo, EpicRtcVideoEncoderInterface** OutEncoder) override;
		UE_API virtual EpicRtcStringView					 GetName() override;
		UE_API virtual EpicRtcVideoCodecInfoArrayInterface* GetSupportedCodecs() override;
		// End EpicRtcVideoEncoderInitializerInterface

	private:
		UE_API TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> CreateSupportedEncoderMap();

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

} // namespace UE::PixelStreaming2

#undef UE_API
