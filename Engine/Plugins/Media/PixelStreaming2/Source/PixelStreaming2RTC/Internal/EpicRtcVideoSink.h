// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "EpicRtcTrack.h"
#include "VideoCapturer.h"
#include "VideoSink.h"
#include "RendererInterface.h"

#include "epic_rtc/core/video/video_frame.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	/**
	 * Video sink class that receives a frame from EpicRtc and passes the frame to all added consumers
	 */
	class FEpicRtcVideoSink : public FVideoSink, public TEpicRtcTrack<EpicRtcVideoTrackInterface>, public TSharedFromThis<FEpicRtcVideoSink>
	{
	public:
		static UE_API TSharedPtr<FEpicRtcVideoSink> Create(TRefCountPtr<EpicRtcVideoTrackInterface> InTrack);
		// Note: destructor will call destroy on any attached video consumers
		virtual ~FEpicRtcVideoSink() = default;

		UE_API void OnEpicRtcFrame(const EpicRtcVideoFrame& Frame);
	private:
		UE_API FEpicRtcVideoSink(TRefCountPtr<EpicRtcVideoTrackInterface> InTrack);

		UE_API void OnFrameCaptured();

		FCriticalSection				  RenderSyncContext;
		FPooledRenderTargetDesc			  RenderTargetDescriptor;
		TRefCountPtr<IPooledRenderTarget> RenderTarget;
		TArray<uint8_t>					  Buffer;
		FTextureRHIRef					  SourceTexture;

		TSharedPtr<FVideoCapturer> VideoCapturer;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
