// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcTrack.h"
#include "EpicRtcVideoCapturer.h"
#include "VideoSource.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	class FVideoSourceGroup;
	/**
	 * A source of video frames for an EpicRtc peer. Has a video input which will provide
	 * frame data to this source. The source will then pass that data to an adapter
	 * which will have one or many adapt processes that are provided by the input object
	 * and are responsible for converting the frame data to the format required for
	 * the selected video encoder.
	 * This video source should be contained within a FVideoSourceGroup which is responsible
	 * for telling each source to push a frame to EpicRtc at the expected rate. This source
	 * will make sure that the adapter has valid output and if so will create a frame
	 * for EpicRtc. Otherwise it will continue to wait until the next frame.
	 */
	class FEpicRtcVideoSource : public FVideoSource, public TEpicRtcTrack<EpicRtcVideoTrackInterface>
	{
	public:
		static UE_API TSharedPtr<FEpicRtcVideoSource> Create(TRefCountPtr<EpicRtcVideoTrackInterface> InVideoTrack, TSharedPtr<FEpicRtcVideoCapturer> InVideoCapturer, TSharedPtr<FVideoSourceGroup> InVideoSourceGroup);
		virtual ~FEpicRtcVideoSource() = default;

		UE_API virtual void PushFrame() override;
		UE_API virtual void ForceKeyFrame() override;

		UE_API TRefCountPtr<EpicRtcVideoTrackInterface> GetVideoTrack();

	private:
		UE_API FEpicRtcVideoSource(TRefCountPtr<EpicRtcVideoTrackInterface> InVideoTrack, TSharedPtr<FEpicRtcVideoCapturer> InVideoCapturer);

	private:
		TSharedPtr<FEpicRtcVideoCapturer> VideoCapturer;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
