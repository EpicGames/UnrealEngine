// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/video/video_track_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#include "EpicRtcVideoTrackObserver.generated.h"

#define UE_API PIXELSTREAMING2RTC_API

UINTERFACE(MinimalAPI)
class UPixelStreaming2VideoTrackObserver : public UInterface
{
	GENERATED_BODY()
};

class IPixelStreaming2VideoTrackObserver
{
	GENERATED_BODY()

public:
	virtual void		OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) = 0;
	virtual void		OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) = 0;
	virtual void		OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) = 0;
	virtual void		OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) = 0;
	virtual EpicRtcBool Enabled() const = 0;
};

namespace UE::PixelStreaming2
{
	class FEpicRtcVideoTrackObserver : public EpicRtcVideoTrackObserverInterface
	{
	public:
		UE_API FEpicRtcVideoTrackObserver(TObserverVariant<IPixelStreaming2VideoTrackObserver> UserObserver);
		virtual ~FEpicRtcVideoTrackObserver() = default;

	private:
		// Begin EpicRtcVideoTrackObserverInterface
		UE_API virtual void		OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) override;
		UE_API virtual void		OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) override;
		UE_API virtual void		OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) override;
		UE_API virtual void		OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) override;
		UE_API virtual EpicRtcBool Enabled() const override;
		// End EpicRtcVideoTrackObserverInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2VideoTrackObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
