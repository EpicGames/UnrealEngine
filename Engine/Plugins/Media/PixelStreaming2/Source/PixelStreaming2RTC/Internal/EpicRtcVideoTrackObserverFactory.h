// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcObserver.h"

#include "epic_rtc/core/video/video_track_observer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

class IPixelStreaming2VideoTrackObserver;

namespace UE::PixelStreaming2
{	
	class FEpicRtcVideoTrackObserverFactory : public EpicRtcVideoTrackObserverFactoryInterface
	{
	public:
		UE_API FEpicRtcVideoTrackObserverFactory(TObserverVariant<IPixelStreaming2VideoTrackObserver> UserObserver);
		virtual ~FEpicRtcVideoTrackObserverFactory() = default;

	public:
		// Begin EpicRtcVideoTrackObserverFactoryInterface
		UE_API virtual EpicRtcErrorCode CreateVideoTrackObserver(const EpicRtcStringView ParticipantId, const EpicRtcStringView VideoTrackId, EpicRtcVideoTrackObserverInterface** OutVideoTrackObserver) override;
		// End EpicRtcVideoTrackObserverFactoryInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TObserverVariant<IPixelStreaming2VideoTrackObserver> UserObserver;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
