// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Tickable.h"

#include "epic_rtc/plugins/signalling/websocket.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

class IWebSocket;

namespace UE::PixelStreaming2
{
	class FEpicRtcWebsocket : public EpicRtcWebsocketInterface, public FTickableGameObject
	{
	public:
		UE_API FEpicRtcWebsocket(bool bKeepAlive = true, TSharedPtr<IWebSocket> WebSocket = nullptr);
		virtual ~FEpicRtcWebsocket() = default;

		// Begin EpicRtcWebsocketInterface
		UE_API virtual EpicRtcBool Connect(EpicRtcStringView Url, EpicRtcWebsocketObserverInterface* Observer) override;
		UE_API virtual void		Disconnect(const EpicRtcStringView Reason) override;
		UE_API virtual void		Send(EpicRtcStringView Message) override;
		// End EpicRtcWebsocketInterface

		// Begin FTickableGameObject
		UE_API virtual void		Tick(float DeltaTime) override;
		virtual bool		IsTickableInEditor() const override { return true; }
		virtual bool		IsTickableWhenPaused() const override { return true; }
		FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(EpicRtcWebSocket, STATGROUP_Tickables); }
		// End FTickableGameObject

	private:
		UE_API void OnConnected();
		UE_API void OnConnectionError(const FString& Error);
		UE_API void OnClosed(int32 StatusCode, const FString& Reason, bool bWasClean);
		UE_API void OnMessage(const FString& Msg);
		UE_API void OnBinaryMessage(const uint8* Data, int32 Length, bool bIsLastFragment);
		UE_API void KeepAlive();

	private:
		FDelegateHandle OnConnectedHandle;
		FDelegateHandle OnConnectionErrorHandle;
		FDelegateHandle OnClosedHandle;
		FDelegateHandle OnMessageHandle;
		FDelegateHandle OnBinaryMessageHandle;

	private:
		FString											Url;
		FString											LastError;
		TSharedPtr<IWebSocket>							WebSocket;
		TRefCountPtr<EpicRtcWebsocketObserverInterface> Observer;
		uint64											LastKeepAliveCycles = 0;
		bool											bSendKeepAlive = false;
		bool											bCloseRequested = false;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};

} // namespace UE::PixelStreaming2

#undef UE_API
