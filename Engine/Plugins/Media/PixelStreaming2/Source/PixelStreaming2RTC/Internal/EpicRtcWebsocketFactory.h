// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcWebsocket.h"

#include "epic_rtc/plugins/signalling/websocket.h"
#include "epic_rtc/plugins/signalling/websocket_factory.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	class FEpicRtcWebsocketFactory : public EpicRtcWebsocketFactoryInterface
	{
	public:
		FEpicRtcWebsocketFactory(bool bInSendKeepAlive = true)
			: bSendKeepAlive(bInSendKeepAlive)
		{
		}

		virtual ~FEpicRtcWebsocketFactory() = default;

		UE_API virtual EpicRtcErrorCode CreateWebsocket(EpicRtcWebsocketInterface** OutWebsocket) override;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		bool bSendKeepAlive;
	};

} // namespace UE::PixelStreaming2

#undef UE_API
