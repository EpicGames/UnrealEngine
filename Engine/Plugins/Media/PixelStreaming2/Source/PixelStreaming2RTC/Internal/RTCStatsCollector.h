// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include "epic_rtc/core/stats.h"

#define UE_API PIXELSTREAMING2RTC_API

namespace UE::PixelStreaming2
{
	class FStats;

	namespace RTCStatCategories
	{
		const FString LocalVideoTrack = TEXT("video-track-outbound");
		const FString LocalAudioTrack = TEXT("audio-track-outbound");
		const FString VideoSource = TEXT("video-source");
		const FString AudioSource = TEXT("audio-source");
		const FString VideoCodec = TEXT("video-codec");
		const FString AudioCodec = TEXT("audio-codec");
		const FString DataChannel = TEXT("data-channel");
		const FString RemoteVideoTrack = TEXT("video-track-inbound");
		const FString RemoteAudioTrack = TEXT("audio-track-inbound");
		const FString CandidatePair = TEXT("candidate-pair");
	} // namespace RTCStatCategories

	enum PIXELSTREAMING2RTC_API EDisplayFlags : uint8
	{
		HIDDEN = 0,
		TEXT = 1 << 0,
		GRAPH = 1 << 1,
	};

	struct FStatConfig
	{
		FName			 Name;
		TOptional<FName> Alias;
		EDisplayFlags	 DisplayFlags = EDisplayFlags::TEXT;
	};

	using FStatVariant = TVariant<TYPE_OF_NULLPTR, double, FString, bool>;

	class FStat
	{
	public:
		FStat() = default;

		UE_API FStat(FStatConfig Config, double InitialValue, int NDecimalPlacesToPrint = 0, bool bSmooth = false);
		UE_API FStat(FStatConfig Config, FString InitialValue);
		UE_API FStat(FStatConfig Config, bool bInitialValue);
		UE_API FStat(const FStat& Other);

		UE_API bool IsNumeric() const;

		UE_API bool IsTextual() const;

		UE_API bool IsBoolean() const;

		UE_API FString ToString();

		UE_API bool SetValue(FStatVariant ValueVariant);

		template <typename T>
		T GetValue()
		{
			checkf(false, TEXT("Trying to get value with incompatible container. Please use FString, double or bool!"));
		}

		template <>
		FString GetValue();

		template <>
		double GetValue();

		template <>
		bool GetValue();

		template <typename T>
		T GetPrevValue()
		{
			checkf(false, TEXT("Trying to get value with incompatible container. Please use FString, double or bool!"));
		}

		template <>
		FString GetPrevValue();

		template <>
		double GetPrevValue();

		template <>
		bool GetPrevValue();

		UE_API bool operator==(const FStat& Other) const;

		UE_API bool IsHidden();

		UE_API bool ShouldGraph();

		UE_API bool ShouldDisplayText();

		UE_API FName GetName() const;

		UE_API FName GetDisplayName() const;

	private:
		UE_API double CalcMA(double InPrevAvg, int InNumSamples, double InValue);
		UE_API double CalcEMA(double InPrevAvg, int InNumSamples, double InValue);

	protected:
		FName			 Name;
		EDisplayFlags	 DisplayFlags = EDisplayFlags::TEXT;
		TOptional<FName> Alias;

		// variable used for calculating and displaying numeric stat data
		int	 NDecimalPlacesToPrint = 0;
		bool bSmooth = false;
		int	 NumSamples = 0;

		FStatVariant StatVariant;
		FStatVariant PrevStatVariant;
	};

	FORCEINLINE uint32 GetTypeHash(const FStat& Obj)
	{
		// From UnrealString.h
		return GetTypeHash(Obj.GetName());
	}

	class FRTCStatsCollector
	{
	public:
		static UE_API TSharedPtr<FRTCStatsCollector> Create(const FString& PlayerId);

	private:
		UE_API FRTCStatsCollector();
		UE_API explicit FRTCStatsCollector(const FString& PlayerId);

		UE_API void OnWebRTCDisableStatsChanged(IConsoleVariable* Var);

	public:
		UE_API void Process(const EpicRtcConnectionStats& InStats);

	private:
		class FStatsSink
		{
		public:
			FStatsSink(FName InCategory);
			virtual ~FStatsSink() = default;

			virtual FStat* Get(const FName& StatName)
			{
				return Stats.Find(StatName);
			}

			virtual void PostProcess(FStats* PSStats, const FString& PeerId, double SecondsDelta);

		protected:
			// Stats that are stored as is.
			TMap<FName, FStat> Stats;
			// Stats we calculate based on the stats map above. This calculation is done in FStatsSink::PostProcess by the `Calculators` below.
			TArray<TFunction<TOptional<FStat>(FStatsSink&, double)>> Calculators;

		protected:
			FName Category;
		};

		class FRTPLocalVideoTrackStatsSink : public FStatsSink
		{
		public:
			FRTPLocalVideoTrackStatsSink(FName Category);
			virtual ~FRTPLocalVideoTrackStatsSink() = default;
			void Process(const EpicRtcLocalTrackRtpStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FRTPLocalAudioTrackStatsSink : public FStatsSink
		{
		public:
			FRTPLocalAudioTrackStatsSink(FName Category);
			virtual ~FRTPLocalAudioTrackStatsSink() = default;
			void Process(const EpicRtcLocalTrackRtpStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FRTPRemoteTrackStatsSink : public FStatsSink
		{
		public:
			FRTPRemoteTrackStatsSink(FName Category);
			virtual ~FRTPRemoteTrackStatsSink() = default;
			void Process(const EpicRtcRemoteTrackRtpStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FVideoSourceStatsSink : public FStatsSink
		{
		public:
			FVideoSourceStatsSink(FName Category);
			virtual ~FVideoSourceStatsSink() = default;
			void Process(const EpicRtcVideoSourceStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FVideoCodecStatsSink : public FStatsSink
		{
		public:
			FVideoCodecStatsSink(FName Category);
			virtual ~FVideoCodecStatsSink() = default;
			void Process(const EpicRtcCodecStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FAudioSourceStatsSink : public FStatsSink
		{
		public:
			FAudioSourceStatsSink(FName Category);
			virtual ~FAudioSourceStatsSink() = default;
			void Process(const EpicRtcAudioSourceStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FAudioCodecStatsSink : public FStatsSink
		{
		public:
			FAudioCodecStatsSink(FName Category);
			virtual ~FAudioCodecStatsSink() = default;
			void Process(const EpicRtcCodecStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		/**
		 * ---------- FDataChannelSink -------------------
		 */
		class FDataTrackStatsSink : public FStatsSink
		{
		public:
			FDataTrackStatsSink(FName Category);
			virtual ~FDataTrackStatsSink() = default;

			void Process(const EpicRtcDataTrackStats& InStats, const FString& PeerId, double SecondsDelta);
		};

		class FCandidatePairStatsSink : public FStatsSink
		{
		public:
			FCandidatePairStatsSink(FName Category);
			virtual ~FCandidatePairStatsSink() = default;
			void Process(const EpicRtcIceCandidatePairStats& InStats, const FString& PeerId, double SecondsDelta);
		};

	private:
		FString AssociatedPlayerId;
		uint64	LastCalculationCycles;
		bool	bIsEnabled;

		//    index,         ssrc                          sink
		TMap<uint64, TMap<uint32, TUniquePtr<FRTPLocalVideoTrackStatsSink>>> LocalVideoTrackSinks;
		TMap<uint64, TMap<uint32, TUniquePtr<FRTPLocalAudioTrackStatsSink>>> LocalAudioTrackSinks;

		TMap<uint64, TMap<uint32, TUniquePtr<FRTPRemoteTrackStatsSink>>> RemoteVideoTrackSinks;
		TMap<uint64, TMap<uint32, TUniquePtr<FRTPRemoteTrackStatsSink>>> RemoteAudioTrackSinks;

		TMap<uint64, TUniquePtr<FVideoSourceStatsSink>> VideoSourceSinks;
		TMap<uint64, TUniquePtr<FVideoCodecStatsSink>>	VideoCodecSinks;

		TMap<uint64, TUniquePtr<FAudioSourceStatsSink>> AudioSourceSinks;
		TMap<uint64, TUniquePtr<FAudioCodecStatsSink>>	AudioCodecSinks;

		TMap<uint64, TUniquePtr<FDataTrackStatsSink>> DataTrackSinks;

		TUniquePtr<FCandidatePairStatsSink> CandidatePairStatsSink;
	};
} // namespace UE::PixelStreaming2

#undef UE_API
