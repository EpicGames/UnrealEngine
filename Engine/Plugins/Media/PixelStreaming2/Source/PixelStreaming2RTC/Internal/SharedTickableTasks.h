// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcConferenceUtils.h"
#include "TickableTask.h"

namespace UE::PixelStreaming2
{
	class FSharedTickableTasks
	{
	public:
		FSharedTickableTasks(TSharedPtr<FEpicRtcTickConferenceTask> InTickConferenceTask)
			: TickConferenceTask(InTickConferenceTask)
		{
		}

	private:
		
		TSharedPtr<FEpicRtcTickConferenceTask> TickConferenceTask;
	};
} // namespace UE::PixelStreaming2
