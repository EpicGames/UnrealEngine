// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <string>

#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"

#include "epic_rtc/common/common.h"
#include "epic_rtc/containers/epic_rtc_string_view.h"
#include "epic_rtc/core/stats.h"
#include "epic_rtc/core/video/video_common.h"

#define BEGIN_ENUM_TO_STRING(EnumType, Value) \
	using Type = EnumType;                   \
    switch (Value) {                         

#define ENUM_CASE_TO_STRING(Name) case Type::Name: return TEXT(#Name);

#define END_ENUM_TO_STRING() \
    default: return TEXT("Unknown"); \
    }

namespace UE::PixelStreaming2
{
	inline FString ToString(EpicRtcPixelFormat Format)
	{
		BEGIN_ENUM_TO_STRING(EpicRtcPixelFormat, Format)
		ENUM_CASE_TO_STRING(Native)
		ENUM_CASE_TO_STRING(I420)
		ENUM_CASE_TO_STRING(I420A)
		ENUM_CASE_TO_STRING(I422)
		ENUM_CASE_TO_STRING(I444)
		ENUM_CASE_TO_STRING(I010)
		ENUM_CASE_TO_STRING(I210)
		ENUM_CASE_TO_STRING(NV12)
		END_ENUM_TO_STRING()
	}

	inline FString ToString(EpicRtcErrorCode Error)
	{
		BEGIN_ENUM_TO_STRING(EpicRtcErrorCode, Error)
		ENUM_CASE_TO_STRING(Ok)
		ENUM_CASE_TO_STRING(GeneralError)
		ENUM_CASE_TO_STRING(BadState)
		ENUM_CASE_TO_STRING(Timeout)
		ENUM_CASE_TO_STRING(Unsupported)
		ENUM_CASE_TO_STRING(PlatformError)
		ENUM_CASE_TO_STRING(FoundExistingPlatform)
		ENUM_CASE_TO_STRING(ConferenceAlreadyExists)
		ENUM_CASE_TO_STRING(ConferenceDoesNotExists)
		ENUM_CASE_TO_STRING(ImATeapot)
		ENUM_CASE_TO_STRING(ConferenceError)
		ENUM_CASE_TO_STRING(SessionAlreadyExists)
		ENUM_CASE_TO_STRING(SessionDoesNotExist)
		ENUM_CASE_TO_STRING(SessionError)
		ENUM_CASE_TO_STRING(SessionCannotConnect)
		ENUM_CASE_TO_STRING(SessionDisconnected)
		ENUM_CASE_TO_STRING(SessionCannotCreateRoom)
		END_ENUM_TO_STRING()
	}

	inline FString ToString(EpicRtcQualityLimitationReason Reason)
	{
		BEGIN_ENUM_TO_STRING(EpicRtcQualityLimitationReason, Reason)
		ENUM_CASE_TO_STRING(None)
		ENUM_CASE_TO_STRING(CPU)
		ENUM_CASE_TO_STRING(Bandwidth)
		ENUM_CASE_TO_STRING(Other)
		END_ENUM_TO_STRING()
	}

	inline FString ToString(EpicRtcVideoCodec Codec)
	{
		BEGIN_ENUM_TO_STRING(EpicRtcVideoCodec, Codec)
		ENUM_CASE_TO_STRING(AV1)
		ENUM_CASE_TO_STRING(H264)
		ENUM_CASE_TO_STRING(VP8)
		ENUM_CASE_TO_STRING(VP9)
		END_ENUM_TO_STRING()
	}

	inline FString ToString(EpicRtcRoomState State)
	{
		BEGIN_ENUM_TO_STRING(EpicRtcRoomState, State)
		ENUM_CASE_TO_STRING(New)
		ENUM_CASE_TO_STRING(Pending)
		ENUM_CASE_TO_STRING(Joined)
		ENUM_CASE_TO_STRING(Left)
		ENUM_CASE_TO_STRING(Failed)
		ENUM_CASE_TO_STRING(Exiting)
		END_ENUM_TO_STRING()
	}

	inline FString ToString(EpicRtcTrackState State)
	{
		BEGIN_ENUM_TO_STRING(EpicRtcTrackState, State)
		ENUM_CASE_TO_STRING(New)
		ENUM_CASE_TO_STRING(Active)
		ENUM_CASE_TO_STRING(Stopped)
		END_ENUM_TO_STRING()
	}

	inline FString ToString(const TSharedPtr<FJsonObject>& JsonObj, bool bPretty = true)
	{
		FString Res;
		if (bPretty)
		{
			auto JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&Res);
			FJsonSerializer::Serialize(JsonObj.ToSharedRef(), JsonWriter);
		}
		else
		{
			auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Res);
			FJsonSerializer::Serialize(JsonObj.ToSharedRef(), JsonWriter);
		}
		return Res;
	}

	inline FString ToString(const EpicRtcStringView& Str)
	{
		FUtf8String Utf8String = FUtf8String::ConstructFromPtrSize(Str._ptr, Str._length);
		return FString(Utf8String);
	}

	inline EpicRtcStringView ToEpicRtcStringView(const FUtf8String& Str)
	{
		return EpicRtcStringView{ ._ptr = (const char*)*Str, ._length = static_cast<uint64>(Str.Len()) };
	}

	/**
	 * Reads a string represented by 2 bytes length (in bytes) followed by UTF16 characters.
	 * String and length are encoded in little endian format.
	 */
	inline FString ReadString(const uint8*& Data, uint32_t& Size)
	{
		uint16_t BytesLength = Data[1] << 8 | Data[0];
		check(Size >= (uint32_t)(BytesLength + 2));
		Data += 2;
		Size -= 2;

		FString Message(BytesLength / sizeof(TCHAR), reinterpret_cast<const TCHAR*>(Data));
		Data += BytesLength;
		Size -= BytesLength;

		return Message;
	}
} // namespace UE::PixelStreaming2

#undef BEGIN_ENUM_TO_STRING
#undef ENUM_CASE_TO_STRING
#undef END_ENUM_TO_STRING
