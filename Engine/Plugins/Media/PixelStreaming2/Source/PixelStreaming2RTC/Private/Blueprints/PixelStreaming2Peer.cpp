// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Peer.h"

#include "Async/Async.h"
#include "EpicRtcSessionObserver.h"
#include "Logging.h"
#include "PixelStreaming2RTCModule.h"
#include "SampleBuffer.h"
#include "SoundGenerator.h"
#include "UtilsString.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2Peer)

uint32_t UPixelStreaming2Peer::PlayerId = 0;

UPixelStreaming2Peer::UPixelStreaming2Peer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SoundGenerator(MakeShared<UE::PixelStreaming2::FSoundGenerator, ESPMode::ThreadSafe>())
{
	PreferredBufferLength = 512u;
	NumChannels = 2;
	PrimaryComponentTick.bCanEverTick = true;
	SetComponentTickEnabled(true);
	bAutoActivate = true;

	PlayerName = FUtf8String::Printf("PixelStreaming2Player%d", PlayerId++);
}

void UPixelStreaming2Peer::BeginPlay()
{
	Super::BeginPlay();

	SessionObserver = MakeRefCount<UE::PixelStreaming2::FEpicRtcSessionObserver>(UE::PixelStreaming2::TObserver<IPixelStreaming2SessionObserver>(this));
	RoomObserver = MakeRefCount<UE::PixelStreaming2::FEpicRtcRoomObserver>(UE::PixelStreaming2::TObserver<IPixelStreaming2RoomObserver>(this));

	AudioTrackObserverFactory = MakeRefCount<UE::PixelStreaming2::FEpicRtcAudioTrackObserverFactory>(UE::PixelStreaming2::TObserver<IPixelStreaming2AudioTrackObserver>(this));
	VideoTrackObserverFactory = MakeRefCount<UE::PixelStreaming2::FEpicRtcVideoTrackObserverFactory>(UE::PixelStreaming2::TObserver<IPixelStreaming2VideoTrackObserver>(this));
	DataTrackObserverFactory = MakeRefCount<UE::PixelStreaming2::FEpicRtcDataTrackObserverFactory>(UE::PixelStreaming2::TObserver<IPixelStreaming2DataTrackObserver>(this));

	EpicRtcConference = UE::PixelStreaming2::FPixelStreaming2RTCModule::GetModule()->GetEpicRtcConference();

	UE::PixelStreaming2::FPixelStreaming2RTCModule::GetModule()->GetStatsCollector()->OnStatsReady.AddUObject(this, &UPixelStreaming2Peer::OnStatsReady);
}

void UPixelStreaming2Peer::BeginDestroy()
{
	Super::BeginDestroy();

	SoundGenerator = nullptr;
}

void UPixelStreaming2Peer::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Disconnect(FString(TEXT("UPixelStreaming2Peer::EndPlay called with reason: ")) + StaticEnum<EEndPlayReason::Type>()->GetNameStringByValue(EndPlayReason));

	Super::EndPlay(EndPlayReason);
}

bool UPixelStreaming2Peer::Connect(const FString& Url)
{
	FUtf8String Utf8Url = *Url;
	FUtf8String ConnectionUrl = Utf8Url + (Utf8Url.Contains(TEXT("?")) ? TEXT("&") : TEXT("?")) + TEXT("isStreamer=false");

	EpicRtcSessionConfig SessionConfig = {
		._id = { ._ptr = reinterpret_cast<const char*>(*PlayerName), ._length = static_cast<uint64_t>(PlayerName.Len()) },
		._url = { ._ptr = reinterpret_cast<const char*>(*ConnectionUrl), ._length = static_cast<uint64_t>(ConnectionUrl.Len()) },
		._observer = SessionObserver.GetReference()
	};

	TickableTasks = UE::PixelStreaming2::FPixelStreaming2RTCModule::GetModule()->GetSharedTickableTasks();

	if (const EpicRtcErrorCode Result = EpicRtcConference->CreateSession(SessionConfig, EpicRtcSession.GetInitReference()); Result != EpicRtcErrorCode::Ok)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create EpicRtc session, CreateSession returned: {0}", UE::PixelStreaming2::ToString(Result));
		return false;
	}

	if (const EpicRtcErrorCode Result = EpicRtcSession->Connect(); Result != EpicRtcErrorCode::Ok)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to connect EpicRtcSession. Connect returned: {0}", UE::PixelStreaming2::ToString(Result));
		return false;
	}
	return true;
}

bool UPixelStreaming2Peer::Disconnect()
{
	return Disconnect(TEXT("Disconnect called from Blueprint"));
}

bool UPixelStreaming2Peer::Disconnect(const FString& OptionalReason)
{
	if (!EpicRtcSession)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to disconnect, EpicRtcSession does not exist");
		return false;
	}

	if (AudioSink)
	{
		if (TSharedPtr<UE::PixelStreaming2::FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
		{
			SafeAudioSink->RemoveAudioConsumer(this);
		}

		AudioSink = nullptr;
		RemoteAudioTrack = 0;
	}

	// NOTE: It is imperative we null out the video sink before we remove room and session. If the sink is still alive
	// during session destruction, webrtc will flush incoming frames (leading to OnVideoTrackFrame) and the engine will
	// lock inside AVCodecs on a RHI fence. With the sink nulled, OnVideoTrackFrame will early exit
	if (VideoSink)
	{
		if (TSharedPtr<UE::PixelStreaming2::FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
		{
			SafeVideoSink->RemoveVideoConsumer(this);
		}
		VideoSink = nullptr;
		RemoteVideoTrack = 0;
	}

	if (EpicRtcRoom)
	{
		EpicRtcRoom->Leave();
		EpicRtcSession->RemoveRoom({ ._ptr = reinterpret_cast<const char*>(*SubscribedStream), ._length = static_cast<uint64_t>(SubscribedStream.Len()) });
	}

	FUtf8String Reason;
	if (OptionalReason.Len())
	{
		Reason = *OptionalReason;
	}
	else
	{
		Reason = "PixelStreaming2Peer Disconnected";
	}

	EpicRtcErrorCode Result = EpicRtcSession->Disconnect(UE::PixelStreaming2::ToEpicRtcStringView(Reason));
	if (Result != EpicRtcErrorCode::Ok)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to disconnect EpicRtcSession. Disconnect returned {0}", UE::PixelStreaming2::ToString(Result));
		return false;
	}

	EpicRtcConference->RemoveSession({ ._ptr = reinterpret_cast<const char*>(*PlayerName), ._length = static_cast<uint64_t>(PlayerName.Len()) });

	TickableTasks.Reset();

	return true;
}

bool UPixelStreaming2Peer::Subscribe(const FString& StreamerId)
{
	if (SessionState != EpicRtcSessionState::Connected)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create subscribe to streamer. EpicRtc session isn't connected");
		return false;
	}

	EpicRtcConnectionConfig ConnectionConfig = {
		// TODO (Migration): RTCP-7032 This info usually comes from the OnConfig signalling message
		._iceServers = EpicRtcIceServerSpan{ ._ptr = nullptr, ._size = 0 },
		._iceConnectionPolicy = EpicRtcIcePolicy::All,
		._disableTcpCandidates = false
	};

	SubscribedStream = *StreamerId;

	EpicRtcRoomConfig RoomConfig = {
		._id = EpicRtcStringView{ ._ptr = reinterpret_cast<const char*>(*SubscribedStream), ._length = static_cast<uint64_t>(SubscribedStream.Len()) },
		._connectionConfig = ConnectionConfig,
		._ticket = EpicRtcStringView{ ._ptr = nullptr, ._length = 0 },
		._joinParameters = EpicRtcSignallingJoinParameters{
			._options = nullptr,
			._reservedAudioStreamsMaxPoolSize = 0,
			._audioOnly = false
		},
		._observer = RoomObserver,
		._audioTrackObserverFactory = AudioTrackObserverFactory,
		._dataTrackObserverFactory = DataTrackObserverFactory,
		._videoTrackObserverFactory = VideoTrackObserverFactory
	};

	EpicRtcErrorCode Result = EpicRtcSession->CreateRoom(RoomConfig, EpicRtcRoom.GetInitReference());
	if (Result != EpicRtcErrorCode::Ok)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create EpicRtc room. CreateRoom returned: {0}", UE::PixelStreaming2::ToString(Result));
		return false;
	}

	EpicRtcRoom->Join();

	// Create a stats collector so we can receive stats from the subscribed streamer
	StatsCollector = UE::PixelStreaming2::FRTCStatsCollector::Create(StreamerId);

	return true;
}

ISoundGeneratorPtr UPixelStreaming2Peer::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	SoundGenerator->Initialize(InParams);
	Initialize(InParams.SampleRate);

	return SoundGenerator;
}

void UPixelStreaming2Peer::OnBeginGenerate()
{
	SoundGenerator->bGeneratingAudio = true;
}

void UPixelStreaming2Peer::OnEndGenerate()
{
	SoundGenerator->bGeneratingAudio = false;
}

void UPixelStreaming2Peer::ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t InNumChannels, size_t NumFrames)
{
	// Sound generator has not been initialized yet.
	if (!SoundGenerator || !SoundGenerator->IsInitialized())
	{
		return;
	}

	SoundGenerator->AddAudio(AudioData, SampleRate, InNumChannels, NumFrames);
}

void UPixelStreaming2Peer::OnAudioConsumerAdded()
{
	SoundGenerator->bShouldGenerateAudio = true;
}

void UPixelStreaming2Peer::OnAudioConsumerRemoved()
{
	if (SoundGenerator)
	{
		SoundGenerator->bShouldGenerateAudio = false;
		SoundGenerator->EmptyBuffers();
	}
}

void UPixelStreaming2Peer::ConsumeFrame(FTextureRHIRef Frame) 
{
	if(VideoConsumer)
	{
		VideoConsumer->ConsumeFrame(Frame);
	}
}
void UPixelStreaming2Peer::OnVideoConsumerAdded() 
{
	
}
void UPixelStreaming2Peer::OnVideoConsumerRemoved() 
{
	
}

void UPixelStreaming2Peer::OnSessionStateUpdate(const EpicRtcSessionState StateUpdate)
{
	switch (StateUpdate)
	{
		case EpicRtcSessionState::New:
		case EpicRtcSessionState::Pending:
		case EpicRtcSessionState::Connected:
		case EpicRtcSessionState::Disconnected:
		case EpicRtcSessionState::Failed:
		case EpicRtcSessionState::Exiting:
			SessionState = StateUpdate;
			break;
		default:
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "OnSessionStateUpdate received unknown EpicRtcSessionState: {0}", static_cast<int>(StateUpdate));
			break;
	}
}

void UPixelStreaming2Peer::OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList)
{
	TArray<FString> Streamers;

	for (uint64_t i = 0; i < RoomsList->Size(); i++)
	{
		EpicRtcStringInterface* RoomName = RoomsList->Get()[i];
		Streamers.Add(FString{ RoomName->Get(), (int32)RoomName->Length() });
	}

	OnStreamerList.Broadcast(Streamers);
}

void UPixelStreaming2Peer::OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate)
{
}

void UPixelStreaming2Peer::OnRoomStateUpdate(const EpicRtcRoomState State)
{
}

void UPixelStreaming2Peer::OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant)
{
	FString ParticipantId{ (int32)Participant->GetId()._length, Participant->GetId()._ptr };
	UE_LOGF(LogPixelStreaming2RTC, Log, "Player (%ls) joined", *ParticipantId);
}

void UPixelStreaming2Peer::OnRoomLeftUpdate(const EpicRtcStringView ParticipantId)
{
}

void UPixelStreaming2Peer::OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack)
{
	FString ParticipantId{ (int32)Participant->GetId()._length, Participant->GetId()._ptr };
	FString VideoTrackId{ (int32)AudioTrack->GetId()._length, AudioTrack->GetId()._ptr };
	UE_LOGF(LogPixelStreaming2RTC, VeryVerbose, "UPixelStreaming2Peer::OnAudioTrackUpdate(Participant [%ls], VideoTrack [%ls])", *ParticipantId, *VideoTrackId);

	if (AudioTrack->IsRemote())
	{
		// We received a remote track. We should now generate audio from it
		AudioSink = UE::PixelStreaming2::FEpicRtcAudioSink::Create(AudioTrack);
		RemoteAudioTrack = reinterpret_cast<uintptr_t>(AudioTrack); // Keep track of which remote track we're receiving audio from as we only support one track

		if (TSharedPtr<UE::PixelStreaming2::FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
		{
			SafeAudioSink->AddAudioConsumer(this);
		}
	}
}

void UPixelStreaming2Peer::OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack)
{
	FString ParticipantId{ (int32)Participant->GetId()._length, Participant->GetId()._ptr };
	FString VideoTrackId{ (int32)VideoTrack->GetId()._length, VideoTrack->GetId()._ptr };
	UE_LOGF(LogPixelStreaming2RTC, VeryVerbose, "UPixelStreaming2Peer::OnVideoTrackUpdate(Participant [%ls], VideoTrack [%ls])", *ParticipantId, *VideoTrackId);

	if (VideoTrack->IsRemote())
	{
		// We received a remote track. We should now create a sink to handle receiving the frames.
		// NOTE: We pass in nullptr as the track because if we store the track on the sink, EpicRtc will be unable to destroy it
		// and webrtc will try to flush remaining frames during session removal. 
		VideoSink = UE::PixelStreaming2::FEpicRtcVideoSink::Create(nullptr);
		RemoteVideoTrack = reinterpret_cast<uintptr_t>(VideoTrack); // Keep track of which remote track we're receiving video from as we only support display one track
		// As EpicRtc work can happen on different threads, we always need to call functions on a copy of the VideoSink to ensure it doesn't get deleted out from underneath us
		if (TSharedPtr<UE::PixelStreaming2::FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
		{
			SafeVideoSink->AddVideoConsumer(this);
		}
	}
}

void UPixelStreaming2Peer::OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack)
{
}

[[nodiscard]] EpicRtcSdpInterface* UPixelStreaming2Peer::OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
{
	return nullptr;
}

[[nodiscard]] EpicRtcSdpInterface* UPixelStreaming2Peer::OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
{
	return nullptr;
}

void UPixelStreaming2Peer::OnRoomErrorUpdate(const EpicRtcErrorCode Error)
{
}

void UPixelStreaming2Peer::OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted)
{
	if (!AudioSink || RemoteAudioTrack != reinterpret_cast<uintptr_t>(AudioTrack))
	{
		return;
	}

	if (TSharedPtr<UE::PixelStreaming2::FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
	{
		SafeAudioSink->SetMuted(static_cast<bool>(bIsMuted));
	}
}

void UPixelStreaming2Peer::OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame)
{
	if (!AudioSink || RemoteAudioTrack != reinterpret_cast<uintptr_t>(AudioTrack))
	{
		return;
	}

	if (TSharedPtr<UE::PixelStreaming2::FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
	{
		SafeAudioSink->OnAudioData(Frame._data, Frame._length, Frame._format._numChannels, Frame._format._sampleRate);
	}
}

void UPixelStreaming2Peer::OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State)
{
}

void UPixelStreaming2Peer::OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted)
{
	if (!VideoSink || RemoteVideoTrack != reinterpret_cast<uintptr_t>(VideoTrack))
	{
		return;
	}

	if (TSharedPtr<UE::PixelStreaming2::FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
	{
		SafeVideoSink->SetMuted(static_cast<bool>(bIsMuted));
	}
}

void UPixelStreaming2Peer::OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame)
{
	if (!VideoSink || RemoteVideoTrack != reinterpret_cast<uintptr_t>(VideoTrack))
	{
		return;
	}

	if (TSharedPtr<UE::PixelStreaming2::FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
	{
		SafeVideoSink->OnEpicRtcFrame(Frame);
	}
}

void UPixelStreaming2Peer::OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State)
{
}

void UPixelStreaming2Peer::OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame)
{
}

EpicRtcBool UPixelStreaming2Peer::Enabled() const
{
	return true;
}

void UPixelStreaming2Peer::OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State)
{
}

void UPixelStreaming2Peer::OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame)
{
}

void UPixelStreaming2Peer::OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error)
{
}

void UPixelStreaming2Peer::OnStatsReady(const FString& PeerId, const EpicRtcConnectionStats& ConnectionStats)
{
	if (!StatsCollector)
	{
		return;
	}

	FString StreamId = *SubscribedStream;
	if (PeerId != StreamId)
	{
		return;
	}

	StatsCollector->Process(ConnectionStats);
}
