// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprints/PixelStreaming2MediaTexture.h"
#include "Components/SynthComponent.h"
#include "Containers/Utf8String.h"
#include "EpicRtcAudioSink.h"
#include "EpicRtcAudioTrackObserver.h"
#include "EpicRtcAudioTrackObserverFactory.h"
#include "EpicRtcDataTrackObserver.h"
#include "EpicRtcDataTrackObserverFactory.h"
#include "EpicRtcVideoSink.h"
#include "EpicRtcVideoTrackObserver.h"
#include "EpicRtcVideoTrackObserverFactory.h"
#include "EpicRtcRoomObserver.h"
#include "EpicRtcSessionObserver.h"
#include "IPixelStreaming2AudioConsumer.h"
#include "RTCStatsCollector.h"
#include "SharedTickableTasks.h"

#include "epic_rtc/core/conference.h"
#include "epic_rtc/core/room.h"
#include "epic_rtc/core/session.h"

#include "PixelStreaming2Peer.generated.h"

DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(FPixelStreamingStreamerList, UPixelStreaming2Peer, OnStreamerList, const TArray<FString>&, StreamerList);

namespace UE::PixelStreaming2
{
	class FSoundGenerator;
} // namespace UE::PixelStreaming2

/**
 * A blueprint representation of a Pixel Streaming Peer Connection. Will accept video sinks to receive video data.
 * NOTE: This class is not a peer of a streamer. This class represents a peer in its own right (akin to the browser) and will subscribe to a stream
 */
// UCLASS(Blueprintable, ClassGroup = (PixelStreaming2), meta = (BlueprintSpawnableComponent))
UCLASS(Blueprintable, Category = "PixelStreaming2", META = (DisplayName = "PixelStreaming Peer Component", BlueprintSpawnableComponent))
class UPixelStreaming2Peer :
	public USynthComponent,
	public IPixelStreaming2AudioConsumer,
	public IPixelStreaming2VideoConsumer,
	public IPixelStreaming2SessionObserver,
	public IPixelStreaming2RoomObserver,
	public IPixelStreaming2AudioTrackObserver,
	public IPixelStreaming2DataTrackObserver,
	public IPixelStreaming2VideoTrackObserver
{
	GENERATED_UCLASS_BODY()
protected:
	// UObject overrides
	virtual void BeginPlay() override;
	virtual void BeginDestroy() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	// USynthComponent overrides
	virtual void			   OnBeginGenerate() override;
	virtual void			   OnEndGenerate() override;
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;

public:
	/**
	 * Attempt to connect to a specified signalling server.
	 * @param Url The url of the signalling server. Ignored if this component has a MediaSource. In that case the URL on the media source will be used instead.
	 * @return Returns false if Connect fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	bool Connect(const FString& Url);

	/**
	 * Disconnect from the signalling server. No action if no connection exists.
	 * @return Returns false if Disconnect fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	bool Disconnect();

	/**
	 * Subscribe this peer to the streams provided by the specified streamer.
	 * @param StreamerId The name of the streamer to subscribe to.
	 * @return Returns false if Subscribe fails.
	 */
	UFUNCTION(BlueprintCallable, Category = "PixelStreaming2")
	bool Subscribe(const FString& StreamerId);

	/**
	 * Fired when the connection the list of available streams from the server.
	 */
	UPROPERTY(BlueprintAssignable, Category = "Components|Activation")
	FPixelStreamingStreamerList OnStreamerList;

	/**
	 * A sink for the video data received once this connection has finished negotiating.
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", META = (DisplayName = "Pixel Streaming Video Consumer", AllowPrivateAccess = true))
	TObjectPtr<UPixelStreaming2MediaTexture> VideoConsumer = nullptr;

private:
	FUtf8String		SubscribedStream;
	FUtf8String		PlayerName;
	static uint32_t PlayerId;

	TSharedPtr<UE::PixelStreaming2::FSharedTickableTasks> TickableTasks;

	TSharedPtr<UE::PixelStreaming2::FEpicRtcAudioSink> AudioSink;
	uintptr_t										   RemoteAudioTrack;

	TSharedPtr<UE::PixelStreaming2::FSoundGenerator, ESPMode::ThreadSafe> SoundGenerator;

	TSharedPtr<UE::PixelStreaming2::FEpicRtcVideoSink> VideoSink;
	uintptr_t										   RemoteVideoTrack;

	TSharedPtr<UE::PixelStreaming2::FRTCStatsCollector> StatsCollector;

	bool Disconnect(const FString& OptionalReason);

public:
	// Begin IPixelStreaming2AudioConsumer Callbacks
	virtual void ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames) override;
	virtual void OnAudioConsumerAdded() override;
	virtual void OnAudioConsumerRemoved() override;
	// End IPixelStreaming2AudioConsumer Callbacks

	// Begin IPixelStreaming2VideoConsumer Callbacks
	virtual void ConsumeFrame(FTextureRHIRef Frame) override;
	virtual void OnVideoConsumerAdded() override;
	virtual void OnVideoConsumerRemoved() override;
	// End IPixelStreaming2VideoConsumer Callbacks

	// Begin IPixelStreaming2SessionObserver
	virtual void OnSessionStateUpdate(const EpicRtcSessionState StateUpdate) override;
	virtual void OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate) override;
	virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) override;
	// End IPixelStreaming2SessionObserver

	// Begin IPixelStreaming2RoomObserver
	virtual void							   OnRoomStateUpdate(const EpicRtcRoomState State) override;
	virtual void							   OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) override;
	virtual void							   OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) override;
	virtual void							   OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) override;
	virtual void							   OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) override;
	virtual void							   OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) override;
	[[nodiscard]] virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
	[[nodiscard]] virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
	virtual void							   OnRoomErrorUpdate(const EpicRtcErrorCode Error) override;
	// End IPixelStreaming2RoomObserver

	// Begin IPixelStreaming2AudioTrackObserver
	virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) override;
	virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) override;
	virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) override;
	// End IPixelStreaming2AudioTrackObserver

	// Begin IPixelStreaming2VideoTrackObserver
	virtual void		OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) override;
	virtual void		OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) override;
	virtual void		OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) override;
	virtual void		OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) override;
	virtual EpicRtcBool Enabled() const override;
	// End IPixelStreaming2VideoTrackObserver

	// Begin IPixelStreaming2DataTrackObserver
	virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) override;
	virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) override;
	virtual void OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error) override;
	// End IPixelStreaming2DataTrackObserver

	void OnStatsReady(const FString& PeerId, const EpicRtcConnectionStats& ConnectionStats);

private:
	EpicRtcSessionState SessionState = EpicRtcSessionState::Disconnected;

	// Begin EpicRtc Classes
	TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
	TRefCountPtr<EpicRtcSessionInterface>	 EpicRtcSession;
	TRefCountPtr<EpicRtcRoomInterface>		 EpicRtcRoom;
	// End EpicRtc Classes

	// Begin EpicRtc Observers
	TRefCountPtr<UE::PixelStreaming2::FEpicRtcSessionObserver>			 SessionObserver;
	TRefCountPtr<UE::PixelStreaming2::FEpicRtcRoomObserver>				 RoomObserver;
	TRefCountPtr<UE::PixelStreaming2::FEpicRtcAudioTrackObserverFactory> AudioTrackObserverFactory;
	TRefCountPtr<UE::PixelStreaming2::FEpicRtcVideoTrackObserverFactory> VideoTrackObserverFactory;
	TRefCountPtr<UE::PixelStreaming2::FEpicRtcDataTrackObserverFactory>	 DataTrackObserverFactory;
	// End EpicRtc Observers
};
