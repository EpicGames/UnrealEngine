// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorConversion.h"

#include "Math/UnrealMathUtility.h"

namespace UE::PixelStreaming2
{
	// TODO (Migration) RTCP-6471 Optimize this so it does not introduce latency over libyuv
	void ConvertI420ToArgb(
		const uint8* RESTRICT SrcY,
		int					  StrideY,
		const uint8* RESTRICT SrcU,
		int					  StrideU,
		const uint8* RESTRICT SrcV,
		int					  StrideV,
		uint8* RESTRICT		  DestArgb,
		int					  StrideDest,
		int					  Width,
		int					  Height)
	{
		for (int j = 0; j < Height; ++j)
		{
			for (int i = 0; i < Width; ++i)
			{
				float y = SrcY[j * StrideY + i] - 16.f;
				float u = SrcU[(j >> 1) * StrideU + (i >> 1)] - 128.f;
				float v = SrcV[(j >> 1) * StrideV + (i >> 1)] - 128.f;

				float r = FMath::Clamp(y * 1.164f + v * 1.596f, 0.f, 255.f);
				float g = FMath::Clamp(y * 1.164f - u * 0.392f - v * 0.813f, 0.f, 255.f);
				float b = FMath::Clamp(y * 1.164f + u * 2.017f, 0.f, 255.f);

				DestArgb[j * StrideDest + i * 4 + 0] = b;
				DestArgb[j * StrideDest + i * 4 + 1] = g;
				DestArgb[j * StrideDest + i * 4 + 2] = r;
				DestArgb[j * StrideDest + i * 4 + 3] = 255;
			}
		}
	}

	// TODO (Migration) RTCP-6471 Optimize this so it does not introduce latency over libyuv
	void ConvertArgbToI420(
		const uint8* RESTRICT SrcArgb,
		int					  StrideArgb,
		uint8* RESTRICT		  DestY,
		int					  StrideY,
		uint8* RESTRICT		  DestU,
		int					  StrideU,
		uint8* RESTRICT		  DestV,
		int					  StrideV,
		int					  Width,
		int					  Height)
	{
		for (int j = 0; j < Height; ++j)
		{
			for (int i = 0; i < Width; ++i)
			{
				float b = SrcArgb[j * StrideArgb + i * 4 + 0];
				float g = SrcArgb[j * StrideArgb + i * 4 + 1];
				float r = SrcArgb[j * StrideArgb + i * 4 + 2];

				float y = FMath::Clamp(0.257f * r + 0.504f * g + 0.098f * b + 16.f, 0.f, 255.f);
				float u = FMath::Clamp(-0.148f * r + -0.291f * g + 0.439f * b + 128.f, 0.f, 255.f);
				float v = FMath::Clamp(0.439f * r + -0.368f * g + -0.071f * b + 128.f, 0.f, 255.f);

				DestY[j * StrideY + i] = y;
				DestU[(j >> 1) * StrideU + (i >> 1)] = u;
				DestV[(j >> 1) * StrideV + (i >> 1)] = v;
			}
		}
	}

	void ConvertI420ToI010(
		const uint8_t* RESTRICT SrcY,
		int						SrcStrideY,
		const uint8_t* RESTRICT SrcU,
		int						SrcStrideU,
		const uint8_t* RESTRICT SrcV,
		int						SrcStrideV,
		uint16_t* RESTRICT		DstY,
		int						DstStrideY,
		uint16_t* RESTRICT		DstU,
		int						DstStrideU,
		uint16_t* RESTRICT		DstV,
		int						DstStrideV,
		int						Width,
		int						Height)
	{
		int HalfWidth = (Width + 1) >> 1;
		int HalfHeight = (Height + 1) >> 1;
		if ((!SrcY && DstY) || !SrcU || !SrcV || !DstU || !DstV || Width <= 0 || Height == 0)
		{
			return;
		}
		// Negative Height means invert the image.
		if (Height < 0)
		{
			Height = -Height;
			HalfHeight = (Height + 1) >> 1;
			SrcY = SrcY + (Height - 1) * SrcStrideY;
			SrcU = SrcU + (HalfHeight - 1) * SrcStrideU;
			SrcV = SrcV + (HalfHeight - 1) * SrcStrideV;
			SrcStrideY = -SrcStrideY;
			SrcStrideU = -SrcStrideU;
			SrcStrideV = -SrcStrideV;
		}

		// Convert Y plane.
		Convert8To16Plane(SrcY, SrcStrideY, DstY, DstStrideY, 1024, Width, Height);
		// Convert UV planes.
		Convert8To16Plane(SrcU, SrcStrideU, DstU, DstStrideU, 1024, HalfWidth, HalfHeight);
		Convert8To16Plane(SrcV, SrcStrideV, DstV, DstStrideV, 1024, HalfWidth, HalfHeight);

		return;
	}

	void Convert8To16Plane(
		const uint8_t* RESTRICT SrcY,
		int						SrcStrideY,
		uint16_t* RESTRICT		DstY,
		int						DstStrideY,
		int						Scale, // 16384 for 10 bits
		int						Width,
		int						Height)
	{
		int y;

		if (Width <= 0 || Height == 0)
		{
			return;
		}
		// Negative Height means invert the image.
		if (Height < 0)
		{
			Height = -Height;
			DstY = DstY + (Height - 1) * DstStrideY;
			DstStrideY = -DstStrideY;
		}
		// Coalesce rows.
		if (SrcStrideY == Width && DstStrideY == Width)
		{
			Width *= Height;
			Height = 1;
			SrcStrideY = DstStrideY = 0;
		}

		// Convert plane
		for (y = 0; y < Height; ++y)
		{
			Convert8To16Row(SrcY, DstY, Scale, Width);
			SrcY += SrcStrideY;
			DstY += DstStrideY;
		}
	}

	// Use Scale to convert lsb formats to msb, depending how many bits there are:
	// 1024 = 10 bits
	void Convert8To16Row(
		const uint8_t* RESTRICT SrcY,
		uint16_t* RESTRICT		DstY,
		int						Scale,
		int						Width)
	{
		int x;
		Scale *= 0x0101; // replicates the byte.
		for (x = 0; x < Width; ++x)
		{
			DstY[x] = (SrcY[x] * Scale) >> 16;
		}
	}

	void CopyI420(const uint8_t* RESTRICT SrcY,
		int								  SrcStrideY,
		const uint8_t* RESTRICT			  SrcU,
		int								  SrcStrideU,
		const uint8_t* RESTRICT			  SrcV,
		int								  SrcStrideV,
		uint8_t* RESTRICT				  DstY,
		int								  DstStrideY,
		uint8_t* RESTRICT				  DstU,
		int								  DstStrideU,
		uint8_t* RESTRICT				  DstV,
		int								  DstStrideV,
		int								  Width,
		int								  Height)
	{
		int HalfWidth = (Width + 1) >> 1;
		int HalfHeight = (Height + 1) >> 1;
		if ((!SrcY && DstY) || !SrcU || !SrcV || !DstU || !DstV || Width <= 0 || Height == 0)
		{
			return;
		}
		// Negative Height means invert the image.
		if (Height < 0)
		{
			Height = -Height;
			HalfHeight = (Height + 1) >> 1;
			SrcY = SrcY + (Height - 1) * SrcStrideY;
			SrcU = SrcU + (HalfHeight - 1) * SrcStrideU;
			SrcV = SrcV + (HalfHeight - 1) * SrcStrideV;
			SrcStrideY = -SrcStrideY;
			SrcStrideU = -SrcStrideU;
			SrcStrideV = -SrcStrideV;
		}

		if (DstY)
		{
			CopyPlane(SrcY, SrcStrideY, DstY, DstStrideY, Width, Height);
		}
		// Copy UV planes.
		CopyPlane(SrcU, SrcStrideU, DstU, DstStrideU, HalfWidth, HalfHeight);
		CopyPlane(SrcV, SrcStrideV, DstV, DstStrideV, HalfWidth, HalfHeight);
	}

	void CopyPlane(const uint8_t* RESTRICT SrcY,
		int								   SrcStrideY,
		uint8_t* RESTRICT				   DstY,
		int								   DstStrideY,
		int								   Width,
		int								   Height)
	{
		if (Width <= 0 || Height == 0)
		{
			return;
		}
		// Negative Height means invert the image.
		if (Height < 0)
		{
			Height = -Height;
			DstY = DstY + (Height - 1) * DstStrideY;
			DstStrideY = -DstStrideY;
		}
		// Coalesce rows.
		if (SrcStrideY == Width && DstStrideY == Width)
		{
			Width *= Height;
			Height = 1;
			SrcStrideY = DstStrideY = 0;
		}
		// Nothing to do.
		if (SrcY == DstY && SrcStrideY == DstStrideY)
		{
			return;
		}

		// Copy plane
		for (int y = 0; y < Height; ++y)
		{
			memcpy(DstY, SrcY, Width);
			SrcY += SrcStrideY;
			DstY += DstStrideY;
		}
	}
} // namespace UE::PixelStreaming2
