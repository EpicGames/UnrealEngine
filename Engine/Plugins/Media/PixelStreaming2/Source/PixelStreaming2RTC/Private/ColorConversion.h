// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::PixelStreaming2
{
	void ConvertI420ToArgb(
		const uint8* RESTRICT SrcY,
		int					  StrideY,
		const uint8* RESTRICT SrcU,
		int					  StrideU,
		const uint8* RESTRICT SrcV,
		int					  StrideV,
		uint8* RESTRICT		  DestArgb,
		int					  StrideDest,
		int					  Width,
		int					  Height);

	void ConvertArgbToI420(
		const uint8* RESTRICT SrcArgb,
		int					  StrideArgb,
		uint8* RESTRICT		  DestY,
		int					  StrideY,
		uint8* RESTRICT		  DestU,
		int					  StrideU,
		uint8* RESTRICT		  DestV,
		int					  StrideV,
		int					  Width,
		int					  Height);

	void ConvertI420ToI010(
		const uint8* RESTRICT SrcY,
		int						SrcStrideY,
		const uint8* RESTRICT SrcU,
		int						SrcStrideU,
		const uint8* RESTRICT SrcV,
		int						SrcStrideV,
		uint16* RESTRICT		DstY,
		int						DstStrideY,
		uint16* RESTRICT		DstU,
		int						DstStrideU,
		uint16* RESTRICT		DstV,
		int						DstStrideV,
		int						Width,
		int						Height);

	void Convert8To16Plane(
		const uint8* RESTRICT SrcY,
		int						SrcStrideY,
		uint16* RESTRICT		DstY,
		int						DstStrideY,
		int						Scale, // 16384 for 10 bits
		int						Width,
		int						Height);

	void Convert8To16Row(
		const uint8* RESTRICT SrcY,
		uint16* RESTRICT		DstY,
		int						Scale,
		int						Width);

	void CopyI420(const uint8* RESTRICT SrcY,
		int								  SrcStrideY,
		const uint8* RESTRICT			  SrcU,
		int								  SrcStrideU,
		const uint8* RESTRICT			  SrcV,
		int								  SrcStrideV,
		uint8* RESTRICT				  DstY,
		int								  DstStrideY,
		uint8* RESTRICT				  DstU,
		int								  DstStrideU,
		uint8* RESTRICT				  DstV,
		int								  DstStrideV,
		int								  Width,
		int								  Height);

	void CopyPlane(const uint8* RESTRICT SrcY,
		int								   SrcStrideY,
		uint8* RESTRICT				   DstY,
		int								   DstStrideY,
		int								   Width,
		int								   Height);

	constexpr size_t CalcBufferSizeArgb(int Width, int Height)
	{
		return Width * Height * 4;
	};
} // namespace UE::PixelStreaming2
