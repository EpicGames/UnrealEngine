// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasTypes.h"
#include "Containers/Array.h"

namespace UE::PixelStreaming2
{
	class FDebugGraph
	{
	public:
		FDebugGraph(FName InName, int InSamples, float InMinRange, float InMaxRange, float InRefValue = 0.0f);
		FDebugGraph(const FDebugGraph& Other);
		~FDebugGraph() = default;

		void AddValue(float InValue);
		void Draw(FCanvas* Canvas, FVector2D Position, FVector2D Size) const;

	private:
		void AddValueInternal(float InValue);

	private:
		FName Name;
		int	  MaxSamples = 0;
		float MinRange = 0.0f;
		float MaxRange = 0.0f;
		float RefValue = 0.0f;

		float		  Sum = 0;
		TArray<float> Values;
		TArray<float> AvgValues;
		int			  InsertIndex = 0;
		int			  BufferedValues = 0;
		bool		  bFirstValue = true;

		mutable FCriticalSection CriticalSection;
	};
} // namespace UE::PixelStreaming2
