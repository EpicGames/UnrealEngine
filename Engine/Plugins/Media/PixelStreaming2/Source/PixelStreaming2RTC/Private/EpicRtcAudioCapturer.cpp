// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcAudioCapturer.h"

#include "AudioDevice.h"
#include "Misc/CoreDelegates.h"
#include "PixelStreaming2PluginSettings.h"

namespace UE::PixelStreaming2
{
	void FEpicRtcAudioCapturer::PushAudio(const float* AudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate)
	{
		check(AudioData != nullptr);
		check(InNumSamples > 0);
		check(InNumChannels > 0);
		check(InSampleRate > 0);
		check(InSampleRate == SampleRate);
		Audio::TSampleBuffer<int16_t> Buffer(AudioData, InNumSamples, InNumChannels, SampleRate);
		RecordingBuffer.Append(Buffer.GetData(), Buffer.GetNumSamples());

		const int32	 SamplesPer10Ms = NumChannels * SampleRate * 0.01f;
		const size_t BytesPerFrame = NumChannels * sizeof(int16_t);

		// Feed in 10ms chunks
		while (RecordingBuffer.Num() >= SamplesPer10Ms)
		{
			OnAudioBuffer.Broadcast(RecordingBuffer.GetData(), SamplesPer10Ms, NumChannels, SampleRate);

			// Remove 10ms of samples from the recording buffer now it is submitted
			RecordingBuffer.RemoveAt(0, SamplesPer10Ms, EAllowShrinking::No);
		}
	}
} // namespace UE::PixelStreaming2