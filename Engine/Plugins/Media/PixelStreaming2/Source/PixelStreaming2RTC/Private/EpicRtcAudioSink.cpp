// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcAudioSink.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FEpicRtcAudioSink> FEpicRtcAudioSink::Create(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack)
	{
		return TSharedPtr<FEpicRtcAudioSink>(new FEpicRtcAudioSink(InTrack));
	}

	FEpicRtcAudioSink::FEpicRtcAudioSink(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack)
		: TEpicRtcTrack(InTrack)
	{
	}
} // namespace UE::PixelStreaming2