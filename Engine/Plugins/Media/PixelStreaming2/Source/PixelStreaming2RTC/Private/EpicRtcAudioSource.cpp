// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcAudioSource.h"

#include "EpicRtcAudioCapturer.h"

namespace UE::PixelStreaming2
{

	TSharedPtr<FEpicRtcAudioSource> FEpicRtcAudioSource::Create(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack, TSharedPtr<FEpicRtcAudioCapturer> InCapturer)
	{
		TSharedPtr<FEpicRtcAudioSource> AudioTrack = TSharedPtr<FEpicRtcAudioSource>(new FEpicRtcAudioSource(InTrack));

		InCapturer->OnAudioBuffer.AddSP(AudioTrack.ToSharedRef(), &FEpicRtcAudioSource::OnAudioBuffer);

		return AudioTrack;
	}

	FEpicRtcAudioSource::FEpicRtcAudioSource(TRefCountPtr<EpicRtcAudioTrackInterface> InTrack)
		: TEpicRtcTrack(InTrack)
	{
	}

	void FEpicRtcAudioSource::OnAudioBuffer(const int16_t* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate)
	{
		if (!Track || bIsMuted)
		{
			return;
		}

		const uint32_t NumFrames = NumSamples / NumChannels;
		// clang-format off
		EpicRtcAudioFrame Frame = {
			._data = const_cast<int16_t*>(AudioData),
			._length = NumFrames,
			._timestamp = 0,
			._format = {
				._sampleRate = static_cast<uint32_t>(SampleRate),
				._numChannels = static_cast<uint32_t>(NumChannels),
				._parameters = nullptr 
			}
		};
		// clang-format on

		// Because UE handles all audio processing, we can bypass the ADM.
		// This also has the added benefit of increasing audio quality
		Track->PushFrame(Frame);
	}

} // namespace UE::PixelStreaming2
