// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcAudioTrackObserver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicRtcAudioTrackObserver)

namespace UE::PixelStreaming2
{
	FEpicRtcAudioTrackObserver::FEpicRtcAudioTrackObserver(TObserverVariant<IPixelStreaming2AudioTrackObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	void FEpicRtcAudioTrackObserver::OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted)
	{
		if (UserObserver)
		{
			UserObserver->OnAudioTrackMuted(AudioTrack, bIsMuted);
		}
	}

	void FEpicRtcAudioTrackObserver::OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame)
	{
		if (UserObserver)
		{
			UserObserver->OnAudioTrackFrame(AudioTrack, Frame);
		}
	}

	void FEpicRtcAudioTrackObserver::OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State)
	{
		if (UserObserver)
		{
			UserObserver->OnAudioTrackState(AudioTrack, State);
		}
	}

} // namespace UE::PixelStreaming2
