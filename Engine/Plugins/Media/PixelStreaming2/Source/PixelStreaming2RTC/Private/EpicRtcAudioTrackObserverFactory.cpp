// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcAudioTrackObserverFactory.h"

#include "EpicRtcAudioTrackObserver.h"

namespace UE::PixelStreaming2
{

	FEpicRtcAudioTrackObserverFactory::FEpicRtcAudioTrackObserverFactory(TObserverVariant<IPixelStreaming2AudioTrackObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	EpicRtcErrorCode FEpicRtcAudioTrackObserverFactory::CreateAudioTrackObserver(const EpicRtcStringView ParticipantId, const EpicRtcStringView AudioTrackId, EpicRtcAudioTrackObserverInterface** OutAudioTrackObserver)
	{
		EpicRtcAudioTrackObserverInterface* AudioTrackObserver = new FEpicRtcAudioTrackObserver(UserObserver);
		// Because the ptr was created with new, we need to call AddRef ourself (ms spec compliant)
		AudioTrackObserver->AddRef();

		*OutAudioTrackObserver = AudioTrackObserver;
		return EpicRtcErrorCode::Ok;
	}

} // namespace UE::PixelStreaming2