// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcDataTrack.h"

#include "Logging.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FEpicRtcDataTrack> FEpicRtcDataTrack::Create(TRefCountPtr<EpicRtcDataTrackInterface> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol)
	{
		TSharedPtr<FEpicRtcDataTrack> DataTrack = TSharedPtr<FEpicRtcDataTrack>(new FEpicRtcDataTrack(InTrack, InDataProtocol));
		return DataTrack;
	}

	FEpicRtcDataTrack::FEpicRtcDataTrack(TRefCountPtr<EpicRtcDataTrackInterface> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol)
		: TEpicRtcTrack(InTrack)
		, WeakDataProtocol(InDataProtocol)
	{
	}

	FEpicRtcDataTrack::FEpicRtcDataTrack(TSharedPtr<FEpicRtcDataTrack> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol)
		: TEpicRtcTrack(InTrack->SendTrack ? InTrack->SendTrack : InTrack->Track)
		, WeakDataProtocol(InDataProtocol)
	{
	}

	bool FEpicRtcDataTrack::IsActive() const
	{
		if (!Track)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Cannot send message when datatrack is null.");
			return false;
		}

		if (Track->GetState() != EpicRtcTrackState::Active)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Cannot send message when datatrack is not active.");
			return false;
		}

		return true;
	}

	bool FEpicRtcDataTrack::GetMessageId(const FString& MessageType, uint8& OutMessageId) const
	{
		TSharedPtr<IPixelStreaming2DataProtocol> DataProtocol = WeakDataProtocol.Pin();

		if (!DataProtocol)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Cannot send message, data protocol was null.");
			return false;
		}

		TSharedPtr<IPixelStreaming2InputMessage> Message = DataProtocol->Find(MessageType);
		if (!Message)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Cannot send message called '%ls' as it is not in the data protocol. Try GetTo/FromStreamerProtocol()->Add()", *MessageType);
			return false;
		}

		OutMessageId = Message->GetID();

		return true;
	}

	bool FEpicRtcDataTrack::Send(TArray<uint8>& Buffer) const
	{
		EpicRtcDataFrame DataFrame{
			._data = Buffer.GetData(),
			._size = static_cast<uint32_t>(Buffer.Num()),
			._binary = true
		};

		TRefCountPtr<EpicRtcDataTrackInterface> OutgoingTrack = SendTrack ? SendTrack : Track;

		EpicRtcBool SendResult = OutgoingTrack->PushFrame(DataFrame);
		if (!SendResult)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "DataTrack PushFrame return false");
		}

		return static_cast<bool>(SendResult);
	}

	bool FEpicRtcDataTrack::SendArbitraryData(const FString& MessageType, const TArray64<uint8>& DataBytes) const
	{
		if (!IsActive())
		{
			return false;
		}

		uint8 Type;
		if (!GetMessageId(MessageType, Type))
		{
			return false;
		}

		// int32 results in a maximum 4GB file (4,294,967,296 bytes)
		const int32 DataSize = DataBytes.Num();

		// Maximum size of a single buffer should be 16KB as this is spec compliant message length for a single data channel transmission
		const int32 MaxBufferBytes = 16 * 1024;
		const int32 MessageHeader = sizeof(Type) + sizeof(DataSize);
		const int32 MaxDataBytesPerMsg = MaxBufferBytes - MessageHeader;

		int32 BytesTransmitted = 0;

		while (BytesTransmitted < DataSize)
		{
			int32 RemainingBytes = DataSize - BytesTransmitted;
			int32 BytesToTransmit = FGenericPlatformMath::Min(MaxDataBytesPerMsg, RemainingBytes);

			FBufferBuilder Builder(MessageHeader + BytesToTransmit);
			PrependData(Builder);

			// Write message type
			Builder.Insert(Forward<uint8>(Type));

			// Write size of payload
			Builder.Insert(Forward<const int32>(DataSize));

			// Write the data bytes payload
			Builder.Serialize(DataBytes.GetData() + BytesTransmitted, BytesToTransmit);

			// TODO (Migration): RTCP-6489 We may need EpicRtc API surface to query the buffered amount in the datachannel so we don't flood it.
			// uint64_t BufferBefore = SendChannel->buffered_amount();
			// while (BufferBefore + BytesToTransmit >= 16 * 1024 * 1024) // 16MB (WebRTC Data Channel buffer size)
			// {
			// 	// As per UE docs a Sleep of 0.0 simply lets other threads take CPU cycles while this is happening.
			// 	FPlatformProcess::Sleep(0.0);
			// 	BufferBefore = SendChannel->buffered_amount();
			// }

			Send(Builder.Buffer);

			// Increment the number of bytes transmitted
			BytesTransmitted += BytesToTransmit;
		}

		return true;
	}

	TSharedPtr<FEpicRtcMutliplexDataTrack> FEpicRtcMutliplexDataTrack::Create(TSharedPtr<FEpicRtcDataTrack> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol, const FString& InPlayerId)
	{
		TSharedPtr<FEpicRtcMutliplexDataTrack> DataTrack = TSharedPtr<FEpicRtcMutliplexDataTrack>(new FEpicRtcMutliplexDataTrack(InTrack, InDataProtocol, InPlayerId));
		return DataTrack;
	}

	FEpicRtcMutliplexDataTrack::FEpicRtcMutliplexDataTrack(TSharedPtr<FEpicRtcDataTrack> InTrack, TWeakPtr<IPixelStreaming2DataProtocol> InDataProtocol, const FString& InPlayerId)
		: FEpicRtcDataTrack(InTrack, InDataProtocol)
		, PlayerId(InPlayerId)
	{
	}

	void FEpicRtcMutliplexDataTrack::PrependData(FBufferBuilder& Builder) const
	{
		uint8 Type;
		if (!GetMessageId(EPixelStreaming2FromStreamerMessage::Multiplexed, Type))
		{
			return;
		}

		uint16 StringLength = static_cast<uint16>(GetByteSizeOf(Forward<FString>(FString(PlayerId))));

		Builder.Buffer.SetNum(Builder.Buffer.Num() + GetByteSizeOf(Forward<uint8>(Type)) + GetByteSizeOf(Forward<uint16>(StringLength)) + StringLength);

		Builder.Insert(Forward<uint8>(Type));
		Builder.Insert(Forward<uint16>(StringLength));
		Builder.Insert(Forward<FString>(FString(PlayerId)));
	}
} // namespace UE::PixelStreaming2
