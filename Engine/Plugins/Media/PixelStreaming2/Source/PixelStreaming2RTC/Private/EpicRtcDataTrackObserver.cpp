// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcDataTrackObserver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicRtcDataTrackObserver)

namespace UE::PixelStreaming2
{
	FEpicRtcDataTrackObserver::FEpicRtcDataTrackObserver(TObserverVariant<IPixelStreaming2DataTrackObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	void FEpicRtcDataTrackObserver::OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State)
	{
		if (UserObserver)
		{
			UserObserver->OnDataTrackState(DataTrack, State);
		}
	}

	void FEpicRtcDataTrackObserver::OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame)
	{
		if (UserObserver)
		{
			UserObserver->OnDataTrackMessage(DataTrack, DataFrame);
		}
	}

	void FEpicRtcDataTrackObserver::OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error)
	{
		if (UserObserver)
		{
			UserObserver->OnDataTrackError(DataTrack, Error);
		}
	}
} // namespace UE::PixelStreaming2
