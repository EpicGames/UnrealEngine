// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcDataTrackObserverFactory.h"

#include "EpicRtcDataTrackObserver.h"

namespace UE::PixelStreaming2
{
	FEpicRtcDataTrackObserverFactory::FEpicRtcDataTrackObserverFactory(TObserverVariant<IPixelStreaming2DataTrackObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	EpicRtcErrorCode FEpicRtcDataTrackObserverFactory::CreateDataTrackObserver(const EpicRtcStringView ParticipantId, const EpicRtcStringView DataTrackId, EpicRtcDataTrackObserverInterface** OutDataTrackObserver)
	{
		EpicRtcDataTrackObserverInterface* DataTrackObserver = new FEpicRtcDataTrackObserver(UserObserver);
		// Because the ptr was created with new, we need to call AddRef ourself (ms spec compliant)
		DataTrackObserver->AddRef();

		*OutDataTrackObserver = DataTrackObserver;
		return EpicRtcErrorCode::Ok;
	}

} // namespace UE::PixelStreaming2
