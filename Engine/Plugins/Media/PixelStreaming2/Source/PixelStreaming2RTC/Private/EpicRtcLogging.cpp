// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcLogging.h"

#include "Logging/StructuredLog.h"
#include "PixelStreaming2PluginSettings.h"

DEFINE_LOG_CATEGORY(LogPixelStreaming2EpicRtc);
DEFINE_LOG_CATEGORY(LogPixelStreaming2WebRtc);

namespace UE::PixelStreaming2
{
	FEpicRtcLogFilter::FEpicRtcLogFilter()
	{
		ParseFilterString(UPixelStreaming2PluginSettings::CVarEpicRtcLogFilter.GetValueOnAnyThread());

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates(); Delegates)
		{
			EpicRtcLogFilterChangedHandle = Delegates->OnEpicRtcLogFilterChanged.AddRaw(this, &FEpicRtcLogFilter::OnEpicRtcLogFilterChanged);
		}
	}

	FEpicRtcLogFilter::FEpicRtcLogFilter(const TArray<FRegexPattern>& RegexPatterns)
		: RegexPatterns(RegexPatterns)
	{
	}

	FEpicRtcLogFilter::~FEpicRtcLogFilter()
	{
		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates(); Delegates && EpicRtcLogFilterChangedHandle.IsValid())
		{
			Delegates->OnEpicRtcLogFilterChanged.Remove(EpicRtcLogFilterChangedHandle);
		}
	}

	bool FEpicRtcLogFilter::IsFiltered(ELogVerbosity::Type LogVerbosity, const FString& LogString)
	{
		bool bFiltered = false;
		for (const FRegexPattern& RegexPattern : RegexPatterns)
		{
			FRegexMatcher RegexMatcher(RegexPattern, LogString);
			bFiltered |= RegexMatcher.FindNext();
		}

		return bFiltered;
	}

	void FEpicRtcLogFilter::OnEpicRtcLogFilterChanged(IConsoleVariable* Var)
	{
		ParseFilterString(Var->GetString());
	}

	void FEpicRtcLogFilter::ParseFilterString(const FString& LogFilterString)
	{
		TArray<FString> Filters;
		LogFilterString.ParseIntoArray(Filters, TEXT("//"), true);

		RegexPatterns.Empty();
		for (const FString& Filter : Filters)
		{
			if (!Filter.IsEmpty())
			{
				RegexPatterns.Add(FRegexPattern(Filter));
			}
		}
	}

	FEpicRtcLogsRedirector::FEpicRtcLogsRedirector(TSharedPtr<ILogManipulator> LogManipulator)
		: LogManipulator(LogManipulator)
	{
	}

	void FEpicRtcLogsRedirector::Log(const EpicRtcLogMessage& Message)
	{
#if !NO_LOGGING
		static const ELogVerbosity::Type EpicRtcToUnrealLogCategoryMap[] = {
			ELogVerbosity::VeryVerbose,
			ELogVerbosity::Verbose,
			ELogVerbosity::Log,
			ELogVerbosity::Warning,
			ELogVerbosity::Error,
			ELogVerbosity::Fatal,
			ELogVerbosity::NoLogging
		};

		if (LogPixelStreaming2EpicRtc.IsSuppressed(EpicRtcToUnrealLogCategoryMap[static_cast<uint8_t>(Message._level)]))
		{
			return;
		}
		FString Msg{ (int32)Message._message._length, Message._message._ptr };

	#define EPICRTC_LOG(LogLevel, LogString)                                                                                                                \
		if (!LogManipulator || (LogManipulator && !LogManipulator->IsFiltered(ELogVerbosity::LogLevel, LogString)))                                         \
		{                                                                                                                                                   \
			UE_LOGFMT(LogPixelStreaming2EpicRtc, LogLevel, "{0}", LogManipulator ? LogManipulator->Censor(ELogVerbosity::LogLevel, LogString) : LogString); \
		}

		switch (Message._level)
		{
			case EpicRtcLogLevel::Trace:
			{
				EPICRTC_LOG(VeryVerbose, Msg);
				break;
			}
			case EpicRtcLogLevel::Debug:
			{
				EPICRTC_LOG(Verbose, Msg);
				break;
			}
			case EpicRtcLogLevel::Info:
			{
				EPICRTC_LOG(Log, Msg);
				break;
			}
			case EpicRtcLogLevel::Warning:
			{
				EPICRTC_LOG(Warning, Msg);
				break;
			}
			case EpicRtcLogLevel::Error:
			{
				EPICRTC_LOG(Error, Msg);
				break;
			}
			case EpicRtcLogLevel::Critical:
			{
				EPICRTC_LOG(Fatal, Msg);
				break;
			}
		}

	#undef EPICRTC_LOG
#endif
	}

} // namespace UE::PixelStreaming2