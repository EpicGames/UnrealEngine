// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcPlatform.h"

#if PLATFORM_ANDROID
	#include "Android/AndroidApplication.h"

	#if USE_ANDROID_JNI
		#include "Android/AndroidJNI.h"
        #include "Android/AndroidJavaEnv.h"
	#endif // USE_ANDROID_JNI

    namespace UE::PixelStreaming2
    {
        JavaVM* FEpicRtcAndroidPlatform::GetJavaVM() 
        {
            return GJavaVM;
        }
    }


    // WebRTC
	#include "sdk/android/native_api/base/init.h"
	#include "modules/utility/include/jvm_android.h"

    extern "C" void InitWebRTCOnAndroid(JavaVM* InJavaVM)
    {
        webrtc::InitAndroid(InJavaVM);
        webrtc::JVM::Initialize(InJavaVM);
    }
#endif // PLATFORM_ANDROID