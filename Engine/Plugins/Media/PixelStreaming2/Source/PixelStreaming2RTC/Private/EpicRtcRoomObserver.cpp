// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcRoomObserver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicRtcRoomObserver)

namespace UE::PixelStreaming2
{
	FEpicRtcRoomObserver::FEpicRtcRoomObserver(TObserverVariant<IPixelStreaming2RoomObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	void FEpicRtcRoomObserver::OnRoomStateUpdate(const EpicRtcRoomState State)
	{
		if (UserObserver)
		{
			UserObserver->OnRoomStateUpdate(State);
		}
	}

	void FEpicRtcRoomObserver::OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant)
	{
		if (UserObserver)
		{
			UserObserver->OnRoomJoinedUpdate(Participant);
		}
	}

	void FEpicRtcRoomObserver::OnRoomLeftUpdate(const EpicRtcStringView ParticipantId)
	{
		if (UserObserver)
		{
			UserObserver->OnRoomLeftUpdate(ParticipantId);
		}
	}

	void FEpicRtcRoomObserver::OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack)
	{
		if (UserObserver)
		{
			UserObserver->OnAudioTrackUpdate(Participant, AudioTrack);
		}
	}

	void FEpicRtcRoomObserver::OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack)
	{
		if (UserObserver)
		{
			UserObserver->OnVideoTrackUpdate(Participant, VideoTrack);
		}
	}

	void FEpicRtcRoomObserver::OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack)
	{
		if (UserObserver)
		{
			UserObserver->OnDataTrackUpdate(Participant, DataTrack);
		}
	}

	[[nodiscard]] EpicRtcSdpInterface* FEpicRtcRoomObserver::OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		if (UserObserver)
		{
			return UserObserver->OnLocalSdpUpdate(Participant, Sdp);
		}
		return Sdp;
	}

	[[nodiscard]] EpicRtcSdpInterface* FEpicRtcRoomObserver::OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		if (UserObserver)
		{
			return UserObserver->OnRemoteSdpUpdate(Participant, Sdp);
		}
		return Sdp;
	}

	void FEpicRtcRoomObserver::OnRoomErrorUpdate(const EpicRtcErrorCode Error)
	{
		if (UserObserver)
		{
			return UserObserver->OnRoomErrorUpdate(Error);
		}
	}

} // namespace UE::PixelStreaming2
