// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcSessionObserver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicRtcSessionObserver)

namespace UE::PixelStreaming2
{

	FEpicRtcSessionObserver::FEpicRtcSessionObserver(TObserverVariant<IPixelStreaming2SessionObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	void FEpicRtcSessionObserver::OnSessionStateUpdate(const EpicRtcSessionState State)
	{
		if (UserObserver)
		{
			return UserObserver->OnSessionStateUpdate(State);
		}
	}

	void FEpicRtcSessionObserver::OnSessionErrorUpdate(const EpicRtcErrorCode Error)
	{
		if (UserObserver)
		{
			return UserObserver->OnSessionErrorUpdate(Error);
		}
	}

	void FEpicRtcSessionObserver::OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList)
	{
		if (UserObserver)
		{
			return UserObserver->OnSessionRoomsAvailableUpdate(RoomsList);
		}
	}

} // namespace UE::PixelStreaming2
