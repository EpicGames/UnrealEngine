// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcStatsCollector.h"

#include "UtilsString.h"

namespace UE::PixelStreaming2
{
	void FEpicRtcStatsCollector::OnStatsDelivered(const EpicRtcStatsReport& InReport)
	{
		/**
		 * (Nazar.Rudenko): We care only for EpicRtcConnectionStats.
		 * Every EpicRtcConnectionStats object corresponds to a Player/Streamer
		 */
		for (int s = 0; s < InReport._sessionStats._size; s++)
		{
			const EpicRtcSessionStats& Session = InReport._sessionStats._ptr[s];

			for (int r = 0; r < Session._roomStats._size; r++)
			{
				const EpicRtcRoomStats& Room = Session._roomStats._ptr[r];

				for (int c = 0; c < Room._connectionStats._size; c++)
				{
					const EpicRtcConnectionStats& Connection = Room._connectionStats._ptr[c];
					FString						  CollectorId = ToString(Connection._connectionId);

					OnStatsReady.Broadcast(CollectorId, Connection);
				}
			}
		}
	}
} // namespace UE::PixelStreaming2