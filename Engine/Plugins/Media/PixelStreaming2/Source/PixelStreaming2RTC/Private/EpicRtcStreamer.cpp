// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcStreamer.h"

#include "AudioCapturer.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2Delegates.h"
#include "Engine/GameEngine.h"
#include "Engine/GameViewportClient.h"
#include "Stats.h"
#include "PixelStreaming2StatNames.h"
#include "PixelStreaming2RTCModule.h"
#include "EpicRtcTrack.h"
#include "UtilsAsync.h"
#include "UtilsCodecs.h"
#include "VideoProducerBackBuffer.h"
#include "RTCInputHandler.h"
#include "PixelStreaming2Common.h"

namespace UE::PixelStreaming2
{
	FEpicRtcStreamer::FEpicRtcStreamer(const FString& InStreamerId, TRefCountPtr<EpicRtcConferenceInterface> Conference)
		: StreamerId(InStreamerId)
		, Participants(new TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>())
		, VideoCapturer(FEpicRtcVideoCapturer::Create())
		, VideoSourceGroup(FVideoSourceGroup::Create(VideoCapturer))
		, ReconnectTimer(MakeShared<FStreamerReconnectTimer>())
		, EpicRtcConference(MoveTemp(Conference))
	{
	}

	FEpicRtcStreamer::~FEpicRtcStreamer()
	{
		StopStreaming();

		// Pin ref counted objects and clear the session if it is still alive.
		// Session may still be alive if the manager was destroyed quickly after stopping the stream
		// and the EpicRtcSessionState::Disconnected event did not have enough time to tick.
		const TRefCountPtr<EpicRtcSessionInterface> Session(EpicRtcSession);
		if (EpicRtcConference.IsValid() && Session.IsValid())
		{
			const FUtf8String Utf8StreamerId(StreamerId);
			EpicRtcConference->RemoveSession(ToEpicRtcStringView(Utf8StreamerId));
			EpicRtcSession = nullptr;
		}
	}

	void FEpicRtcStreamer::Initialize()
	{
		TSharedPtr<FEpicRtcStreamer> Streamer = AsShared();
		TWeakPtr<FEpicRtcStreamer>	 WeakStreamer = Streamer;

		SessionObserver = MakeRefCount<FEpicRtcSessionObserver>(TObserver<IPixelStreaming2SessionObserver>(WeakStreamer));
		RoomObserver = MakeRefCount<FEpicRtcRoomObserver>(TObserver<IPixelStreaming2RoomObserver>(WeakStreamer));

		AudioTrackObserverFactory = MakeRefCount<FEpicRtcAudioTrackObserverFactory>(TObserver<IPixelStreaming2AudioTrackObserver>(WeakStreamer));
		VideoTrackObserverFactory = MakeRefCount<FEpicRtcVideoTrackObserverFactory>(TObserver<IPixelStreaming2VideoTrackObserver>(WeakStreamer));
		DataTrackObserverFactory = MakeRefCount<FEpicRtcDataTrackObserverFactory>(TObserver<IPixelStreaming2DataTrackObserver>(WeakStreamer));

		VideoCapturer->OnFrameCapturerCreated.AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::OnFrameCapturerCreated);

		if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
		{
			Delegates->OnStatChangedNative.AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::ConsumeStats);
			Delegates->OnAllConnectionsClosedNative.AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::TriggerMouseLeave);
		}

		FPixelStreaming2RTCModule::GetModule()->GetStatsCollector()->OnStatsReady.AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::OnStatsReady);
	}

	void FEpicRtcStreamer::OnProtocolUpdated()
	{
		Participants->Apply([this](FString DataPlayerId, TSharedPtr<FPlayerContext> PlayerContext) {
			SendProtocol(DataPlayerId);
		});
	}

	void FEpicRtcStreamer::SetStreamFPS(int32 InFramesPerSecond)
	{
		VideoSourceGroup->SetFPS(InFramesPerSecond);
	}

	int32 FEpicRtcStreamer::GetStreamFPS()
	{
		return VideoSourceGroup->GetFPS();
	}

	void FEpicRtcStreamer::SetCoupleFramerate(bool bCouple)
	{
		VideoSourceGroup->SetDecoupleFramerate(!bCouple);
	}

	void FEpicRtcStreamer::SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> Producer)
	{
		VideoCapturer->SetVideoProducer(Producer);
	}

	TWeakPtr<IPixelStreaming2VideoProducer> FEpicRtcStreamer::GetVideoProducer()
	{
		return VideoCapturer->GetVideoProducer();
	}

	void FEpicRtcStreamer::AddAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) 
	{
		{
			FScopeLock Lock(&CustomAudioProducersCS);
			CustomAudioProducers.Add(AudioProducer);
		}

		if (!AudioCapturer)
		{
			return;
		}

		AudioCapturer->AddAudioProducer(AudioProducer);
	}

	void FEpicRtcStreamer::RemoveAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) 
	{
		{
			FScopeLock Lock(&CustomAudioProducersCS);
			if (CustomAudioProducers.Contains(AudioProducer))
			{
				CustomAudioProducers.Remove(AudioProducer);
			}
		}

		if (!AudioCapturer)
		{
			return;
		}

		AudioCapturer->RemoveAudioProducer(AudioProducer);
	}

	TArray<TWeakPtr<IPixelStreaming2AudioProducer>> FEpicRtcStreamer::GetAudioProducers() 
	{
		TArray<TWeakPtr<IPixelStreaming2AudioProducer>> WeakArray;
		WeakArray.SetNum(CustomAudioProducers.Num());

		Algo::Transform(CustomAudioProducers, WeakArray, [](const TSharedPtr<IPixelStreaming2AudioProducer>& Shared)
		{
    		return TWeakPtr<IPixelStreaming2AudioProducer>(Shared);
		});

		return WeakArray;
	}

	void FEpicRtcStreamer::SetConnectionURL(const FString& InConnectionURL)
	{
		CurrentSignallingServerURL = InConnectionURL;
	}

	FString FEpicRtcStreamer::GetConnectionURL()
	{
		return CurrentSignallingServerURL;
	}

	void FEpicRtcStreamer::StartStreaming()
	{
		if (StreamState != EStreamState::Disconnected)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Log, "Streamer is already streaming. Ignoring subsequent call to StartStreaming!");
			return;
		}
		
		if (CurrentSignallingServerURL.IsEmpty())
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Attempted to start streamer ({0}) but no signalling server URL has been set. Use Streamer->SetConnectionURL(URL) or -PixelStreamingConnectionURL=", StreamerId);
			return;
		}

		check(EpicRtcConference.IsValid());

		StreamState = EStreamState::Connecting;

		ReconnectTimer->Stop();

		VideoCapturer->ResetFrameCapturer();

		TickableTasks = FPixelStreaming2RTCModule::GetModule()->GetSharedTickableTasks();
		AudioCapturer = FAudioCapturer::Create<FEpicRtcAudioCapturer>();

		{
			FScopeLock Lock(&CustomAudioProducersCS);
			if (CustomAudioProducers.Num() > 0)
			{
				for (TSharedPtr<IPixelStreaming2AudioProducer>& AudioProducer : CustomAudioProducers)
				{
					AudioCapturer->AddAudioProducer(AudioProducer);
				}
			}
		}

		// Broadcast the preconnection event just before we do `TryConnect`
		OnPreConnection().Broadcast(this);

		VideoSourceGroup->Start();

		FUtf8String Utf8StreamerId(StreamerId);
		FUtf8String Utf8CurrentSignallingServerURL(CurrentSignallingServerURL);

		EpicRtcSessionConfig SessionConfig{
			._id = ToEpicRtcStringView(Utf8StreamerId),
			._url = ToEpicRtcStringView(Utf8CurrentSignallingServerURL),
			._observer = SessionObserver
		};

		if (const EpicRtcErrorCode Result = EpicRtcConference->CreateSession(SessionConfig, EpicRtcSession.GetInitReference()); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create EpicRtc session. CreateSession returned {0}", *ToString(Result));
			StreamState = EStreamState::Disconnected;
			return;
		}

		// TODO (william.belcher): This should move to OnSessionStateUpdate(EpicRtcSessionState::New) once EpicRtc starts broadcasting that state
		if (const EpicRtcErrorCode Result = EpicRtcSession->Connect(); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to connect EpicRtcSession. Connect returned {0}", ToString(Result));
			StreamState = EStreamState::Disconnected;
			return;
		}
	}

	void FEpicRtcStreamer::StopStreaming()
	{
		if (StreamState == EStreamState::Disconnected || StreamState == EStreamState::Disconnecting)
		{
			// We've received another call to StopStreaming while we're mid disconnection. Stop the reconnect timer if it was active
			ReconnectTimer->Stop();
			return;
		}

		StreamState = EStreamState::Disconnecting;

		if (const TRefCountPtr<EpicRtcRoomInterface> Room = EpicRtcRoom)
		{
			Room->Leave();
		}

		VideoSourceGroup->Stop();
		TriggerMouseLeave(StreamerId);

		DeleteAllPlayerSessions();

		TickableTasks.Reset();
		AudioCapturer.Reset();
	}

	void FEpicRtcStreamer::OnStatsReady(const FString& PlayerId, const EpicRtcConnectionStats& ConnectionStats)
	{
		TSharedPtr<FRTCStatsCollector> StatsCollector;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			StatsCollector = Participant->StatsCollector;
		}

		if (StatsCollector)
		{
			StatsCollector->Process(ConnectionStats);
		}
	}

	void FEpicRtcStreamer::OnFrameCapturerCreated()
	{
		if (FStats* PSStats = FStats::Get())
		{
			PSStats->RemoveAllApplicationStats();

			// Re-add the cvar based application stats so that they're at the top
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.Encoder.MinQuality")) }, UPixelStreaming2PluginSettings::CVarEncoderMinQuality.GetValueOnAnyThread(), 0));
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.Encoder.MaxQuality")) }, UPixelStreaming2PluginSettings::CVarEncoderMaxQuality.GetValueOnAnyThread(), 0));
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.Encoder.KeyframeInterval (frames)")) }, UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval.GetValueOnAnyThread(), 0));
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.Fps")) }, UPixelStreaming2PluginSettings::CVarWebRTCFps.GetValueOnAnyThread(), 0));
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.StartBitrate")) }, UPixelStreaming2PluginSettings::CVarWebRTCStartBitrate.GetValueOnAnyThread(), 0));
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.MinBitrate")) }, UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread(), 0));
			PSStats->StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.MaxBitrate")) }, UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread(), 0));
		}
	}

	void FEpicRtcStreamer::OnUIInteraction(FMemoryReader Ar)
	{
		if (Ar.TotalSize() < 2)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer[\"{0}\"]::OnUIInteraction: Malformed message. Expected to contain at least 2 bytes representing the descriptor length in characters", StreamerId);
			return;
		}

		uint16 StringLength = 0;
		Ar << StringLength;

		if (Ar.TotalSize() < static_cast<int64>(2 + StringLength * sizeof(TCHAR)))
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer[\"{0}\"]::OnUIInteraction: Malformed message. Descriptor length of {1} characters exceeds remaining message size", StreamerId, StringLength);
			return;
		}

		FString Descriptor;
		Descriptor.GetCharArray().SetNumUninitialized(StringLength + 1 /* +1 for null terminator */);
		// Read archive into the string's char array
		Ar.Serialize(Descriptor.GetCharArray().GetData(), StringLength * sizeof(TCHAR));
		// Null terminate
		Descriptor.GetCharArray()[StringLength] = TEXT('\0');

		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer[\"{0}\"]::OnUIInteraction({1})", StreamerId, Descriptor);
		InputComponents.Apply([Descriptor](uintptr_t Key, UPixelStreaming2Input* Value) { 
			Value->OnInputEvent.Broadcast(Descriptor); 
		});
	}

	void FEpicRtcStreamer::OnSendMessage(FString MessageName, FMemoryReader Ar)
	{
		FString Descriptor;
		Ar << Descriptor;
		SendAllPlayersMessage(MessageName, Descriptor);
	}

	EpicRtcBitrate FEpicRtcStreamer::GetBitrates(bool bIncludeStartBitrate) const
	{
		const int MinBitrate = UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread();
		const int MaxBitrate = UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread();

		EpicRtcBitrate Bitrates = {
			._minBitrateBps = MinBitrate,
			._hasMinBitrateBps = true,
			._maxBitrateBps = MaxBitrate,
			._hasMaxBitrateBps = true,
			._startBitrateBps = 0,
			._hasStartBitrateBps = false
		};

		if (bIncludeStartBitrate)
		{
			const int StartBitrate = FMath::Clamp(UPixelStreaming2PluginSettings::CVarWebRTCStartBitrate.GetValueOnAnyThread(), MinBitrate, MaxBitrate);
			Bitrates._startBitrateBps = StartBitrate;
			Bitrates._hasStartBitrateBps = true;
		}

		return Bitrates;
	}

	IPixelStreaming2Streamer::FPreConnectionEvent& FEpicRtcStreamer::OnPreConnection()
	{
		return StreamingPreConnectionEvent;
	}

	IPixelStreaming2Streamer::FStreamingStartedEvent& FEpicRtcStreamer::OnStreamingStarted()
	{
		return StreamingStartedEvent;
	}

	IPixelStreaming2Streamer::FStreamingStoppedEvent& FEpicRtcStreamer::OnStreamingStopped()
	{
		return StreamingStoppedEvent;
	}

	void FEpicRtcStreamer::ForceKeyFrame()
	{
		VideoSourceGroup->ForceKeyFrame();
	}

	void FEpicRtcStreamer::FreezeStream(UTexture2D* Texture)
	{
		if (FreezeFrame)
		{
			FreezeFrame->StartFreeze(Texture);
		}
	}

	void FEpicRtcStreamer::UnfreezeStream()
	{
		// Force a keyframe so when stream unfreezes if player has never received a frame before they can still connect.
		ForceKeyFrame();
		if (FreezeFrame)
		{
			FreezeFrame->StopFreeze();
		}
	}

	void FEpicRtcStreamer::SendAllPlayersMessage(FString MessageType, const FString& Descriptor)
	{
		Participants->Apply([&MessageType, &Descriptor](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
			if (Participant->DataTrack && !IsSFU(PlayerId))
			{
				Participant->DataTrack->SendMessage(MessageType, Descriptor);
			}
		});
	}

	void FEpicRtcStreamer::SendPlayerMessage(FString PlayerId, FString MessageType, const FString& Descriptor)
	{
		if (IsSFU(PlayerId))
		{
			return;
		}

		TSharedPtr<FEpicRtcDataTrack> DataTrack;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			DataTrack = Participant->DataTrack;
		}

		if (DataTrack)
		{
			DataTrack->SendMessage(MessageType, Descriptor);
		}
	}

	void FEpicRtcStreamer::SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension)
	{
		// TODO this should be dispatched as an async task, but because we lock when we visit the data
		// channels it might be a bad idea. At some point it would be good to take a snapshot of the
		// keys in the map when we start, then one by one get the channel and send the data

		Participants->Apply([&ByteData, &MimeType, &FileExtension](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
			if (!Participant->DataTrack)
			{
				return;
			}

			// Send the mime type first
			Participant->DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::FileMimeType, MimeType);

			// Send the extension next
			Participant->DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::FileExtension, FileExtension);

			// Send the contents of the file. Note to callers: consider running this on its own thread, it can take a while if the file is big.
			Participant->DataTrack->SendArbitraryData(EPixelStreaming2FromStreamerMessage::FileContents, ByteData);
		});
	}

	void FEpicRtcStreamer::KickPlayer(FString PlayerId)
	{
		TRefCountPtr<EpicRtcParticipantInterface> ParticipantInterface;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			ParticipantInterface = Participant->ParticipantInterface;
		}

		if (ParticipantInterface)
		{
			ParticipantInterface->Kick();
		}
	}

	TArray<FString> FEpicRtcStreamer::GetConnectedPlayers()
	{
		TSet<FString> ConnectedParticipantIds;
		Participants->GetKeys(ConnectedParticipantIds);

		return ConnectedParticipantIds.Array();
	}

	void FEpicRtcStreamer::SetInputHandler(TSharedPtr<IPixelStreaming2InputHandler> Handler)
	{
		InputHandler = Handler;

		if (!InputHandler)
		{
			return;
		}

		// Re-create the freeze frame manager with the new input handler
		FreezeFrame = FFreezeFrame::Create(Participants, VideoCapturer, InputHandler);

		InputHandler->SetElevatedCheck([this](FString PlayerId) {
			return GetEnumFromCVar<EInputControllerMode>(UPixelStreaming2PluginSettings::CVarInputController) == EInputControllerMode::Any
				|| InputControllingId == INVALID_PLAYER_ID
				|| PlayerId == InputControllingId;
		});

		TSharedPtr<FEpicRtcStreamer> Streamer = AsShared();
		TWeakPtr<FEpicRtcStreamer>	 WeakStreamer = Streamer;

		InputHandler->GetToStreamerProtocol()->OnProtocolUpdated().AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::OnProtocolUpdated);
		InputHandler->GetFromStreamerProtocol()->OnProtocolUpdated().AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::OnProtocolUpdated);

		// Set Encoder.MinQP Legacy CVar
		InputHandler->SetCommandHandler(TEXT("Encoder.MinQP"), [](FString PlayerId, FString Descriptor, FString MinQPString) {
			int MinQP = FCString::Atoi(*MinQPString);
			UPixelStreaming2PluginSettings::CVarEncoderMaxQuality->SetWithCurrentPriority(FMath::RoundToFloat(100.0f * (1.0f - (FMath::Clamp<int32>(MinQP, 0, 51) / 51.0f))));
		});

		// Set Encoder.MaxQP Legacy CVar
		InputHandler->SetCommandHandler(TEXT("Encoder.MaxQP"), [](FString PlayerId, FString Descriptor, FString MaxQPString) {
			int MaxQP = FCString::Atoi(*MaxQPString);
			UPixelStreaming2PluginSettings::CVarEncoderMinQuality->SetWithCurrentPriority(FMath::RoundToFloat(100.0f * (1.0f - (FMath::Clamp<int32>(MaxQP, 0, 51) / 51.0f))));
		});

		// Set Encoder.MinQuality CVar
		InputHandler->SetCommandHandler(TEXT("Encoder.MinQuality"), [](FString PlayerId, FString Descriptor, FString MinQualityString) {
			int MinQuality = FCString::Atoi(*MinQualityString);
			UPixelStreaming2PluginSettings::CVarEncoderMinQuality->SetWithCurrentPriority(FMath::Clamp<int32>(MinQuality, 0, 100));
		});

		// Set Encoder.MaxQuality CVar
		InputHandler->SetCommandHandler(TEXT("Encoder.MaxQuality"), [](FString PlayerId, FString Descriptor, FString MaxQualityString) {
			int MaxQuality = FCString::Atoi(*MaxQualityString);
			UPixelStreaming2PluginSettings::CVarEncoderMaxQuality->SetWithCurrentPriority(FMath::Clamp<int32>(MaxQuality, 0, 100));
		});

		// Set WebRTC max FPS
		InputHandler->SetCommandHandler(TEXT("WebRTC.Fps"), [](FString PlayerId, FString Descriptor, FString FPSString) {
			int FPS = FCString::Atoi(*FPSString);
			UPixelStreaming2PluginSettings::CVarWebRTCFps->SetWithCurrentPriority(FPS);
		});

		// Set MinBitrate
		InputHandler->SetCommandHandler(TEXT("WebRTC.MinBitrate"), [WeakStreamer](FString PlayerId, FString Descriptor, FString MinBitrateString) {
			// This check pattern is kind of verbose, but as the messages are enqueued on a different thread to where they were added
			// we need to make sure that both the streamer and input handler are still alive when we process the command
			TSharedPtr<FEpicRtcStreamer> Streamer = WeakStreamer.Pin();
			if (!Streamer)
			{
				return;
			}
			TSharedPtr<IPixelStreaming2InputHandler> InputHandler = Streamer->GetInputHandler().Pin();
			if (!InputHandler)
			{
				return;
			}

			if (InputHandler->IsElevated(PlayerId))
			{
				int MinBitrate = FCString::Atoi(*MinBitrateString);
				UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate->SetWithCurrentPriority(MinBitrate);
			}
		});

		// Set MaxBitrate
		InputHandler->SetCommandHandler(TEXT("WebRTC.MaxBitrate"), [WeakStreamer](FString PlayerId, FString Descriptor, FString MaxBitrateString) {
			TSharedPtr<FEpicRtcStreamer> Streamer = WeakStreamer.Pin();
			if (!Streamer)
			{
				return;
			}
			TSharedPtr<IPixelStreaming2InputHandler> InputHandler = Streamer->GetInputHandler().Pin();
			if (!InputHandler)
			{
				return;
			}

			if (InputHandler->IsElevated(PlayerId))
			{
				int MaxBitrate = FCString::Atoi(*MaxBitrateString);
				UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate->SetWithCurrentPriority(MaxBitrate);
			}
		});

		InputHandler->RegisterMessageHandler(EPixelStreaming2ToStreamerMessage::UIInteraction, [WeakStreamer](FString PlayerId, FMemoryReader Ar) {
			TSharedPtr<FEpicRtcStreamer> Streamer = WeakStreamer.Pin();
			if (!Streamer)
			{
				return;
			}
			Streamer->OnUIInteraction(Ar);
		});

		// Handle special cases when the InputHandler itself wants to send a message out to all the peers.
		// Some special cases include when virtual gamepads are connected and a controller id needs to be transmitted.
		InputHandler->OnSendMessage.AddSP(Streamer.ToSharedRef(), &FEpicRtcStreamer::OnSendMessage);

		// We need to update the protocol for each existing participant's data track
		TSharedPtr<IPixelStreaming2InputHandler> SharedInputHandler = InputHandler;
		Participants->Apply([SharedInputHandler](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
			TSharedPtr<IPixelStreaming2DataProtocol> Protocol = SharedInputHandler->GetFromStreamerProtocol();
			if (Participant->DataTrack && Protocol)
			{
				Participant->DataTrack->SetDataProtocol(Protocol);
			}
		});
	}

	TWeakPtr<IPixelStreaming2AudioSink> FEpicRtcStreamer::GetPeerAudioSink(FString PlayerId)
	{
		TWeakPtr<IPixelStreaming2AudioSink> Result = nullptr;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			Result = Participant->AudioSink ? Participant->AudioSink : nullptr;
		}

		return Result;
	}

	TWeakPtr<IPixelStreaming2AudioSink> FEpicRtcStreamer::GetUnlistenedAudioSink()
	{
		TWeakPtr<IPixelStreaming2AudioSink> Result = nullptr;
		Participants->ApplyUntil([&Result](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
			if (Participant->AudioSink)
			{
				if (!Participant->AudioSink->HasAudioConsumers())
				{
					Result = Participant->AudioSink;
					return true;
				}
			}
			return false;
		});
		return Result;
	}

	TWeakPtr<IPixelStreaming2VideoSink> FEpicRtcStreamer::GetPeerVideoSink(FString PlayerId)
	{
		TWeakPtr<IPixelStreaming2VideoSink> Result = nullptr;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			Result = Participant->VideoSink ? Participant->VideoSink : nullptr;
		}

		return Result;
	}

	TWeakPtr<IPixelStreaming2VideoSink> FEpicRtcStreamer::GetUnwatchedVideoSink()
	{
		TWeakPtr<IPixelStreaming2VideoSink> Result = nullptr;
		Participants->ApplyUntil([&Result](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
			if (Participant->VideoSink)
			{
				if (!Participant->VideoSink->HasVideoConsumers())
				{
					Result = Participant->VideoSink;
					return true;
				}
			}
			return false;
		});
		return Result;
	}

	void FEpicRtcStreamer::SetConfigOption(const FName& OptionName, const FString& Value)
	{
		if (Value.IsEmpty())
		{
			ConfigOptions.Remove(OptionName);
		}
		else
		{
			ConfigOptions.Add(OptionName, Value);
		}
	}

	bool FEpicRtcStreamer::GetConfigOption(const FName& OptionName, FString& OutValue)
	{
		FString* OptionValue = ConfigOptions.Find(OptionName);
		if (OptionValue)
		{
			OutValue = *OptionValue;
			return true;
		}
		else
		{
			return false;
		}
	}

	void FEpicRtcStreamer::PlayerRequestsBitrate(FString PlayerId, int MinBitrate, int MaxBitrate)
	{
		UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.AsVariable()->SetWithCurrentPriority(MinBitrate);
		UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.AsVariable()->SetWithCurrentPriority(MaxBitrate);
	}

	void FEpicRtcStreamer::RefreshStreamBitrate()
	{
		if (!EpicRtcRoom)
		{
			return;
		}

		TRefCountPtr<EpicRtcConnectionInterface> ConnectionInterface;
		const EpicRtcErrorCode					 Result = EpicRtcRoom->GetConnection(ConnectionInterface.GetInitReference());
		if (Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::RefreshStreamBitrate: Failed to get connection interface. GetConnection returned {0}", *ToString(Result));
			return;
		}

		TSet<FString> PlayerIds;
		Participants->GetKeys(PlayerIds);
		// We need to iterate over just the keys as we can potentially deadlock during player connection with OnDataTrackState having locked the peer connection
		// and SetConnectionRates needing to use the peer connection. Iterating over just the keys is the easiest way to prevent this
		for (const FString& PlayerId : PlayerIds)
		{
			// We don't include the starting bitrate during a bitrate refresh as that causes the peer connection
			// to go back to that value and have to work its way up. This function is also only triggered by changes to min and max
			FUtf8String Utf8PlayerId = *PlayerId;
			ConnectionInterface->SetConnectionRates(ToEpicRtcStringView(Utf8PlayerId), GetBitrates(false /* bIncludeStartBitrate */));
		}
	}

	void FEpicRtcStreamer::ForEachPlayer(const TFunction<void(FString, TSharedPtr<FPlayerContext>)>& Func)
	{
		Participants->Apply(Func);
	}

	void FEpicRtcStreamer::ConsumeStats(FString PlayerId, FName StatName, float StatValue)
	{
		if (IsSFU(PlayerId))
		{
			return;
		}

		if (StatName != PixelStreaming2StatNames::MeanQPPerSecond)
		{
			return;
		}

		TSharedPtr<FEpicRtcDataTrack> DataTrack;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			DataTrack = Participant->DataTrack;
		}

		if (!DataTrack)
		{
			return;
		}

		DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::VideoEncoderAvgQP, FString::FromInt(static_cast<int>(StatValue)));
	}

	void FEpicRtcStreamer::DeletePlayerSession(FString PlayerId)
	{
		// We need to keep a copy of the participant to delete as when destroying the ParticipantInterface,
		// webrtc will finish tasks in the other threads which can execute FindRef and end up deadlocking
		// the map.
		// TL:DR Participant deletion needs to happen outside the TThreadSafeMap scope lock
		TSharedPtr<FPlayerContext> ParticipantToDelete;
		if (!Participants->RemoveAndCopyValue(PlayerId, ParticipantToDelete) || !ParticipantToDelete.IsValid())
		{
			return;
		}

		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::DeletePlayerSession(Participant [{0}])", PlayerId);

		// FIXME (RTCP-7928): EpicRtc currently isn't broadcasting a stopped track state for remote track
		if (ParticipantToDelete->VideoSink)
		{
			CloseVideoTrack(ParticipantToDelete, PlayerId, true);
		}

		if (ParticipantToDelete->AudioSink)
		{
			CloseAudioTrack(ParticipantToDelete, PlayerId, true);
		}

		if (ParticipantToDelete->DataTrack)
		{
			CloseDataTrack(ParticipantToDelete, PlayerId);
		}

		// NOTE (william.belcher) [13/10/25]: As EpicRtc stores a single local track for the streamer that is then mapped to all participants internal to EpicRtc,
		// preventing calls to OnVideoTrackState and OnAudioTrackState, we will manually call CloseVideoTrack and CloseAudioTrack for the local tracks to ensure 
		// our delegates retain functionality
		CloseVideoTrack(ParticipantToDelete, PlayerId, false);
		CloseAudioTrack(ParticipantToDelete, PlayerId, false);

		DoOnGameThread([StreamerId = this->StreamerId, PlayerId, bIsEmpty = Participants->IsEmpty()]() {
			if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
			{
				Delegates->OnClosedConnection.Broadcast(StreamerId, PlayerId);
				Delegates->OnClosedConnectionNative.Broadcast(StreamerId, PlayerId);
				if (bIsEmpty)
				{
					Delegates->OnAllConnectionsClosed.Broadcast(StreamerId);
					Delegates->OnAllConnectionsClosedNative.Broadcast(StreamerId);
				}
			}
		});

		if (FStats* PSStats = FStats::Get())
		{
			PSStats->RemovePeerStats(PlayerId);
		}
	}

	void FEpicRtcStreamer::DeleteAllPlayerSessions()
	{
		if (FStats* PSStats = FStats::Get())
		{
			PSStats->RemoveAllPeerStats();
		}

		TSet<FString> PlayerIds;
		Participants->GetKeys(PlayerIds);
		// We have to iterate the keys separately as OnDataTrackClosed also loops through the player map
		// and we will deadlock
		for (const FString& PlayerId : PlayerIds)
		{
			DeletePlayerSession(PlayerId);
		}

		// Further cleanup
		VideoSourceGroup->RemoveAllVideoSources();
		AudioTrackPlayerIdMap.Empty();
		VideoTrackPlayerIdMap.Empty();
		Participants->Empty();
		InputControllingId = INVALID_PLAYER_ID;
	}

	void FEpicRtcStreamer::OnDataTrackOpen(FString PlayerId)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnDataTrackOpen(Participant [{0}])", PlayerId);

		// Only time we automatically make a new peer the input controlling host is if they are the first peer (and not the SFU).
		const bool HostControlsInput = GetEnumFromCVar<EInputControllerMode>(UPixelStreaming2PluginSettings::CVarInputController) == EInputControllerMode::Host;
		if (HostControlsInput && !IsSFU(PlayerId) && InputControllingId == INVALID_PLAYER_ID)
		{
			InputControllingId = PlayerId;
		}

		DoOnGameThread([StreamerId = this->StreamerId, PlayerId]() {
			if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
			{
				Delegates->OnDataTrackOpen.Broadcast(StreamerId, PlayerId);
				Delegates->OnDataTrackOpenNative.Broadcast(StreamerId, PlayerId);
			}
		});

		// When data channel is open
		SendProtocol(PlayerId);
		SendInitialSettings(PlayerId);
		SendPeerControllerMessages(PlayerId);
		
		// Try to send cached freeze frame (if we have one)
		if (FreezeFrame)
		{
			FreezeFrame->SendCachedFreezeFrameTo(PlayerId);
		}
	}

	void FEpicRtcStreamer::OnDataTrackClosed(FString PlayerId)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnDataTrackClosed(Participant [{0}])", PlayerId);

		TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId);

		CloseDataTrack(Participant, PlayerId);
	}

	void FEpicRtcStreamer::CloseDataTrack(const TSharedPtr<FPlayerContext>& Participant, FString PlayerId)
	{
		if (Participant.IsValid())
		{
			Participant->DataTrack = nullptr;
		}

		if (InputControllingId == PlayerId)
		{
			InputControllingId = INVALID_PLAYER_ID;
			// just get the first channel we have and give it input control.
			Participants->ApplyUntil([this](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
				if (!Participant->DataTrack)
				{
					return false;
				}
				if (IsSFU(PlayerId))
				{
					return false;
				}
				InputControllingId = PlayerId;
				Participant->DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::InputControlOwnership, 1 /* ControlsInput */);
				return true;
			});
		}

		DoOnGameThread([StreamerId = this->StreamerId, PlayerId]() {
			if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
			{
				Delegates->OnDataTrackClosed.Broadcast(StreamerId, PlayerId);
				Delegates->OnDataTrackClosedNative.Broadcast(StreamerId, PlayerId);
			}
		});
	}

	void FEpicRtcStreamer::OnAudioTrackOpen(FString PlayerId, bool bIsRemote)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnAudioTrackOpen(Participant [{0}], IsRemote [{1}])", PlayerId, bIsRemote);

		if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
		{
			// NOTE: Native delegates aren't required to be broadcast on game thread
			Delegates->OnAudioTrackOpenNative.Broadcast(StreamerId, PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::OnAudioTrackClosed(FString PlayerId, bool bIsRemote)
	{
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			CloseAudioTrack(Participant, PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::CloseAudioTrack(const TSharedPtr<FPlayerContext>& Participant, FString PlayerId, bool bIsRemote)
	{
		bool bCallDelegate = false;
		if (bIsRemote)
		{
			if (TSharedPtr<FEpicRtcAudioSink> AudioSink = Participant->AudioSink)
			{
				bCallDelegate = !Participant->BroadcastRemoteAudioTrackClosed.Trigger();
				Participant->AudioSink = nullptr;
			}
		}
		else
		{
			bCallDelegate = !Participant->BroadcastLocalAudioTrackClosed.Trigger();
		}

		if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get(); bCallDelegate && Delegates)
		{
			// NOTE: Native delegates aren't required to be broadcast on game thread
			Delegates->OnAudioTrackClosedNative.Broadcast(StreamerId, PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::OnVideoTrackOpen(FString PlayerId, bool bIsRemote)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnVideoTrackOpen(Participant [{0}], IsRemote [{1}])", PlayerId, bIsRemote);

		if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
		{
			// NOTE: Native delegates aren't required to be broadcast on game thread
			Delegates->OnVideoTrackOpenNative.Broadcast(StreamerId, PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::OnVideoTrackClosed(FString PlayerId, bool bIsRemote)
	{
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			CloseVideoTrack(Participant, PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::CloseVideoTrack(const TSharedPtr<FPlayerContext>& Participant, FString PlayerId, bool bIsRemote)
	{
		bool bCallDelegate = false;
		if (bIsRemote)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnVideoTrackClosed Clear VideoSink (Participant [{0}], IsRemote [{1}])", PlayerId, bIsRemote);
			if (TSharedPtr<FEpicRtcVideoSink> VideoSink = Participant->VideoSink)
			{
				bCallDelegate = !Participant->BroadcastRemoteVideoTrackClosed.Trigger();
				Participant->VideoSink = nullptr;
			}
		}
		else
		{
			bCallDelegate = !Participant->BroadcastLocalVideoTrackClosed.Trigger();
		}

		if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get(); bCallDelegate && Delegates)
		{
			// NOTE: Native delegates aren't required to be broadcast on game thread
			Delegates->OnVideoTrackClosedNative.Broadcast(StreamerId, PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::SendInitialSettings(FString PlayerId) const
	{
		const FString PixelStreaming2Payload = FString::Printf(TEXT("{ \"AllowPixelStreamingCommands\": %s, \"DisableLatencyTest\": %s }"),
			UPixelStreaming2PluginSettings::CVarInputAllowConsoleCommands.GetValueOnAnyThread() ? TEXT("true") : TEXT("false"),
			UPixelStreaming2PluginSettings::CVarDisableLatencyTester.GetValueOnAnyThread() ? TEXT("true") : TEXT("false"));

		const FString WebRTCPayload = FString::Printf(TEXT("{ \"FPS\": %d, \"MinBitrate\": %d, \"MaxBitrate\": %d }"),
			UPixelStreaming2PluginSettings::CVarWebRTCFps.GetValueOnAnyThread(),
			UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread(),
			UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread());

		const FString EncoderPayload = FString::Printf(TEXT("{ \"TargetBitrate\": %d, \"MinQuality\": %d, \"MaxQuality\": %d }"),
			UPixelStreaming2PluginSettings::CVarEncoderTargetBitrate.GetValueOnAnyThread(),
			UPixelStreaming2PluginSettings::CVarEncoderMinQuality.GetValueOnAnyThread(),
			UPixelStreaming2PluginSettings::CVarEncoderMaxQuality.GetValueOnAnyThread());

		FString ConfigPayload = TEXT("{ ");
		bool	bComma = false; // Simplest way to avoid complaints from pedantic JSON parsers
		for (const TPair<FName, FString>& Option : ConfigOptions)
		{
			if (bComma)
			{
				ConfigPayload.Append(TEXT(", "));
			}
			ConfigPayload.Append(FString::Printf(TEXT("\"%s\": \"%s\""), *Option.Key.ToString(), *Option.Value));
			bComma = true;
		}
		ConfigPayload.Append(TEXT("}"));

		const FString FullPayload = FString::Printf(TEXT("{ \"PixelStreaming\": %s, \"Encoder\": %s, \"WebRTC\": %s, \"ConfigOptions\": %s }"), *PixelStreaming2Payload, *EncoderPayload, *WebRTCPayload, *ConfigPayload);

		TSharedPtr<FEpicRtcDataTrack> DataTrack;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			DataTrack = Participant->DataTrack;
		}

		if (!DataTrack)
		{
			return;
		}

		DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::InitialSettings, FullPayload);
	}

	void FEpicRtcStreamer::SendProtocol(FString PlayerId) const
	{
		if (!InputHandler)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Cannot send protocol when input handler is not valid!");
			return;
		}

		const TArray<TSharedPtr<IPixelStreaming2DataProtocol>> Protocols = { InputHandler->GetToStreamerProtocol(), InputHandler->GetFromStreamerProtocol() };
		for (TSharedPtr<IPixelStreaming2DataProtocol> Protocol : Protocols)
		{
			TSharedPtr<FJsonObject>	  ProtocolJson = Protocol->ToJson();
			FString					  Body;
			TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&Body);
			if (!ensure(FJsonSerializer::Serialize(ProtocolJson.ToSharedRef(), JsonWriter)))
			{
				UE_LOGF(LogPixelStreaming2RTC, Warning, "Cannot serialize protocol json object");
				return;
			}

			TSharedPtr<FEpicRtcDataTrack> DataTrack;
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
			{
				DataTrack = Participant->DataTrack;
			}

			if (!DataTrack)
			{
				return;
			}

			DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::Protocol, Body);
		}
	}

	void FEpicRtcStreamer::SendPeerControllerMessages(FString PlayerId) const
	{
		TSharedPtr<FEpicRtcDataTrack> DataTrack;
		{
			TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId);
			if (!Participant)
			{
				return;
			}

			DataTrack = Participant->DataTrack;
		}

		if (!DataTrack)
		{
			return;
		}

		const uint8 ControlsInput = (GetEnumFromCVar<EInputControllerMode>(UPixelStreaming2PluginSettings::CVarInputController) == EInputControllerMode::Host) ? (PlayerId == InputControllingId) : 1;
		// Even though the QualityController feature is removed we send it for backwards compatibility with older frontends (can probably remove 2 versions after 5.5)
		DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::InputControlOwnership, ControlsInput);
		DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::QualityControlOwnership, 1 /* True */);
	}

	void FEpicRtcStreamer::SendLatencyReport(FString PlayerId) const
	{
		if (UPixelStreaming2PluginSettings::CVarDisableLatencyTester.GetValueOnAnyThread())
		{
			return;
		}

		double ReceiptTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		DoOnGameThread([this, PlayerId, ReceiptTimeMs]() {
			FString ReportToTransmitJSON;

			if (!UPixelStreaming2PluginSettings::CVarWebRTCDisableStats.GetValueOnAnyThread())
			{
				double EncodeMs = -1.0;
				double CaptureToSendMs = 0.0;

				FStats* Stats = FStats::Get();
				if (Stats)
				{
					Stats->QueryPeerStat(PlayerId, FName(*RTCStatCategories::LocalVideoTrack), PixelStreaming2StatNames::MeanEncodeTime, EncodeMs);
					Stats->QueryPeerStat(PlayerId, FName(*RTCStatCategories::LocalVideoTrack), PixelStreaming2StatNames::MeanSendDelay, CaptureToSendMs);
				}

				double TransmissionTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
				ReportToTransmitJSON = FString::Printf(
					TEXT("{ \"ReceiptTimeMs\": %.2f, \"EncodeMs\": %.2f, \"CaptureToSendMs\": %.2f, \"TransmissionTimeMs\": %.2f }"),
					ReceiptTimeMs,
					EncodeMs,
					CaptureToSendMs,
					TransmissionTimeMs);
			}
			else
			{
				double TransmissionTimeMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
				ReportToTransmitJSON = FString::Printf(
					TEXT("{ \"ReceiptTimeMs\": %.2f, \"EncodeMs\": \"Pixel Streaming stats are disabled\", \"CaptureToSendMs\": \"Pixel Streaming stats are disabled\", \"TransmissionTimeMs\": %.2f }"),
					ReceiptTimeMs,
					TransmissionTimeMs);
			}

			TSharedPtr<FEpicRtcDataTrack> DataTrack;
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
			{
				DataTrack = Participant->DataTrack;
			}

			if (!DataTrack)
			{
				return;
			}

			DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::LatencyTest, ReportToTransmitJSON);
		});
	}

	void FEpicRtcStreamer::HandleRelayStatusMessage(const uint8_t* Data, uint32_t Size, EpicRtcDataTrackInterface* DataTrack)
	{
		// skip type
		Data++;
		Size--;
		FString														   PlayerId = ReadString(Data, Size);
		checkf(Size > 0, TEXT("Malformed relay status message!")) bool bIsOn = static_cast<bool>(Data[0]);

		FString DataTrackId = ToString(DataTrack->GetId());
		if (bIsOn)
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::HandleRelayStatusMessage() Adding new PlayerId [%ls] with DataTrackId [%ls]", *PlayerId, *DataTrackId);

			FString SFUId;
			if (FindPlayerFromTrack(DataTrack, SFUId))
			{
				TSharedPtr<FEpicRtcDataTrack> SFUDataTrack;
				if (TSharedPtr<FPlayerContext> SFUParticipant = Participants->FindRef(SFUId); SFUParticipant.IsValid())
				{
					SFUDataTrack = SFUParticipant->DataTrack;
				}

				if (SFUDataTrack)
				{
					TSharedPtr<FPlayerContext>& Participant = Participants->FindOrAdd(PlayerId);
					Participant = MakeShared<FPlayerContext>();
					if (!InputHandler)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Cannot create SFU data track with an invalid input handler!");
						return;
					}
					Participant->DataTrack = FEpicRtcMutliplexDataTrack::Create(SFUDataTrack, InputHandler->GetFromStreamerProtocol(), PlayerId);
					OnDataTrackOpen(PlayerId);
				}
			}
			else
			{
				UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::HandleRelayStatusMessage() Failed to find SFU PlayerContext");
			}
		}
		else
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::HandleRelayStatusMessage() Removing PlayerId [%ls] with DataTrackId [%ls]", *PlayerId, *DataTrackId);

			OnDataTrackClosed(PlayerId);
			Participants->Remove(PlayerId);
		}
	}

	void FEpicRtcStreamer::TriggerMouseLeave(FString InStreamerId)
	{
		if (!IsEngineExitRequested() && StreamerId == InStreamerId)
		{
			TSharedPtr<IPixelStreaming2InputHandler> SharedInputHandler = InputHandler;

			// Force a MouseLeave event. This prevents the PixelStreaming2ApplicationWrapper from
			// still wrapping the base FSlateApplication after we stop streaming
			const auto MouseLeaveFunction = [SharedInputHandler]() {
				if (SharedInputHandler.IsValid())
				{
					TArray<uint8>							EmptyArray;
					TFunction<void(FString, FMemoryReader)> MouseLeaveHandler = SharedInputHandler->FindMessageHandler("MouseLeave");
					if (MouseLeaveHandler)
					{
						MouseLeaveHandler("", FMemoryReader(EmptyArray));
					}
				}
			};

			if (IsInGameThread())
			{
				MouseLeaveFunction();
			}
			else
			{
				DoOnGameThread([MouseLeaveFunction]() {
					MouseLeaveFunction();
				});
			}
		}
	}

	void FEpicRtcStreamer::OnSessionStateUpdate(const EpicRtcSessionState State)
	{
		switch (State)
		{
			case EpicRtcSessionState::New: // Indicates newly created session.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionStateUpdate State=New");
				break;
			}
			case EpicRtcSessionState::Pending: // Indicates connection is in progress.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionStateUpdate State=Pending");
				break;
			}
			case EpicRtcSessionState::Connected: // Indicates session is connected to signalling server.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionStateUpdate State=Connected");
				DoOnGameThread([StreamerId = this->StreamerId]() {
					if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
					{
						Delegates->OnConnectedToSignallingServer.Broadcast(StreamerId);
						Delegates->OnConnectedToSignallingServerNative.Broadcast(StreamerId);
					}
				});

				const EpicRtcBitrate Bitrate = GetBitrates(true /* bIncludeStartBitrate */);

				EpicRtcPortAllocator PortAllocator = {
					._minPort = UPixelStreaming2PluginSettings::CVarWebRTCMinPort.GetValueOnAnyThread(),
					._hasMinPort = true,
					._maxPort = UPixelStreaming2PluginSettings::CVarWebRTCMaxPort.GetValueOnAnyThread(),
					._hasMaxPort = true,
					._portAllocation = static_cast<EpicRtcPortAllocatorOptions>(UPixelStreaming2PluginSettings::GetPortAllocationFlags())
				};

				EpicRtcConnectionConfig ConnectionConfig = {
					._iceServers = { ._ptr = nullptr, ._size = 0 }, // This can stay empty because EpicRtc handles the ice servers internally
					._portAllocator = PortAllocator,
					._bitrate = Bitrate,
					._iceConnectionPolicy = EpicRtcIcePolicy::All,
					._disableTcpCandidates = false
				};

				FUtf8String		  Utf8StreamerId(StreamerId);
				EpicRtcRoomConfig RoomConfig = {
					._id = ToEpicRtcStringView(Utf8StreamerId),
					._connectionConfig = ConnectionConfig,
					._ticket = EpicRtcStringView{ ._ptr = nullptr, ._length = 0 },
					._joinParameters = EpicRtcSignallingJoinParameters{
						._options = nullptr,
						._reservedAudioStreamsMaxPoolSize = 0,
						._audioOnly = false
					},
					._observer = RoomObserver,
					._audioTrackObserverFactory = AudioTrackObserverFactory,
					._dataTrackObserverFactory = DataTrackObserverFactory,
					._videoTrackObserverFactory = VideoTrackObserverFactory
				};

				EpicRtcErrorCode Result = EpicRtcSession->CreateRoom(RoomConfig, EpicRtcRoom.GetInitReference());
				if (Result != EpicRtcErrorCode::Ok)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create EpicRtc room. CreateRoom returned {0}", ToString(Result));
					break;
				}

				TRefCountPtr<EpicRtcConnectionInterface> ConnectionInterface;
				Result = EpicRtcRoom->GetConnection(ConnectionInterface.GetInitReference());
				if (Result != EpicRtcErrorCode::Ok)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnSessionStateUpdate: Failed to get connection interface. GetConnection returned {0}", *ToString(Result));
					break;
				}

				const EVideoCodec SelectedCodec = GetEnumFromCVar<EVideoCodec>(UPixelStreaming2PluginSettings::CVarEncoderCodec);
				const bool		  bNegotiateCodecs = UPixelStreaming2PluginSettings::CVarWebRTCNegotiateCodecs.GetValueOnAnyThread();
				const bool		  bTransmitUEVideo = !UPixelStreaming2PluginSettings::CVarWebRTCDisableTransmitVideo.GetValueOnAnyThread();
				bool			  bReceiveBrowserVideo = !UPixelStreaming2PluginSettings::CVarWebRTCDisableReceiveVideo.GetValueOnAnyThread();

				// Check if the user has selected only H.264 on a AMD gpu and disable receiving video.
				// WebRTC does not support using SendRecv if the encoding and decoding do not support the same codec.
				// AMD GPUs currently have decoding disabled so WebRTC fails to create SDP codecs with SendRecv.
				// TODO (Eden.Harris) RTCP-8039: This workaround won't be needed once H.264 decoding is enabled with AMD GPUs.
				if (IsRHIDeviceAMD() && (bNegotiateCodecs || (!bNegotiateCodecs && SelectedCodec == EVideoCodec::H264)))
				{
					if (bReceiveBrowserVideo)
					{
						bReceiveBrowserVideo = false;
						UE_LOGFMT(LogPixelStreaming2RTC, Warning, "AMD GPUs do not support receiving H.264 video.");
					}
				}

				if (bTransmitUEVideo || bReceiveBrowserVideo)
				{
					TArray<EpicRtcVideoEncodingConfig> VideoEncodingConfigs;
					// We need ensure the Rids have the same lifetime as the VideoEncodingConfigs
					// to ensure the contents don't get deleted before we can call AddVideoSource
					TArray<FUtf8String> Rids;

					int MaxFramerate = UPixelStreaming2PluginSettings::CVarWebRTCFps.GetValueOnAnyThread();

					// WebRTC cannot set the bitrate outside the first initial biterate set by the VideoEncodingConfig.
					// By setting a high value here, the real value can be set by SetConnectionRates which can fit within this range.
					// Without this, changing the max bitrate at runtime will be capped at the initial max bitrate.
					// SetConnectionRates below will set the real max bitrate.
					constexpr uint32 InitialMinBitrate = 1'000;
					constexpr uint32 InitialMaxBitrate = 1'000'000'000;

					TArray<FPixelStreaming2SimulcastLayer> SimulcastParams = UE::PixelStreaming2::GetSimulcastParameters();
					if (SimulcastParams.Num() > 1)
					{
						for (int i = 0; i < SimulcastParams.Num(); ++i)
						{
							const FPixelStreaming2SimulcastLayer& SpatialLayer = SimulcastParams[i];

							Rids.Add(FUtf8String("simulcast") + FUtf8String::FromInt(i));

							EpicRtcVideoEncodingConfig VideoEncodingConfig = {
								// clang-format off
							._rid = EpicRtcStringView{._ptr = (const char*)(*(Rids[i])), ._length = static_cast<uint64>(Rids[i].Len()) },
								// clang-format on
								._scaleResolutionDownBy = SpatialLayer.Scaling,
								._scalabilityMode = static_cast<EpicRtcVideoScalabilityMode>(GetEnumFromCVar<EScalabilityMode>(UPixelStreaming2PluginSettings::CVarEncoderScalabilityMode)), // HACK if the Enums become un-aligned
								._minBitrate = static_cast<uint32>(SpatialLayer.MinBitrate),
								._maxBitrate = static_cast<uint32>(SpatialLayer.MaxBitrate),
								._maxFrameRate = static_cast<uint8_t>(MaxFramerate)
							};

							VideoEncodingConfigs.Add(VideoEncodingConfig);
						}
					}
					else
					{
						// Default video config for P2P or if simulcast disabled
						EpicRtcVideoEncodingConfig VideoEncodingConfig = {
							// clang-format off
							// TODO (Migration): RTCP-7027 Maybe bug in EpicRtc? Setting an rid if there's only one config results in no video
							._rid = EpicRtcStringView{._ptr = nullptr, ._length = 0 },
							// clang-format on
							._scaleResolutionDownBy = 1.f,
							._scalabilityMode = static_cast<EpicRtcVideoScalabilityMode>(GetEnumFromCVar<EScalabilityMode>(UPixelStreaming2PluginSettings::CVarEncoderScalabilityMode)), // HACK if the Enums become un-aligned
							._minBitrate = InitialMinBitrate,
							._maxBitrate = InitialMaxBitrate,
							._maxFrameRate = static_cast<uint8_t>(MaxFramerate)
						};

						VideoEncodingConfigs.Add(VideoEncodingConfig);
					}

					EpicRtcVideoEncodingConfigSpan VideoEncodingConfigSpan = {
						._ptr = VideoEncodingConfigs.GetData(),
						._size = static_cast<uint64_t>(VideoEncodingConfigs.Num())
					};

					EpicRtcMediaSourceDirection VideoDirection;
					if (bTransmitUEVideo && bReceiveBrowserVideo)
					{
						VideoDirection = EpicRtcMediaSourceDirection::SendRecv;
					}
					else if (bTransmitUEVideo)
					{
						VideoDirection = EpicRtcMediaSourceDirection::SendOnly;
					}
					else if (bReceiveBrowserVideo)
					{
						VideoDirection = EpicRtcMediaSourceDirection::RecvOnly;
					}
					else
					{
						VideoDirection = EpicRtcMediaSourceDirection::RecvOnly;
					}

					FUtf8String		   VideoStreamID = GetVideoStreamID();
					EpicRtcVideoSource RtcVideoSource = {
						._streamId = ToEpicRtcStringView(VideoStreamID),
						._encodings = VideoEncodingConfigSpan,
						._direction = VideoDirection
					};

					Result = ConnectionInterface->AddVideoSource(RtcVideoSource);
					if (Result != EpicRtcErrorCode::Ok)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnSessionStateUpdate: Failed to add video source. AddVideoSource returned {0}", *ToString(Result));
						break;
					}
				}

				const bool bTransmitUEAudio = !UPixelStreaming2PluginSettings::CVarWebRTCDisableTransmitAudio.GetValueOnAnyThread();
				const bool bReceiveBrowserAudio = !UPixelStreaming2PluginSettings::CVarWebRTCDisableReceiveAudio.GetValueOnAnyThread();
				if (bTransmitUEAudio || bReceiveBrowserAudio)
				{
					EpicRtcMediaSourceDirection AudioDirection;
					if (bTransmitUEAudio && bReceiveBrowserAudio)
					{
						AudioDirection = EpicRtcMediaSourceDirection::SendRecv;
					}
					else if (bTransmitUEAudio)
					{
						AudioDirection = EpicRtcMediaSourceDirection::SendOnly;
					}
					else if (bReceiveBrowserAudio)
					{
						AudioDirection = EpicRtcMediaSourceDirection::RecvOnly;
					}
					else
					{
						AudioDirection = EpicRtcMediaSourceDirection::RecvOnly;
					}

					FUtf8String		   AudioStreamID = GetAudioStreamID();
					EpicRtcAudioSource RtcAudioSource = {
						._streamId = ToEpicRtcStringView(AudioStreamID),
						._bitrate = 510000,
						._channels = 2,
						._direction = AudioDirection
					};

					Result = ConnectionInterface->AddAudioSource(RtcAudioSource);
					if (Result != EpicRtcErrorCode::Ok)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnSessionStateUpdate: Failed to add audio source. AddAudioSource returned {0}", *ToString(Result));
						break;
					}
				}

				EpicRtcRoom->Join();
				break;
			}
			case EpicRtcSessionState::Disconnected: // Indicates session is disconnected from the signalling server.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionStateUpdate State=Disconnected");

				// If we were successfully connected previously then we should broadcast the delegates about disconnection
				bool bBroadcastDelegates = StreamState == EStreamState::Connected || StreamState == EStreamState::Disconnecting;

				// If the session unexpectedly disconnects (ie signalling server goes away), we should try and reconnect
				if (StreamState == EStreamState::Connected || StreamState == EStreamState::Connecting)
				{
					// Call stop streaming first
					StopStreaming();

					if (TRefCountPtr<EpicRtcSessionInterface> PinnedSession = EpicRtcSession; EpicRtcRoom && PinnedSession)
					{
						FUtf8String Utf8StreamerId(StreamerId);
						PinnedSession->RemoveRoom(ToEpicRtcStringView(Utf8StreamerId));
						EpicRtcRoom = nullptr;
					}

					ReconnectTimer->Start(AsShared());
				}

				// The session has been disconnected (either through a call to StopStreaming or by an error)
				// so remove it
				if (EpicRtcConference.IsValid())
				{
					FUtf8String Utf8StreamerId(StreamerId);
					EpicRtcConference->RemoveSession(ToEpicRtcStringView(Utf8StreamerId));
					EpicRtcSession = nullptr;
				}

				if (bBroadcastDelegates)
				{
					DoOnGameThread([StreamerId = this->StreamerId]() {
						if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
						{
							Delegates->OnDisconnectedFromSignallingServer.Broadcast(StreamerId);
							Delegates->OnDisconnectedFromSignallingServerNative.Broadcast(StreamerId);
						}
					});

					OnStreamingStopped().Broadcast(this);
				}

				// We are fully disconnect at this point so we update the state so we can StartStreaming again
				StreamState = EStreamState::Disconnected;

				break;
			}
			case EpicRtcSessionState::Failed: // Indicates session failed and is unusable.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionStateUpdate State=Failed");
				break;
			}
			case EpicRtcSessionState::Exiting: // Indicates session has terminated without a result as a response to the application exiting.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionStateUpdate State=Exiting");
				break;
			}
			default:
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnSessionStateUpdate An unhandled session state was encountered. This switch might be missing a case.");
				checkNoEntry(); // All cases should be handled
				break;
			}
		}
	}

	void FEpicRtcStreamer::OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionErrorUpdate does nothing");
	}

	void FEpicRtcStreamer::OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnSessionRoomsAvailableUpdate does nothing");
	}

	void FEpicRtcStreamer::OnRoomStateUpdate(const EpicRtcRoomState State)
	{
		switch (State)
		{
			case EpicRtcRoomState::New: // Indicates newly created EpicRtcRoomInterface.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomStateUpdate State=New");

				break;
			}
			case EpicRtcRoomState::Pending: // Indicates join of the local participant is in progress.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomStateUpdate State=Pending");

				break;
			}
			case EpicRtcRoomState::Joined: // Indicates local participant (this streamer) is joined.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomStateUpdate State=Joined");

				StreamState = EStreamState::Connected;
				ReconnectTimer->Reset();

				OnStreamingStarted().Broadcast(this);
				break;
			}
			case EpicRtcRoomState::Left: // Indicates local participant (this streamer) has left this EpicRtcRoomInterface. Room is not usable once in this state.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomStateUpdate State=Left");

				TRefCountPtr<EpicRtcSessionInterface> PinnedSession = EpicRtcSession;
				if (!PinnedSession)
				{
					break;
				}

				FUtf8String Utf8StreamerId(StreamerId);
				PinnedSession->RemoveRoom(ToEpicRtcStringView(Utf8StreamerId));
				EpicRtcRoom = nullptr;

				const EpicRtcErrorCode Result = PinnedSession->Disconnect(ToEpicRtcStringView("Streaming Session Removed"));
				if (Result == EpicRtcErrorCode::SessionDisconnected)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "Session disconnected cleanly.");
				}
				else if (Result != EpicRtcErrorCode::Ok)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to disconnect EpicRtcSession. Disconnect returned {0}", *ToString(Result));
				}

				break;
			}
			case EpicRtcRoomState::Failed: // Indicates EpicRtcRoomInterface failed and is unusable.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomStateUpdate State=Failed");

				break;
			}
			case EpicRtcRoomState::Exiting: // Indicates EpicRtcRoomInterface has terminated without a result as a response to the application exiting.
			{
				UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomStateUpdate State=Exiting");

				break;
			}
			default:
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomStateUpdate An unhandled room state was encountered. This switch might be missing a case.");
				checkNoEntry(); // All cases should be handled
				break;
			}
		}
	}

	FUtf8String FEpicRtcStreamer::GetAudioStreamID()
	{
		const bool bSyncVideoAndAudio = !UPixelStreaming2PluginSettings::CVarWebRTCDisableAudioSync.GetValueOnAnyThread();
		return bSyncVideoAndAudio ? "pixelstreaming_av_stream_id" : "pixelstreaming_audio_stream_id";
	}

	FUtf8String FEpicRtcStreamer::GetVideoStreamID()
	{
		const bool bSyncVideoAndAudio = !UPixelStreaming2PluginSettings::CVarWebRTCDisableAudioSync.GetValueOnAnyThread();
		return bSyncVideoAndAudio ? "pixelstreaming_av_stream_id" : "pixelstreaming_video_stream_id";
	}

	void FEpicRtcStreamer::OnRoomJoinedUpdate(EpicRtcParticipantInterface* ParticipantInterface)
	{
		const FString ParticipantId = ToString(ParticipantInterface->GetId());
		UE_LOGF(LogPixelStreaming2RTC, Log, "Player (%ls) joined", *ParticipantId);

		if (ParticipantId == StreamerId)
		{
			return;
		}

		DoOnGameThread([StreamerId = this->StreamerId, ParticipantId]() {
			if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
			{
				Delegates->OnNewConnection.Broadcast(StreamerId, ParticipantId);
				Delegates->OnNewConnectionNative.Broadcast(StreamerId, ParticipantId);
			}
		});

		TRefCountPtr<EpicRtcConnectionInterface> ConnectionInterface;
		EpicRtcErrorCode						 Result = EpicRtcRoom->GetConnection(ConnectionInterface.GetInitReference());
		if (Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to get connection interface. GetConnection returned {0}", *ToString(Result));
			return;
		}

		TSharedPtr<FPlayerContext>& Participant = Participants->FindOrAdd(ParticipantId);
		Participant = MakeShared<FPlayerContext>();
		Participant->ParticipantInterface = ParticipantInterface;
		Participant->StatsCollector = FRTCStatsCollector::Create(ParticipantId);

		FUtf8String Utf8ParticipantId = *ParticipantId;

		if (IsSFU(ParticipantId))
		{
			FString			  RecvLabel(TEXT("recv-datachannel"));
			FUtf8String		  Utf8RecvLabel = *RecvLabel;
			EpicRtcDataSource RecvDataSource = {
				._label = ToEpicRtcStringView(Utf8RecvLabel),
				._maxRetransmitTime = 0,
				._maxRetransmits = 0,
				._isOrdered = true,
				._protocol = EpicRtcDataSourceProtocol::Sctp,
				._negotiated = true,
				._transportChannelId = 1
			};
			Result = ConnectionInterface->AddDataSource(ToEpicRtcStringView(Utf8ParticipantId), RecvDataSource);
			if (Result != EpicRtcErrorCode::Ok)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to add recv data source. AddDataSource returned {0}", *ToString(Result));
				return;
			}

			FString			  SendLabel(TEXT("send-datachannel"));
			FUtf8String		  Utf8SendLabel = *SendLabel;
			EpicRtcDataSource SendDataSource = {
				._label = ToEpicRtcStringView(Utf8SendLabel),
				._maxRetransmitTime = 0,
				._maxRetransmits = 0,
				._isOrdered = true,
				._protocol = EpicRtcDataSourceProtocol::Sctp,
				._negotiated = true,
				._transportChannelId = 0
			};
			Result = ConnectionInterface->AddDataSource(ToEpicRtcStringView(Utf8ParticipantId), SendDataSource);
			if (Result != EpicRtcErrorCode::Ok)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to add send data source. AddDataSource returned {0}", *ToString(Result));
				return;
			}
		}
		else
		{
			EpicRtcDataSource DataSource = {
				._label = ParticipantInterface->GetId(),
				._maxRetransmitTime = 0,
				._maxRetransmits = 0,
				._isOrdered = true,
				._protocol = EpicRtcDataSourceProtocol::Sctp
			};
			Result = ConnectionInterface->AddDataSource(ToEpicRtcStringView(Utf8ParticipantId), DataSource);
			if (Result != EpicRtcErrorCode::Ok)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to add data source. AddDataSource returned {0}", *ToString(Result));
				return;
			}
		}

		Result = ConnectionInterface->SetConnectionRates(ToEpicRtcStringView(Utf8ParticipantId), GetBitrates(true /* bIncludeStartBitrate */));
		if (Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to set connection rates. SetConnectionRates returned {0}", *ToString(Result));
			return;
		}

		TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>> PreferredCodecs;
		for (auto& Codec : UPixelStreaming2PluginSettings::GetCodecPreferences())
		{
			TRefCountPtr<EpicRtcVideoCodecInfoInterface> CodecInfo(new FEpicRtcVideoCodecInfo(ToEpicRtcVideoCodec(Codec)));
			PreferredCodecs.Add(CodecInfo);
		}

		TRefCountPtr<EpicRtcVideoCodecInfoArrayInterface> VideoCodecInfoArray(new FVideoCodecInfoArray(PreferredCodecs));
		Result = ConnectionInterface->SetPreferredVideoCodecs(ToEpicRtcStringView(Utf8ParticipantId), VideoCodecInfoArray);
		if (Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to set preferred video codecs. SetPreferredVideoCodecs returned {0}", *ToString(Result));
			return;
		}


		Result = ConnectionInterface->StartNegotiation(ToEpicRtcStringView(Utf8ParticipantId));
		if (Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FEpicRtcStreamer::OnRoomJoinedUpdate: Failed to start negotiation. StartNegotiation returned {0}", *ToString(Result));
			return;
		}

		// NOTE (william.belcher) [13/10/25]: As EpicRtc stores a single local track for the streamer that is then mapped to all participants internal to EpicRtc,
		// preventing calls to OnVideoTrackUpdate and OnAudioTrackUpdate, we will manually call OnVideoTrackOpen and OnAudioTrackOpen for the local tracks to ensure 
		// our delegates retain functionality
		OnVideoTrackOpen(ParticipantId, false);
		OnAudioTrackOpen(ParticipantId, false);
	}

	void FEpicRtcStreamer::OnRoomLeftUpdate(const EpicRtcStringView Participant)
	{
		FString ParticipantId = ToString(Participant);
		UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnRoomLeftUpdate(Participant [%ls] left the room.)", *ParticipantId);

		if (ParticipantId == StreamerId)
		{
			return;
		}

		// Remove the player
		DeletePlayerSession(ParticipantId);
	}

	void FEpicRtcStreamer::OnAudioTrackUpdate(EpicRtcParticipantInterface* ParticipantInterface, EpicRtcAudioTrackInterface* AudioTrack)
	{
		const FString ParticipantId = ToString(ParticipantInterface->GetId());
		const FString AudioTrackId = ToString(AudioTrack->GetId());
		const bool	  bIsRemote = static_cast<bool>(AudioTrack->IsRemote());

		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnAudioTrackUpdate(Participant [{0}], AudioTrack [{1}], IsRemote [{2}])", ParticipantId, AudioTrackId, bIsRemote);

		AudioTrackPlayerIdMap.Add(reinterpret_cast<uintptr_t>(AudioTrack), ParticipantId);
		if (ParticipantId == StreamerId)
		{
			// A user may hit a race condition if they call StopStreaming (which resets AudioCapturer) before this callback 
			// has a chance to  be executed by the task thread. We opt to create an AudioCapturer here instead of early
			// returning to ensure AudioSource is always valid for the streamer.
			if (!AudioCapturer)
			{
				AudioCapturer = FAudioCapturer::Create<FEpicRtcAudioCapturer>();
			}
			// NOTE: we don't broadcast the audio track open for a streamer
			AudioSource = FEpicRtcAudioSource::Create(AudioTrack, AudioCapturer);
		}
		else
		{
			OnAudioTrackOpen(ParticipantId, bIsRemote);
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(ParticipantId); Participant.IsValid())
			{
				if (bIsRemote)
				{
					Participant->AudioSink = FEpicRtcAudioSink::Create(AudioTrack);
				}
				else
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnAudioTrackUpdate: Remote participants shouldn't have local tracks!");
				}
			}
		}
	}

	void FEpicRtcStreamer::OnVideoTrackUpdate(EpicRtcParticipantInterface* ParticipantInterface, EpicRtcVideoTrackInterface* VideoTrack)
	{
		const FString ParticipantId = ToString(ParticipantInterface->GetId());
		const FString VideoTrackId = ToString(VideoTrack->GetId());
		const bool	  bIsRemote = static_cast<bool>(VideoTrack->IsRemote());

		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnVideoTrackUpdate(Participant [{0}], VideoTrack [{1}], IsRemote[{2}])", ParticipantId, VideoTrackId, bIsRemote);

		VideoTrackPlayerIdMap.Add(reinterpret_cast<uintptr_t>(VideoTrack), ParticipantId);
		if (ParticipantId == StreamerId)
		{
			// NOTE: we don't broadcast the video track open for a streamer
			VideoSource = FEpicRtcVideoSource::Create(VideoTrack, VideoCapturer, VideoSourceGroup);
		}
		else
		{
			OnVideoTrackOpen(ParticipantId, bIsRemote);
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(ParticipantId); Participant.IsValid())
			{
				if (bIsRemote)
				{
					// We received a remote track. We should now create a sink to handle receiving the frames.
					// NOTE: We pass in nullptr as the track because if we store the track on the sink, EpicRtc will be unable to destroy it
					// and webrtc will try to flush remaining frames during session removal.
					Participant->VideoSink = FEpicRtcVideoSink::Create(nullptr);
				}
				else
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnVideoTrackUpdate: Remote participants shouldn't have local tracks!");
				}
			}
		}
	}

	void FEpicRtcStreamer::OnDataTrackUpdate(EpicRtcParticipantInterface* ParticipantInterface, EpicRtcDataTrackInterface* DataTrack)
	{
		FString ParticipantId = ToString(ParticipantInterface->GetId());
		FString DataTrackId = ToString(DataTrack->GetId());
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnDataTrackUpdate(Participant [{0}], DataTrack [{1}])", ParticipantId, DataTrackId);

		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(ParticipantId); Participant.IsValid())
		{
			if (!Participant->DataTrack)
			{
				if (!InputHandler)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Cannot create participant data track when input handler is invalid!");
					return;
				}
				Participant->DataTrack = FEpicRtcDataTrack::Create(DataTrack, InputHandler->GetFromStreamerProtocol());
			}
			else
			{
				Participant->DataTrack->SetSendTrack(DataTrack);
			}
		}
	}

	[[nodiscard]] EpicRtcSdpInterface* FEpicRtcStreamer::OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		FString ParticipantId = ToString(Participant->GetId());
		FString SdpType = TEXT("");
		switch (Sdp->GetType())
		{
			case EpicRtcSdpType::Offer:
				SdpType = TEXT("Offer");
				break;
			case EpicRtcSdpType::Answer:
				SdpType = TEXT("Answer");
				break;
		}

		UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnLocalSdpUpdate(Participant [%ls], Type [%ls])", *ParticipantId, *SdpType);

		return nullptr;
	}

	[[nodiscard]] EpicRtcSdpInterface* FEpicRtcStreamer::OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		FString ParticipantId = ToString(Participant->GetId());
		FString SdpType = TEXT("");
		switch (Sdp->GetType())
		{
			case EpicRtcSdpType::Offer:
				SdpType = TEXT("Offer");
				break;
			case EpicRtcSdpType::Answer:
				SdpType = TEXT("Answer");
				break;
		}

		UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnRemoteSdpUpdate(Participant [%ls], Type [%ls])", *ParticipantId, *SdpType);

		return nullptr;
	}

	void FEpicRtcStreamer::OnRoomErrorUpdate(const EpicRtcErrorCode Error)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnRoomErrorUpdate does nothing");
	}

	void FEpicRtcStreamer::OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted)
	{
		FString PlayerId;
		bool	bFoundPlayer = FindPlayerFromTrack(AudioTrack, PlayerId);
		FString AudioTrackId = ToString(AudioTrack->GetId());
		UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnAudioTrackMuted(AudioTrack [%ls], bIsMuted[%ls], PlayerId[%ls])", *AudioTrackId, bIsMuted ? TEXT("True") : TEXT("False"), *PlayerId);

		if (!bFoundPlayer)
		{
			UE_LOGF(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnAudioTrackMuted(Failed to find a player for audio track [%ls])", *AudioTrackId);
			return;
		}

		if (PlayerId == StreamerId)
		{
			if (AudioSource)
			{
				AudioSource->SetMuted(static_cast<bool>(bIsMuted));
			}
			return;
		}
		else
		{
			if (!AudioTrack->IsRemote())
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnAudioTrackMuted: Remote participants shouldn't have local tracks!");
				return;
			}

			TSharedPtr<FEpicRtcAudioSink> AudioSink;
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
			{
				AudioSink = Participant->AudioSink;
			}

			if (AudioSink)
			{
				AudioSink->SetMuted(static_cast<bool>(bIsMuted));
			}
		}
	}

	void FEpicRtcStreamer::OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame)
	{
		FString PlayerId;
		bool	bFoundPlayer = FindPlayerFromTrack(AudioTrack, PlayerId);
		FString AudioTrackId = ToString(AudioTrack->GetId());

		if (!bFoundPlayer)
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnAudioTrackFrame(Failed to find a player for audio track [%ls])", *AudioTrackId);
			return;
		}

		TSharedPtr<FEpicRtcAudioSink> AudioSink;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			AudioSink = Participant->AudioSink;
		}

		if (AudioSink)
		{
			AudioSink->OnAudioData(Frame._data, Frame._length, Frame._format._numChannels, Frame._format._sampleRate);
		}
	}

	void FEpicRtcStreamer::OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State)
	{
		FString		  PlayerId;
		const bool	  bFoundPlayer = FindPlayerFromTrack(AudioTrack, PlayerId);
		const bool	  bIsRemote = static_cast<bool>(AudioTrack->IsRemote());
		const FString AudioTrackId = ToString(AudioTrack->GetId());

		if (!bFoundPlayer)
		{
			// Not finding a player is expected as OnAudioTrackState will happen for OnAudioTrackUpdate during track addition
			UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnAudioTrackState(Cannot to find a player for audio track [{0}])", AudioTrackId);
			return;
		}

		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnAudioTrackState(AudioTrack=[{0}], Player=[{1}], State=[{2}])", AudioTrackId, PlayerId, ToString(State));
		if (State == EpicRtcTrackState::Stopped)
		{
			AudioTrackPlayerIdMap.Remove(reinterpret_cast<uintptr_t>(AudioTrack));
			OnAudioTrackClosed(PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted)
	{
		FString PlayerId;
		bool	bFoundPlayer = FindPlayerFromTrack(VideoTrack, PlayerId);
		FString VideoTrackId = ToString(VideoTrack->GetId());
		UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnVideoTrackMuted(VideoTrack [%ls], bIsMuted[%ls], PlayerId[%ls])", *VideoTrackId, bIsMuted ? TEXT("True") : TEXT("False"), *PlayerId);

		if (!bFoundPlayer)
		{
			UE_LOGF(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnVideoTrackMuted(Failed to find a player for video track [%ls])", *VideoTrackId);
			return;
		}

		if (PlayerId == StreamerId)
		{
			if (VideoSource)
			{
				VideoSource->SetMuted(static_cast<bool>(bIsMuted));
			}
			return;
		}
		else
		{
			if (!VideoTrack->IsRemote())
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnVideoTrackMuted: Remote participants shouldn't have local tracks!");
				return;
			}

			TSharedPtr<FEpicRtcVideoSink> VideoSink;
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
			{
				VideoSink = Participant->VideoSink;
			}

			if (VideoSink)
			{
				VideoSink->SetMuted(static_cast<bool>(bIsMuted));
			}
		}
	}

	void FEpicRtcStreamer::OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame)
	{
		FString PlayerId;
		bool	bFoundPlayer = FindPlayerFromTrack(VideoTrack, PlayerId);
		FString VideoTrackId = ToString(VideoTrack->GetId());

		if (!bFoundPlayer)
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnVideoTrackFrame(Failed to find a player for video track [%ls])", *VideoTrackId);
			return;
		}

		TSharedPtr<FEpicRtcVideoSink> VideoSink;
		if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
		{
			VideoSink = Participant->VideoSink;
		}

		if (VideoSink)
		{
			VideoSink->OnEpicRtcFrame(Frame);
		}
	}

	void FEpicRtcStreamer::OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State)
	{
		FString		  PlayerId;
		const bool	  bFoundPlayer = FindPlayerFromTrack(VideoTrack, PlayerId);
		const bool	  bIsRemote = static_cast<bool>(VideoTrack->IsRemote());
		const FString VideoTrackId = ToString(VideoTrack->GetId());

		if (!bFoundPlayer)
		{
			// Not finding a player is expected as OnVideoTrackState will happen for OnVideoTrackUpdate during track addition
			UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FEpicRtcStreamer::OnVideoTrackState(Cannot to find a player for video track [{0}])", VideoTrackId);
			return;
		}

		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnVideoTrackState(VideoTrack=[{0}], Player=[{1}], State=[{2}])", VideoTrackId, PlayerId, ToString(State));
		if (State == EpicRtcTrackState::Stopped)
		{
			VideoTrackPlayerIdMap.Remove(reinterpret_cast<uintptr_t>(VideoTrack));
			OnVideoTrackClosed(PlayerId, bIsRemote);
		}
	}

	void FEpicRtcStreamer::OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame)
	{
	}

	EpicRtcBool FEpicRtcStreamer::Enabled() const
	{
		return true;
	}

	void FEpicRtcStreamer::OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State)
	{
		FString PlayerId;
		bool	bFoundPlayer = FindPlayerFromTrack(DataTrack, PlayerId);
		FString DataTrackId = ToString(DataTrack->GetId());

		if (!bFoundPlayer)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FEpicRtcStreamer::OnDataTrackState(Failed to find a player for data track [{0}])", DataTrackId);
			return;
		}

		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnDataTrackState(DataTrack=[{0}], Player=[{1}], State=[{2}])", DataTrackId, PlayerId, ToString(State));
		if (State == EpicRtcTrackState::Active)
		{
			OnDataTrackOpen(PlayerId);
		}
		else if (State == EpicRtcTrackState::Stopped)
		{
			OnDataTrackClosed(PlayerId);
		}
	}

	void FEpicRtcStreamer::OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame)
	{
		const uint8_t* Data = DataFrame._data;
		uint32_t	   DataSize = DataFrame._size;

		if (DataSize == 0)
		{
			return;
		}

		FString DataTrackId = ToString(DataTrack->GetId());
		FString PlayerId;

		uint8 Type = Data[0];
		if (!InputHandler)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Cannot handle data track message when input handler is invalid!");
			return;
		}
		TSharedPtr<IPixelStreaming2DataProtocol> ToStreamerProtocol = InputHandler->GetToStreamerProtocol();
		if (Type == ToStreamerProtocol->Find(EPixelStreaming2ToStreamerMessage::Multiplexed)->GetID())
		{
			// skip type
			Data++;
			DataSize--;
			PlayerId = ReadString(Data, DataSize);

			if (DataSize == 0)
			{
				return;
			}
			Type = Data[0];
			UE_LOGF(LogPixelStreaming2RTC, VeryVerbose, "FEpicRtcStreamer::OnDataTrackMessage(Received multiplexed message of type [%d] with PlayerId [%ls])", Type, *PlayerId);
		}
		else if (Type == ToStreamerProtocol->Find(EPixelStreaming2ToStreamerMessage::ChannelRelayStatus)->GetID())
		{
			HandleRelayStatusMessage(Data, DataSize, DataTrack);
			return;
		}
		else if (!FindPlayerFromTrack(DataTrack, PlayerId))
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "FEpicRtcStreamer::OnDataTrackMessage(Failed to find a player for data track [%ls])", *DataTrackId);
			return;
		}

		if (Type == ToStreamerProtocol->Find(EPixelStreaming2ToStreamerMessage::LatencyTest)->GetID())
		{
			SendLatencyReport(PlayerId);
		}
		else if (Type == ToStreamerProtocol->Find(EPixelStreaming2ToStreamerMessage::RequestInitialSettings)->GetID())
		{
			SendInitialSettings(PlayerId);
		}
		else if (Type == ToStreamerProtocol->Find(EPixelStreaming2ToStreamerMessage::IFrameRequest)->GetID())
		{
			ForceKeyFrame();
		}
		else if (Type == ToStreamerProtocol->Find(EPixelStreaming2ToStreamerMessage::TestEcho)->GetID())
		{
			TSharedPtr<FEpicRtcDataTrack> ParticipantDataTrack;
			if (TSharedPtr<FPlayerContext> Participant = Participants->FindRef(PlayerId); Participant.IsValid())
			{
				ParticipantDataTrack = Participant->DataTrack;
			}

			if (ParticipantDataTrack)
			{
				const size_t  DescriptorSize = (DataSize - 1) / sizeof(TCHAR);
				const TCHAR*  DescPtr = reinterpret_cast<const TCHAR*>(Data + 1);
				const FString Message(DescriptorSize, DescPtr);
				ParticipantDataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::TestEcho, Message);
			}
		}
		else if (!IsEngineExitRequested())
		{
			// If we are in "Host" mode and the current peer is not the host, then discard this input.
			if (GetEnumFromCVar<EInputControllerMode>(UPixelStreaming2PluginSettings::CVarInputController) == EInputControllerMode::Host && InputControllingId != PlayerId)
			{
				return;
			}

			TArray<uint8> MessageData(Data, DataSize);
			if (InputHandler)
			{
				InputHandler->OnMessage(PlayerId, MessageData);
			}
		}
	}

	void FEpicRtcStreamer::OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error)
	{
	}

	FString FRTCStreamerFactory::GetStreamType()
	{
		return RTC_STREAM_TYPE;
	}

	FRTCStreamerFactory::FRTCStreamerFactory(TRefCountPtr<EpicRtcConferenceInterface> Conference)
		: EpicRtcConference(MoveTemp(Conference))
	{
	}

	TSharedPtr<IPixelStreaming2Streamer> FRTCStreamerFactory::CreateNewStreamer(const FString& StreamerId)
	{
		TSharedPtr<IPixelStreaming2Streamer> NewStreamer = MakeShared<FEpicRtcStreamer>(StreamerId, EpicRtcConference);
		NewStreamer->SetInputHandler(FRTCInputHandler::Create());
		NewStreamer->SetVideoProducer(FVideoProducerBackBuffer::Create());

		return NewStreamer;
	}
} // namespace UE::PixelStreaming2
