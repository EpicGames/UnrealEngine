// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcAudioCapturer.h"
#include "EpicRtcAudioTrackObserver.h"
#include "EpicRtcAudioTrackObserverFactory.h"
#include "EpicRtcDataTrackObserver.h"
#include "EpicRtcDataTrackObserverFactory.h"
#include "EpicRtcVideoTrackObserver.h"
#include "EpicRtcVideoTrackObserverFactory.h"
#include "EpicRtcRoomObserver.h"
#include "EpicRtcSessionObserver.h"
#include "FreezeFrame.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "PlayerContext.h"
#include "SharedTickableTasks.h"
#include "StreamerReconnectTimer.h"
#include "ThreadSafeMap.h"
#include "UtilsString.h"
#include "VideoSourceGroup.h"


#include "epic_rtc/core/conference.h"
#include "epic_rtc/core/stats.h"

class IPixelStreaming2RTCModule;

namespace UE::PixelStreaming2
{
	static const FString INVALID_PLAYER_ID = TEXT("Invalid Player Id");
	static const FString RTC_STREAM_TYPE = TEXT("DefaultRtc");

	class FEpicRtcStreamer :
		public IPixelStreaming2Streamer,
		public TSharedFromThis<FEpicRtcStreamer>,
		public IPixelStreaming2SessionObserver,
		public IPixelStreaming2RoomObserver,
		public IPixelStreaming2AudioTrackObserver,
		public IPixelStreaming2DataTrackObserver,
		public IPixelStreaming2VideoTrackObserver
	{
	public:
		FEpicRtcStreamer(const FString& StreamerId, TRefCountPtr<EpicRtcConferenceInterface> Conference);
		virtual ~FEpicRtcStreamer();

		virtual void  Initialize() override;
		virtual void  SetStreamFPS(int32 InFramesPerSecond) override;
		virtual int32 GetStreamFPS() override;
		virtual void  SetCoupleFramerate(bool bCouple) override;

		virtual void									SetVideoProducer(TSharedPtr<IPixelStreaming2VideoProducer> Input) override;
		virtual TWeakPtr<IPixelStreaming2VideoProducer> GetVideoProducer() override;

		virtual void AddAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) override;
		virtual void RemoveAudioProducer(TSharedPtr<IPixelStreaming2AudioProducer> AudioProducer) override;
		virtual TArray<TWeakPtr<IPixelStreaming2AudioProducer>> GetAudioProducers() override;

		virtual void	SetConnectionURL(const FString& InConnectionURL) override;
		virtual FString GetConnectionURL() override;

		virtual FString GetId() override { return StreamerId; };
		virtual bool	IsConnected() override { return StreamState == EStreamState::Connected; }
		virtual void	StartStreaming() override;
		virtual void	StopStreaming() override;
		virtual bool	IsStreaming() const override { return StreamState == EStreamState::Connected || StreamState == EStreamState::Connecting; }

		virtual FPreConnectionEvent&	OnPreConnection() override;
		virtual FStreamingStartedEvent& OnStreamingStarted() override;
		virtual FStreamingStoppedEvent& OnStreamingStopped() override;

		virtual void ForceKeyFrame() override;

		virtual void FreezeStream(UTexture2D* Texture) override;
		virtual void UnfreezeStream() override;

		virtual void			SendAllPlayersMessage(FString MessageType, const FString& Descriptor) override;
		virtual void			SendPlayerMessage(FString PlayerId, FString MessageType, const FString& Descriptor) override;
		virtual void			SendFileData(const TArray64<uint8>& ByteData, FString& MimeType, FString& FileExtension) override;
		virtual void			KickPlayer(FString PlayerId) override;
		virtual TArray<FString> GetConnectedPlayers() override;

		virtual TWeakPtr<IPixelStreaming2InputHandler> GetInputHandler() override { return InputHandler; }
		virtual void SetInputHandler(TSharedPtr<IPixelStreaming2InputHandler> Handler) override;

		virtual TWeakPtr<IPixelStreaming2AudioSink> GetPeerAudioSink(FString PlayerId) override;
		virtual TWeakPtr<IPixelStreaming2AudioSink> GetUnlistenedAudioSink() override;
		virtual TWeakPtr<IPixelStreaming2VideoSink> GetPeerVideoSink(FString PlayerId) override;
		virtual TWeakPtr<IPixelStreaming2VideoSink> GetUnwatchedVideoSink() override;

		virtual void SetConfigOption(const FName& OptionName, const FString& Value) override;
		virtual bool GetConfigOption(const FName& OptionName, FString& OutValue) override;

		virtual void PlayerRequestsBitrate(FString PlayerId, int MinBitrate, int MaxBitrate) override;

		virtual void RefreshStreamBitrate() override;

		void ForEachPlayer(const TFunction<void(FString, TSharedPtr<FPlayerContext>)>& Func);

	private:
		// own methods
		void OnProtocolUpdated();
		void ConsumeStats(FString PlayerId, FName StatName, float StatValue);
		void DeletePlayerSession(FString PlayerId);
		void DeleteAllPlayerSessions();
		void SendInitialSettings(FString PlayerId) const;
		void SendProtocol(FString PlayerId) const;
		void SendPeerControllerMessages(FString PlayerId) const;
		void SendLatencyReport(FString PlayerId) const;
		void HandleRelayStatusMessage(const uint8_t* Data, uint32_t Size, EpicRtcDataTrackInterface* DataTrack);
		void TriggerMouseLeave(FString InStreamerId);
		FUtf8String GetAudioStreamID();
		FUtf8String GetVideoStreamID();

		// Function called on track events. Handles broadcasting of delegates and any other logic not related to the track itself (ie sending protocol)
		void OnDataTrackOpen(FString PlayerId);
		void OnDataTrackClosed(FString PlayerId);
		void OnAudioTrackOpen(FString PlayerId, bool bIsRemote);
		void OnAudioTrackClosed(FString PlayerId, bool bIsRemote);
		void OnVideoTrackOpen(FString PlayerId, bool bIsRemote);
		void OnVideoTrackClosed(FString PlayerId, bool bIsRemote);
		void CloseAudioTrack(const TSharedPtr<FPlayerContext>& Participant, FString PlayerId, bool bIsRemote);
		void CloseDataTrack(const TSharedPtr<FPlayerContext>& Participant, FString PlayerId);
		void CloseVideoTrack(const TSharedPtr<FPlayerContext>& Participant, FString PlayerId, bool bIsRemote);

		void		   OnStatsReady(const FString& PlayerId, const EpicRtcConnectionStats& ConnectionStats);
		void		   OnFrameCapturerCreated();
		void		   OnUIInteraction(FMemoryReader Ar);
		void		   OnSendMessage(FString MessageName, FMemoryReader Ar);
		EpicRtcBitrate GetBitrates(bool bIncludeStartBitrate) const;

		template <typename T>
		bool FindPlayerFromTrack(T Track, FString& OutPlayerId)
		{
			OutPlayerId = "";
			if constexpr (std::is_same_v<T, EpicRtcVideoTrackInterface*>)
			{
				FString* FoundPlayerId = VideoTrackPlayerIdMap.Find(reinterpret_cast<uintptr_t>(Track));
				if (FoundPlayerId)
				{
					OutPlayerId = *FoundPlayerId;
				}
			}
			else if constexpr (std::is_same_v<T, EpicRtcAudioTrackInterface*>)
			{
				FString* FoundPlayerId = AudioTrackPlayerIdMap.Find(reinterpret_cast<uintptr_t>(Track));
				if (FoundPlayerId)
				{
					OutPlayerId = *FoundPlayerId;
				}
			}
			else if constexpr (std::is_same_v<T, EpicRtcDataTrackInterface*>)
			{
				FString DataTrackId = ToString(Track->GetId());
				Participants->ApplyUntil([DataTrackId, &OutPlayerId](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
					if (Participant->DataTrack)
					{
						FString TrackId = ToString(Participant->DataTrack->GetTrackId());
						if (TrackId == DataTrackId)
						{
							OutPlayerId = PlayerId;
							return true;
						}
					}
					return false;
				});
			}
			else
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "FindPlayerFromTrack using unknown type. Ensure you're using a track interface pointer");
			}

			return !OutPlayerId.IsEmpty();
		}

	private:
		FString StreamerId;
		FString CurrentSignallingServerURL;

		TSharedPtr<IPixelStreaming2InputHandler> InputHandler;

		TSharedPtr<TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>> Participants;

		FString InputControllingId = INVALID_PLAYER_ID;

		enum class EStreamState : uint8
		{
			Disconnected = 0,
			Connecting,
			Connected,
			Disconnecting,
		};

		EStreamState StreamState = EStreamState::Disconnected;

		FPreConnectionEvent	   StreamingPreConnectionEvent;
		FStreamingStartedEvent StreamingStartedEvent;
		FStreamingStoppedEvent StreamingStoppedEvent;

		TSharedPtr<FEpicRtcAudioCapturer> AudioCapturer;
		TSharedPtr<FEpicRtcVideoCapturer> VideoCapturer;
		TSharedPtr<FVideoSourceGroup>	  VideoSourceGroup;
		TSharedPtr<FFreezeFrame>		  FreezeFrame;

		TSharedPtr<FEpicRtcVideoSource> VideoSource;
		TSharedPtr<FEpicRtcAudioSource> AudioSource;

		TMap<FName, FString>														  ConfigOptions;
		TMap<uintptr_t /* EpicRtcAudioTrackInterface* */, FString /*participant id*/> AudioTrackPlayerIdMap;
		TMap<uintptr_t /* EpicRtcVideoTrackInterface* */, FString /*participant id*/> VideoTrackPlayerIdMap;

		TSharedPtr<FStreamerReconnectTimer> ReconnectTimer;
		TSharedPtr<FSharedTickableTasks>	TickableTasks;

		FCriticalSection CustomAudioProducersCS;
		TArray<TSharedPtr<IPixelStreaming2AudioProducer>> CustomAudioProducers;

	public:
		// Begin IPixelStreaming2SessionObserver
		virtual void OnSessionStateUpdate(const EpicRtcSessionState StateUpdate) override;
		virtual void OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate) override;
		virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) override;
		// End IPixelStreaming2SessionObserver

		// Begin IPixelStreaming2RoomObserver
		virtual void							   OnRoomStateUpdate(const EpicRtcRoomState State) override;
		virtual void							   OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) override;
		virtual void							   OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) override;
		virtual void							   OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) override;
		virtual void							   OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) override;
		virtual void							   OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) override;
		[[nodiscard]] virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		[[nodiscard]] virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		virtual void							   OnRoomErrorUpdate(const EpicRtcErrorCode Error) override;
		// End IPixelStreaming2RoomObserver

		// Begin IPixelStreaming2AudioTrackObserver
		virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) override;
		virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) override;
		virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) override;
		// End IPixelStreaming2AudioTrackObserver

		// Begin IPixelStreaming2VideoTrackObserver
		virtual void		OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) override;
		virtual void		OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) override;
		virtual void		OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) override;
		virtual void		OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) override;
		virtual EpicRtcBool Enabled() const override;
		// End IPixelStreaming2VideoTrackObserver

		// Begin IPixelStreaming2DataTrackObserver
		virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) override;
		virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) override;
		virtual void OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error) override;
		// End IPixelStreaming2DataTrackObserver

	private:
		// Begin EpicRtc Classes
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
		TRefCountPtr<EpicRtcSessionInterface>	 EpicRtcSession;
		TRefCountPtr<EpicRtcRoomInterface>		 EpicRtcRoom;
		// End EpicRtc Classes

		// Begin EpicRtc Observers
		TRefCountPtr<FEpicRtcSessionObserver>			SessionObserver;
		TRefCountPtr<FEpicRtcRoomObserver>				RoomObserver;
		TRefCountPtr<FEpicRtcAudioTrackObserverFactory> AudioTrackObserverFactory;
		TRefCountPtr<FEpicRtcVideoTrackObserverFactory> VideoTrackObserverFactory;
		TRefCountPtr<FEpicRtcDataTrackObserverFactory>	DataTrackObserverFactory;
		// End EpicRtc Observers
	};

	class FRTCStreamerFactory : public IPixelStreaming2StreamerFactory
	{
	public:
		FRTCStreamerFactory(TRefCountPtr<EpicRtcConferenceInterface> Conference);
		virtual ~FRTCStreamerFactory() = default;

		virtual FString								 GetStreamType() override;
		virtual TSharedPtr<IPixelStreaming2Streamer> CreateNewStreamer(const FString& StreamerId) override;

	private:
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
	};
} // namespace UE::PixelStreaming2
