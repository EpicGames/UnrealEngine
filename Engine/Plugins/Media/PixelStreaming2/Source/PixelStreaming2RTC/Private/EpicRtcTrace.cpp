// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcTrace.h"

#include "Trace/Trace.inl"

UE_TRACE_CHANNEL_DEFINE(EpicRtcChannel);