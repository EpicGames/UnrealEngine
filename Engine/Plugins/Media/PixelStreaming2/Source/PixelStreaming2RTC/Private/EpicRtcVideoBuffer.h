// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureBufferFormat.h"

#include "epic_rtc/core/video/video_buffer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

namespace UE::PixelStreaming2
{
	/**
	 * The base VideoBuffer class that PixelStreaming2 uses. It partially implements
	 * the EpicRtcVideoBufferInterface by always returning EpicRtcPixelFormat::Native to
	 * signal to EpicRtc that it doesn't need to wrap this buffer in ANOTHER buffer before
	 * passing to WebRTC. It provides the GetBufferFormat virtual method for use cases
	 * to check the format of the buffer ie RHI or I420 etc
	 */
	class FEpicRtcVideoBuffer : public EpicRtcVideoBufferInterface
	{
	public:
		FEpicRtcVideoBuffer() = default;

		virtual ~FEpicRtcVideoBuffer() = default;

	public:
		virtual int32 GetBufferFormat()
		{
			return PixelCaptureBufferFormat::FORMAT_UNKNOWN;
		}

		// We always need to return Native so EpicRtc doesn't try to wrap
		// these buffers in software wrappers
		virtual EpicRtcPixelFormat GetFormat() override
		{
			return EpicRtcPixelFormat::Native;
		}

		virtual EpicRtcVideoBufferInterface* ToI420() override
		{
			return nullptr;
		}

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};
} // namespace UE::PixelStreaming2