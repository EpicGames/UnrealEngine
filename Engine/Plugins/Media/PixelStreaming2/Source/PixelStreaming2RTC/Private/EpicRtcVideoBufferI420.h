// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcVideoBuffer.h"
#include "PixelCaptureBufferI420.h"

namespace UE::PixelStreaming2
{
	class FEpicRtcVideoBufferI420 : public FEpicRtcVideoBuffer
	{
	public:
		FEpicRtcVideoBufferI420(TSharedPtr<FPixelCaptureBufferI420> Buffer)
			: Buffer(Buffer)
		{
		}

		virtual ~FEpicRtcVideoBufferI420() = default;

	public:
		// Begin FEpicRtcVideoBuffer
		virtual int32 GetBufferFormat() override
		{
			return PixelCaptureBufferFormat::FORMAT_I420;
		}
		// End FEpicRtcVideoBuffer

		// Begin EpicRtcVideoBufferInterface
#if PLATFORM_ANDROID
		// Android uses the internal EpicRtc buffers so we report I420 format here
		virtual EpicRtcPixelFormat GetFormat() override
		{
			return EpicRtcPixelFormat::I420;
		}
#endif

		virtual void* GetData() override
		{
			return Buffer->GetMutableData();
		}

		virtual int GetWidth() override
		{
			return Buffer->GetWidth();
		}

		virtual int GetHeight() override
		{
			return Buffer->GetHeight();
		}
		// End EpicRtcVideoBufferInterface

		TSharedPtr<FPixelCaptureBufferI420> GetBuffer()
		{
			return Buffer;
		}

	private:
		TSharedPtr<FPixelCaptureBufferI420> Buffer;
	};
} // namespace UE::PixelStreaming2