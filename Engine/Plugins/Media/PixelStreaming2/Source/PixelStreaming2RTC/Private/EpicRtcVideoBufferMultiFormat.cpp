// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoBufferMultiFormat.h"

#include "EpicRtcVideoBufferI420.h"
#include "PixelCaptureOutputFrameI420.h"
#include "Stats.h"

namespace UE::PixelStreaming2
{
	FEpicRtcVideoBufferMultiFormatBase::FEpicRtcVideoBufferMultiFormatBase(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer)
		: FrameCapturer(InFrameCapturer)
	{
	}

	FEpicRtcVideoBufferMultiFormatLayered::FEpicRtcVideoBufferMultiFormatLayered(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, FIntPoint SourceResolution)
		: FEpicRtcVideoBufferMultiFormatBase(InFrameCapturer)
		, SourceResolution(SourceResolution)
	{
	}

	int FEpicRtcVideoBufferMultiFormatLayered::GetWidth()
	{
		return SourceResolution.X;
	}

	int FEpicRtcVideoBufferMultiFormatLayered::GetHeight()
	{
		return SourceResolution.Y;
	}

	EpicRtcVideoBufferInterface* FEpicRtcVideoBufferMultiFormatLayered::ToI420() 
	{
		TRefCountPtr<FEpicRtcVideoBufferMultiFormat> MultiFormatBuffer = GetLayer(SourceResolution);

		IPixelCaptureOutputFrame* RequestedFrame = MultiFormatBuffer->RequestFormat(PixelCaptureBufferFormat::FORMAT_I420);
		if (!RequestedFrame)
		{
			return nullptr;
		}

		FPixelCaptureOutputFrameI420& I420Frame = StaticCast<FPixelCaptureOutputFrameI420&>(*RequestedFrame);
		if (I420Frame.GetI420Buffer() == nullptr)
		{
			return nullptr;
		}
		
		I420Frame.Metadata.UseCount++;
		FStats::Get()->AddFrameTimingStats(I420Frame.Metadata, { I420Frame.GetWidth(), I420Frame.GetHeight() });
		return new FEpicRtcVideoBufferI420(I420Frame.GetI420Buffer());  
	}

	TRefCountPtr<FEpicRtcVideoBufferMultiFormat> FEpicRtcVideoBufferMultiFormatLayered::GetLayer(FIntPoint TargetResolution) const
	{
		return new FEpicRtcVideoBufferMultiFormat(FrameCapturer, TargetResolution);
	}

	FEpicRtcVideoBufferMultiFormat::FEpicRtcVideoBufferMultiFormat(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, FIntPoint TargetResolution)
		: FEpicRtcVideoBufferMultiFormatBase(InFrameCapturer)
		, Resolution(TargetResolution)
	{
	}

	int FEpicRtcVideoBufferMultiFormat::GetWidth()
	{
		return Resolution.X;
	}

	int FEpicRtcVideoBufferMultiFormat::GetHeight()
	{
		return Resolution.Y;
	}

	IPixelCaptureOutputFrame* FEpicRtcVideoBufferMultiFormat::RequestFormat(int32 Format) const
	{
		// ensure this frame buffer will always refer to the same frame
		if (TSharedPtr<IPixelCaptureOutputFrame>* CachedFrame = CachedFormat.Find(Format))
		{
			return CachedFrame->Get();
		}

		if (!FrameCapturer)
		{
			return nullptr;
		}
		TSharedPtr<IPixelCaptureOutputFrame> Frame = FrameCapturer->RequestFormat(Format, Resolution);
		if (!Frame) 
		{
			return nullptr;
		}
		
		CachedFormat.Add(Format, Frame);
		return Frame.Get();
	}
} // namespace UE::PixelStreaming2