// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelCaptureCapturerMultiFormat.h"
#include "Video/Resources/VideoResourceRHI.h"

#include "epic_rtc/core/video/video_buffer.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

namespace UE::PixelStreaming2
{
	class FEpicRtcVideoBufferMultiFormat;

	/**
	 * Base class for our multi format buffers.
	 */
	class FEpicRtcVideoBufferMultiFormatBase : public EpicRtcVideoBufferInterface
	{
	public:
		FEpicRtcVideoBufferMultiFormatBase(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer);
		virtual ~FEpicRtcVideoBufferMultiFormatBase() = default;

	public:
		// Begin EpicRtcVideoBufferInterface
		virtual void* GetData() override
		{
			return nullptr;
		}

		virtual EpicRtcVideoBufferInterface* ToI420() override
		{
			return nullptr;
		}

		virtual EpicRtcPixelFormat GetFormat() override
		{
			return EpicRtcPixelFormat::Native;
		}
		// End EpicRtcVideoBufferInterface

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	protected:
		TSharedPtr<FPixelCaptureCapturerMultiFormat> FrameCapturer;
	};

	/**
	 * A multi layered, multi format frame buffer for our encoder.
	 */
	class FEpicRtcVideoBufferMultiFormatLayered : public FEpicRtcVideoBufferMultiFormatBase
	{
	public:
		FEpicRtcVideoBufferMultiFormatLayered(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, FIntPoint SourceResolution);
		virtual ~FEpicRtcVideoBufferMultiFormatLayered() = default;

		virtual int GetWidth() override;
		virtual int GetHeight() override;
		virtual EpicRtcVideoBufferInterface* ToI420() override;


		TRefCountPtr<FEpicRtcVideoBufferMultiFormat> GetLayer(FIntPoint Resolution) const;

	private:
		FIntPoint SourceResolution;
	};

	/**
	 * A single layer, multi format frame buffer.
	 */
	class FEpicRtcVideoBufferMultiFormat : public FEpicRtcVideoBufferMultiFormatBase
	{
	public:
		FEpicRtcVideoBufferMultiFormat(TSharedPtr<FPixelCaptureCapturerMultiFormat> InFrameCapturer, FIntPoint InResolution);
		virtual ~FEpicRtcVideoBufferMultiFormat() = default;

		virtual int GetWidth() override;
		virtual int GetHeight() override;

		IPixelCaptureOutputFrame* RequestFormat(int32 Format) const;

	private:
		FIntPoint Resolution;
		// we want the frame buffer to always refer to the same frame. so the first request for a format
		// will fill this cache.
		mutable TMap<int32, TSharedPtr<IPixelCaptureOutputFrame>> CachedFormat;
	};
} // namespace UE::PixelStreaming2