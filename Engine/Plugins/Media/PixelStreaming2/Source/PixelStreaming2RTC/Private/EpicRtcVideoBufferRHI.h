// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcVideoBuffer.h"
#include "Video/Resources/VideoResourceRHI.h"

namespace UE::PixelStreaming2
{
	class FEpicRtcVideoBufferRHI : public FEpicRtcVideoBuffer
	{
	public:
		FEpicRtcVideoBufferRHI(TSharedPtr<FVideoResourceRHI> VideoResourceRHI)
			: VideoResourceRHI(VideoResourceRHI)
		{
			VideoResourceRHI->SetUsing(true);
		}

		virtual ~FEpicRtcVideoBufferRHI()
		{
			VideoResourceRHI->SetUsing(false);
		}

	public:
		// Begin FEpicRtcVideoBuffer
		virtual int32 GetBufferFormat() override
		{
			return PixelCaptureBufferFormat::FORMAT_RHI;
		}
		// End FEpicRtcVideoBuffer

		// Begin EpicRtcVideoBufferInterface
		virtual void* GetData() override
		{
			return nullptr;
		}

		virtual int GetWidth() override
		{
			return VideoResourceRHI->GetDescriptor().Width;
		}

		virtual int GetHeight() override
		{
			return VideoResourceRHI->GetDescriptor().Height;
		}
		// End EpicRtcVideoBufferInterface

		TSharedPtr<FVideoResourceRHI> GetVideoResource()
		{
			return VideoResourceRHI;
		}

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TSharedPtr<FVideoResourceRHI> VideoResourceRHI;
	};
} // namespace UE::PixelStreaming2