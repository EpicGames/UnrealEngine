// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoCapturer.h"

#include "EpicRtcVideoBufferMultiFormat.h"
#include "PixelStreaming2PluginSettings.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FEpicRtcVideoCapturer> FEpicRtcVideoCapturer::Create(bool bInIsSRGB)
	{
		TSharedPtr<FEpicRtcVideoCapturer> VideoCapturer = TSharedPtr<FEpicRtcVideoCapturer>(new FEpicRtcVideoCapturer(bInIsSRGB));

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnSimulcastEnabledChanged.AddSP(VideoCapturer.ToSharedRef(), &FEpicRtcVideoCapturer::OnSimulcastEnabledChanged);
			Delegates->OnCaptureUseFenceChanged.AddSP(VideoCapturer.ToSharedRef(), &FEpicRtcVideoCapturer::OnCaptureUseFenceChanged);
			Delegates->OnUseMediaCaptureChanged.AddSP(VideoCapturer.ToSharedRef(), &FEpicRtcVideoCapturer::OnUseMediaCaptureChanged);
		}
		return VideoCapturer;
	}

	FEpicRtcVideoCapturer::FEpicRtcVideoCapturer(bool bIsSRGB)
		: FVideoCapturer(bIsSRGB)
	{
		CreateFrameCapturer();
	}

	TRefCountPtr<EpicRtcVideoBufferInterface> FEpicRtcVideoCapturer::GetFrameBuffer()
	{
		return new FEpicRtcVideoBufferMultiFormatLayered(FrameCapturer, { LastFrameWidth, LastFrameHeight });
	}
} // namespace UE::PixelStreaming2