// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoDecoder.h"

#include "ColorConversion.h"
#include "EpicRtcVideoBufferI420.h"
#include "EpicRtcVideoBufferRHI.h"
#include "PixelStreaming2Trace.h"
#include "UtilsString.h"
#include "UtilsCodecs.h"

#include "epic_rtc/core/video/video_codec_info.h"

namespace UE::PixelStreaming2
{
	template <std::derived_from<FVideoResource> TVideoResource>
	TEpicRtcVideoDecoder<TVideoResource>::TEpicRtcVideoDecoder(EpicRtcVideoCodecInfoInterface* CodecInfo)
		: CodecInfo(CodecInfo)
		, FrameCount(0)
	{
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcStringView TEpicRtcVideoDecoder<TVideoResource>::GetName() const
	{
		static FUtf8String Name("PixelStreamingVideoDecoderHardware");
		return ToEpicRtcStringView(Name);
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcVideoDecoderConfig TEpicRtcVideoDecoder<TVideoResource>::GetConfig() const
	{
		return DecoderConfig;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcMediaResult TEpicRtcVideoDecoder<TVideoResource>::SetConfig(const EpicRtcVideoDecoderConfig& VideoDecoderConfig)
	{
		DecoderConfig = VideoDecoderConfig;

		switch (CodecInfo->GetCodec())
		{
			case EpicRtcVideoCodec::H264:
			{
				TUniquePtr<FVideoDecoderConfigH264> VideoConfig = MakeUnique<FVideoDecoderConfigH264>();
				InitialVideoConfig = MoveTemp(VideoConfig);
				break;
			}
			case EpicRtcVideoCodec::AV1:
			{
				TUniquePtr<FVideoDecoderConfigAV1> VideoConfig = MakeUnique<FVideoDecoderConfigAV1>();
				InitialVideoConfig = MoveTemp(VideoConfig);
				break;
			}
			case EpicRtcVideoCodec::VP8:
			{
				TUniquePtr<FVideoDecoderConfigVP8> VideoConfig = MakeUnique<FVideoDecoderConfigVP8>();
				VideoConfig->NumberOfCores = DecoderConfig._numberOfCores;
				InitialVideoConfig = MoveTemp(VideoConfig);
				break;
			}
			case EpicRtcVideoCodec::VP9:
			{
				TUniquePtr<FVideoDecoderConfigVP9> VideoConfig = MakeUnique<FVideoDecoderConfigVP9>();
				VideoConfig->NumberOfCores = DecoderConfig._numberOfCores;
				InitialVideoConfig = MoveTemp(VideoConfig);
				break;
			}
			default:
				// We don't support hardware decoders for other codecs
				checkNoEntry();
		}

		return EpicRtcMediaResult::Ok;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcVideoDecoderInfo TEpicRtcVideoDecoder<TVideoResource>::GetInfo() 
	{
		EpicRtcVideoDecoderInfo VideoDecoderInfo;

		if constexpr (std::is_same_v<TVideoResource, FVideoResourceRHI>)
		{
			VideoDecoderInfo._isHardwareAccelerated = true;
		}
		else if constexpr (std::is_same_v<TVideoResource, FVideoResourceCPU>)
		{
			VideoDecoderInfo._isHardwareAccelerated = false;
		}

		return VideoDecoderInfo;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcMediaResult TEpicRtcVideoDecoder<TVideoResource>::Decode(const EpicRtcEncodedVideoFrame& Frame)
	{
		// Capture the Callback to ensure it is not released in a different thread.
		TRefCountPtr<EpicRtcVideoDecoderCallbackInterface> CallbackDecoded(VideoDecoderCallback);
		if (!CallbackDecoded)
		{
			return EpicRtcMediaResult::Uninitialized;
		}

		if (!Decoder && !LateInitDecoder())
		{
			return EpicRtcMediaResult::Error;
		}

		const double TimestampDecodeStart = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming2 Decoding Video", PixelStreaming2Channel);

		FAVResult Result = Decoder->SendPacket(FVideoPacket(
			// HACK GetData returns a `const uint8_t*` so requires const cast. If VideoPacket accepted TSharedPtr<cosnt uint8> const&, this would not be needed.
			MakeShareable<uint8>(const_cast<uint8*>(Frame._buffer->GetData()), TFakeDeleter<uint8>()),
			Frame._buffer->GetSize(),
			Frame._timestampRtp,
			FrameCount++,
			Frame._qp,
			Frame._frameType == EpicRtcVideoFrameType::I));

		if (Result.IsNotError())
		{
			EpicRtcVideoFrame DecodedFrame = {
				._id = FrameCount,
				._timestampUs = Frame._timestampUs,
				._timestampRtp = Frame._timestampRtp,
				._isBackedByWebRtc = false,
				._buffer = nullptr
			};

			FAVResult DecodeResult = Decoder->ReceiveFrame(DecoderResource);
			if (DecodeResult.IsSuccess())
			{
				
				if constexpr (std::is_same_v<TVideoResource, FVideoResourceRHI>)
				{
					DecodedFrame._buffer = new FEpicRtcVideoBufferRHI(DecoderResource);
				}
				else if constexpr (std::is_same_v<TVideoResource, FVideoResourceCPU>)
				{
					const uint32 FrameWidth = DecoderResource->GetWidth();
					const uint32 FrameHeight = DecoderResource->GetHeight();
					
					TSharedPtr<FPixelCaptureBufferI420> I420Buffer = MakeShared<FPixelCaptureBufferI420>(FrameWidth, FrameHeight);
					
					const uint32 DataSizeY = FrameWidth * FrameHeight;
					const uint32 DataSizeUV = ((FrameWidth + 1) / 2) * ((FrameHeight + 1) / 2);
					
					CopyI420(
							 DecoderResource->GetRaw().Get(), FrameWidth,
							 DecoderResource->GetRaw().Get() + DataSizeY, (FrameWidth + 1) / 2,
							 DecoderResource->GetRaw().Get() + DataSizeY + DataSizeUV, (FrameWidth + 1) / 2,
							 I420Buffer->GetMutableDataY(), I420Buffer->GetStrideY(),
							 I420Buffer->GetMutableDataU(), I420Buffer->GetStrideUV(),
							 I420Buffer->GetMutableDataV(), I420Buffer->GetStrideUV(),
							 FrameWidth, FrameHeight);
					
					DecodedFrame._buffer = new FEpicRtcVideoBufferI420(I420Buffer);
				}
				else
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "VideoResource isn't a compatible type! Expected either a FVideoResourceRHI or FVideoResourceCPU. Received: {0}", UE_STRINGIZE(TVideoResource));
					return EpicRtcMediaResult::Error;
				}	
				
				check(DecodedFrame._buffer->GetWidth() != 0 && DecodedFrame._buffer->GetHeight() != 0);
				
				CallbackDecoded->Decoded(DecodedFrame, FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64()) - TimestampDecodeStart, Frame._qp);
				
				return EpicRtcMediaResult::Ok;
			}
			else 
			{
				UE_LOGF(LogPixelStreaming2RTC, Warning, "TEpicRtcVideoDecoder::Decode FAILED TO RECEIVE FRAME");				
				return EpicRtcMediaResult::Error;
			}
		}
		else
		{
			UE_LOGF(LogPixelStreaming2RTC, Warning, "TEpicRtcVideoDecoder::Decode DECODE FAILED");
			return EpicRtcMediaResult::OkRequestKeyframe;
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoDecoder<TVideoResource>::RegisterCallback(EpicRtcVideoDecoderCallbackInterface* Callback)
	{
		VideoDecoderCallback = Callback;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoDecoder<TVideoResource>::Reset()
	{
		Decoder.Reset();
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	bool TEpicRtcVideoDecoder<TVideoResource>::LateInitDecoder()
	{
		switch (CodecInfo->GetCodec())
		{
			case EpicRtcVideoCodec::H264:
			{
				FVideoDecoderConfigH264& VideoConfig = *StaticCast<FVideoDecoderConfigH264*>(InitialVideoConfig.Get());
				Decoder = FVideoDecoder::CreateChecked<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
				if (!Decoder)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "PixelStreamingVideoDecoder: Unable to get or create H264 Decoder");
					return false;
				}
				break;
			}
			case EpicRtcVideoCodec::AV1:
			{
				FVideoDecoderConfigAV1& VideoConfig = *StaticCast<FVideoDecoderConfigAV1*>(InitialVideoConfig.Get());
				Decoder = FVideoDecoder::CreateChecked<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
				if (!Decoder)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "PixelStreamingVideoDecoder: Unable to get or create AV1 Decoder");
					return false;
				}
				break;
			}
			case EpicRtcVideoCodec::VP8:
			{
				FVideoDecoderConfigVP8& VideoConfig = *StaticCast<FVideoDecoderConfigVP8*>(InitialVideoConfig.Get());
				Decoder = FVideoDecoder::CreateChecked<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
				if (!Decoder)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "PixelStreamingVideoDecoder: Unable to get or create VP8 Decoder");
					return false;
				}
				break;
			}
			case EpicRtcVideoCodec::VP9:
			{
				FVideoDecoderConfigVP9& VideoConfig = *StaticCast<FVideoDecoderConfigVP9*>(InitialVideoConfig.Get());
				Decoder = FVideoDecoder::CreateChecked<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
				if (!Decoder)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "PixelStreamingVideoDecoder: Unable to get or create VP9 Decoder");
					return false;
				}
				break;
			}
			default:
				// We don't support decoders for other codecs
				checkNoEntry();
				return false;
		}

		return true;
	}

	// Explicit specialisation
	template class TEpicRtcVideoDecoder<FVideoResourceRHI>;
	template class TEpicRtcVideoDecoder<FVideoResourceCPU>;
} // namespace UE::PixelStreaming2
