// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2Stats.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP8.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"
#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Resources/VideoResourceRHI.h"

#include "epic_rtc/core/video/video_decoder.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

#include <atomic>
#include <type_traits>

namespace UE::PixelStreaming2
{
	template <std::derived_from<FVideoResource> TVideoResource>
	class TEpicRtcVideoDecoder : public EpicRtcVideoDecoderInterface
	{
	public:
		TEpicRtcVideoDecoder(EpicRtcVideoCodecInfoInterface* CodecInfo);

		// Begin EpicRtcVideoDecoderInterface
		[[nodiscard]] virtual EpicRtcStringView GetName() const override;
		virtual EpicRtcVideoDecoderConfig		GetConfig() const override;
		virtual EpicRtcMediaResult				SetConfig(const EpicRtcVideoDecoderConfig& VideoDecoderConfig) override;
		virtual EpicRtcVideoDecoderInfo			GetInfo() override;
		virtual EpicRtcMediaResult				Decode(const EpicRtcEncodedVideoFrame& Frame) override;
		virtual void							RegisterCallback(EpicRtcVideoDecoderCallbackInterface* Callback) override;
		virtual void							Reset() override;
		// End EpicRtcVideoDecoderInterface
	private:
		TSharedPtr<TVideoDecoder<TVideoResource>>		   Decoder;
		TUniquePtr<FVideoDecoderConfig>					   InitialVideoConfig;
		EpicRtcVideoDecoderConfig						   DecoderConfig;
		TRefCountPtr<EpicRtcVideoDecoderCallbackInterface> VideoDecoderCallback;
		TRefCountPtr<EpicRtcVideoCodecInfoInterface>	   CodecInfo;
		uint16_t										   FrameCount;

		// clang-format off
		using DecoderResourceType = typename std::conditional<std::is_same_v<TVideoResource, FVideoResourceRHI>, FResolvableVideoResourceRHI, 
								    typename std::conditional<std::is_same_v<TVideoResource, FVideoResourceCPU>, FResolvableVideoResourceCPU, 
								    // We only support RHI and CPU resources so if we create one that isn't this type it will error (on purpose)
								    void>::type>::type; 

		DecoderResourceType DecoderResource;
		// clang-format on

	private:
		bool LateInitDecoder();

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};
} // namespace UE::PixelStreaming2
