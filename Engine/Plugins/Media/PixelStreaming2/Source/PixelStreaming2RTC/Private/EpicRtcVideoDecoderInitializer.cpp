// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoDecoderInitializer.h"

#include "EpicRtcVideoDecoder.h"
#include "UtilsCoder.h"
#include "PixelStreaming2PluginSettings.h"
#include "UtilsString.h"
#include "UtilsCodecs.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP8.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"

namespace
{
	template <typename TConfig>
	EpicRtcVideoDecoderInterface* CreateDecoder(EpicRtcVideoCodecInfoInterface* CodecInfo)
	{
		EpicRtcVideoDecoderInterface* Decoder = nullptr;

		if (UE::PixelStreaming2::IsHardwareDecoderSupported<TConfig>())
		{
			Decoder = new UE::PixelStreaming2::TEpicRtcVideoDecoder<FVideoResourceRHI>(CodecInfo);
		}
		else if (UE::PixelStreaming2::IsSoftwareDecoderSupported<TConfig>())
		{
			Decoder = new UE::PixelStreaming2::TEpicRtcVideoDecoder<FVideoResourceCPU>(CodecInfo);
		}

		return Decoder;
	}

	// Explicit initialisation. We don't want this util namespace public
	template EpicRtcVideoDecoderInterface* CreateDecoder<FVideoDecoderConfigH264>(EpicRtcVideoCodecInfoInterface* CodecInfo);
	template EpicRtcVideoDecoderInterface* CreateDecoder<FVideoDecoderConfigAV1>(EpicRtcVideoCodecInfoInterface* CodecInfo);
	template EpicRtcVideoDecoderInterface* CreateDecoder<FVideoDecoderConfigVP8>(EpicRtcVideoCodecInfoInterface* CodecInfo);
	template EpicRtcVideoDecoderInterface* CreateDecoder<FVideoDecoderConfigVP9>(EpicRtcVideoCodecInfoInterface* CodecInfo);
} // namespace

namespace UE::PixelStreaming2
{
	void FEpicRtcVideoDecoderInitializer::CreateDecoder(EpicRtcVideoCodecInfoInterface* CodecInfo, EpicRtcVideoDecoderInterface** OutDecoder)
	{
		EpicRtcVideoDecoderInterface* Decoder = nullptr;

		switch (CodecInfo->GetCodec())
		{
			case EpicRtcVideoCodec::H264:
				Decoder = ::CreateDecoder<FVideoDecoderConfigH264>(CodecInfo);
				break;
			case EpicRtcVideoCodec::AV1:
				Decoder = ::CreateDecoder<FVideoDecoderConfigAV1>(CodecInfo);
				break;
			case EpicRtcVideoCodec::VP8:
				Decoder = ::CreateDecoder<FVideoDecoderConfigVP8>(CodecInfo);
				break;
			case EpicRtcVideoCodec::VP9:
				Decoder = ::CreateDecoder<FVideoDecoderConfigVP9>(CodecInfo);
				break;
			default:
				// Every codec should be accounted for
				checkNoEntry();
		}
		if (!Decoder)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create decoder!");
			return;
		}

		// Because the ptr was created with new, we need to call AddRef ourself (ms spec compliant)
		Decoder->AddRef();

		*OutDecoder = Decoder;
	}

	EpicRtcStringView FEpicRtcVideoDecoderInitializer::GetName()
	{
		static FUtf8String Name("PixelStreamingVideoDecoder");
		return UE::PixelStreaming2::ToEpicRtcStringView(Name);
	}

	// We want this method to return all the formats we have decoders for but the selected codecs formats should be first in the list.
	// There is some nuance to this though, we cannot simply return just the selected codec. The reason for this because when we receive
	// video from another pixel streaming source, for some reason WebRTC will query the decoder factory on the receiving end and if it
	// doesnt support the video we are receiving then transport_cc is not enabled which leads to very low bitrate streams.
	EpicRtcVideoCodecInfoArrayInterface* FEpicRtcVideoDecoderInitializer::GetSupportedCodecs()
	{
		const TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> SupportedCodecMap = CreateSupportedDecoderMap();

		const EVideoCodec SelectedCodec = UE::PixelStreaming2::GetEnumFromString<EVideoCodec>(GSelectedCodec);
		const bool		  bNegotiateCodecs = UPixelStreaming2PluginSettings::CVarWebRTCNegotiateCodecs.GetValueOnAnyThread();

		// Build a list of supported codecs
		TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>> SupportedCodecs;

		// Todo: Check if all HW decoder sessions are used up.

		// If we are not negotiating codecs simply return just the one codec that is selected in UE
		if (!bNegotiateCodecs)
		{
			if (SupportedCodecMap.Contains(SelectedCodec))
			{
				SupportedCodecs.Append(SupportedCodecMap[SelectedCodec]);
			}
			else
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Selected codec was not a supported codec.");
			}
		}
		else
		{
			for (auto& Codec : UPixelStreaming2PluginSettings::GetCodecPreferences())
			{
				if (SupportedCodecMap.Contains(Codec))
				{
					SupportedCodecs.Append(SupportedCodecMap[Codec]);
				}
			}
		}

		return new FVideoCodecInfoArray(SupportedCodecs);
	}

	TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> FEpicRtcVideoDecoderInitializer::CreateSupportedDecoderMap()
	{
		TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> Codecs;
		for (auto& Codec : SupportedVideoCodecs)
		{
			Codecs.Add(Codec);
		}

		if (UE::PixelStreaming2::IsDecoderSupported<FVideoDecoderConfigVP8>())
		{
			Codecs[EVideoCodec::VP8].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::VP8,
				nullptr,
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
		}

		if (UE::PixelStreaming2::IsDecoderSupported<FVideoDecoderConfigVP9>())
		{
			using namespace UE::AVCodecCore::VP9;
			Codecs[EVideoCodec::VP9].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::VP9,
				UE::PixelStreaming2::CreateVP9Format(EProfile::Profile0),
				new FEpicRtcScalabilityModeArray(UE::PixelStreaming2::AllScalabilityModes)
			)));

			/**
			 * Only advertise profile 0 until EpicRtc provides us with a way to extract the negotiated profile in the encoder config
			 * 
			 * Codecs[EVideoCodec::VP9].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
			 *	EpicRtcVideoCodec::VP9,
			 *	UE::PixelStreaming2::CreateVP9Format(EProfile::Profile2),
			 *	new FEpicRtcScalabilityModeArray(AllScalabilityModes))));
			 */
		}

		if (UE::PixelStreaming2::IsDecoderSupported<FVideoDecoderConfigH264>())
		{
			using namespace UE::AVCodecCore::H264;
			Codecs[EVideoCodec::H264].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::H264,
				UE::PixelStreaming2::CreateH264Format(EH264Profile::ConstrainedBaseline, EH264Level::Level_3_1),
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
			Codecs[EVideoCodec::H264].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::H264,
				UE::PixelStreaming2::CreateH264Format(EH264Profile::Baseline, EH264Level::Level_3_1),
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
		}

		if (UE::PixelStreaming2::IsDecoderSupported<FVideoDecoderConfigAV1>())
		{
			Codecs[EVideoCodec::AV1].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::AV1,
				nullptr,
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
		}

		return Codecs;
	}

} // namespace UE::PixelStreaming2