// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoEncoder.h"

#include "EpicRtcVideoBufferI420.h"
#include "EpicRtcVideoBufferMultiFormat.h"
#include "EpicRtcVideoBufferRHI.h"
#include "EpicRtcContainers.h"
#include "Logging.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2Trace.h"
#include "Stats.h"
#include "UtilsString.h"
#include "UtilsCodecs.h"

#include "epic_rtc/core/video/video_buffer.h"
#include "epic_rtc/core/video/video_encoder_callback.h"

#include <type_traits>

namespace UE::PixelStreaming2
{
	template <std::derived_from<FVideoResource> TVideoResource>
	TEpicRtcVideoEncoder<TVideoResource>::TEpicRtcVideoEncoder(EpicRtcVideoCodecInfoInterface* CodecInfo)
		: CodecInfo(CodecInfo)
		, PreferredPixelFormats(new FEpicRtcPixelFormatArray({ EpicRtcPixelFormat::Native }))
		, ResolutionBitrateLimits(new FEpicRtcVideoResolutionBitrateLimitsArray({}))
	{
		EncoderConfig._simulcastStreams = nullptr;
		EncoderConfig._spatialLayers = nullptr;

		if (UPixelStreaming2PluginSettings::CVarEncoderDebugDumpFrame.GetValueOnAnyThread())
		{
			CreateDumpFile();
		}

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			DelegateHandle = Delegates->OnEncoderDebugDumpFrameChanged.AddRaw(this, &TEpicRtcVideoEncoder<TVideoResource>::OnEncoderDebugDumpFrameChanged);
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	TEpicRtcVideoEncoder<TVideoResource>::~TEpicRtcVideoEncoder()
	{
		if (EncoderConfig._simulcastStreams)
		{
			EncoderConfig._simulcastStreams->Release();
		}

		if (EncoderConfig._spatialLayers)
		{
			EncoderConfig._spatialLayers->Release();
		}

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnEncoderDebugDumpFrameChanged.Remove(DelegateHandle);
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcStringView TEpicRtcVideoEncoder<TVideoResource>::GetName() const
	{
		static FUtf8String Utf8ImplementationString = [this]() {
			FString EncoderLocation = TEXT("Unknown");
			if constexpr (std::is_same_v<TVideoResource, FVideoResourceRHI>)
			{
				EncoderLocation = TEXT("GPU");
			}
			else if constexpr (std::is_same_v<TVideoResource, FVideoResourceCPU>)
			{
				EncoderLocation = TEXT("CPU");
			}

			FString ImplementationString = FString::Printf(TEXT("PixelStreamingVideoEncoder(%s)(%s)"), *ToString(CodecInfo->GetCodec()), *EncoderLocation);
			return FUtf8String(ImplementationString);
		}();

		return ToEpicRtcStringView(Utf8ImplementationString);
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcVideoEncoderConfig TEpicRtcVideoEncoder<TVideoResource>::GetConfig() const
	{
		return EncoderConfig;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	TSharedPtr<TVideoEncoder<TVideoResource>> TEpicRtcVideoEncoder<TVideoResource>::CreateEncoder(const EpicRtcVideoEncoderConfig& Config, const EpicRtcVideoCodec Codec)
	{
		switch (Codec)
		{
			case EpicRtcVideoCodec::H264:
			{
				FVideoEncoderConfigH264 VideoConfig;
				SetInitialSettings(Config, VideoConfig);
				VideoConfig.Profile = GetEnumFromCVar<EH264Profile>(UPixelStreaming2PluginSettings::CVarEncoderH264Profile);
				VideoConfig.RepeatSPSPPS = true;
				VideoConfig.IntraRefreshPeriodFrames = 0;
				VideoConfig.IntraRefreshCountFrames = 0;
				VideoConfig.KeyframeInterval = UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval.GetValueOnAnyThread();
				// The WebRTC spec can only guarantee that the Baseline profile is supported. Therefore we use Baseline, but enable these extra
				// features to improve bitrate usage
				VideoConfig.AdaptiveTransformMode = EH264AdaptiveTransformMode::Enable;
				VideoConfig.EntropyCodingMode = EH264EntropyCodingMode::CABAC;

				return FVideoEncoder::Create<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
			}
			case EpicRtcVideoCodec::AV1:
			{
				FVideoEncoderConfigAV1 VideoConfig;
				SetInitialSettings(Config, VideoConfig);
				VideoConfig.RepeatSeqHdr = true;
				VideoConfig.IntraRefreshPeriodFrames = 0;
				VideoConfig.IntraRefreshCountFrames = 0;
				VideoConfig.KeyframeInterval = UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval.GetValueOnAnyThread();

				return FVideoEncoder::Create<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
			}
			case EpicRtcVideoCodec::VP8:
			{
				FVideoEncoderConfigVP8 VideoConfig;
				SetInitialSettings(Config, VideoConfig);

				return FVideoEncoder::Create<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
			}
			case EpicRtcVideoCodec::VP9:
			{
				FVideoEncoderConfigVP9 VideoConfig;
				SetInitialSettings(Config, VideoConfig);
				VideoConfig.ScalabilityMode = GetEnumFromCVar<EScalabilityMode>(UPixelStreaming2PluginSettings::CVarEncoderScalabilityMode);
				VideoConfig.NumberOfCores = Config._numberOfCores;
				VideoConfig.bDenoisingOn = static_cast<bool>(Config._isDenoisingOn);
				VideoConfig.bAdaptiveQpMode = static_cast<bool>(Config._isAdaptiveQpMode);
				// TODO RTCP-7994 (Eden.Harris) bAutomaticResizeOn can result in sporadic frame corruption.
				VideoConfig.bAutomaticResizeOn = false; // static_cast<bool>(EncoderConfig._isAutomaticResizeOn);
				VideoConfig.bFlexibleMode = static_cast<bool>(Config._isFlexibleMode);
				VideoConfig.InterLayerPrediction = static_cast<UE::AVCodecCore::VP9::EInterLayerPrediction>(Config._interLayerPred);
				for (size_t i = 0; i < Config._spatialLayers->Size(); i++)
				{
					EpicRtcSpatialLayer SpatialLayer = Config._spatialLayers->Get()[i];
					VideoConfig.SpatialLayers[i] = FSpatialLayer{
						.Width = static_cast<uint32>(SpatialLayer._resolution._width),
						.Height = static_cast<uint32>(SpatialLayer._resolution._height),
						.Framerate = SpatialLayer._maxFramerate,
						.NumberOfTemporalLayers = SpatialLayer._numberOfTemporalLayers,
						.MaxBitrate = static_cast<int32>(SpatialLayer._maxBitrate),
						.TargetBitrate = static_cast<int32>(SpatialLayer._targetBitrate),
						.MinBitrate = static_cast<int32>(SpatialLayer._minBitrate),
						.MaxQP = static_cast<int32>(SpatialLayer._qpMax),
						.bActive = static_cast<bool>(SpatialLayer._active)
					};
				}

				return FVideoEncoder::Create<TVideoResource>(FAVDevice::GetHardwareDevice(), VideoConfig);
			}
			default:
				// We don't support encoders for other codecs
				checkNoEntry();
				return nullptr;
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::SetInitialSettings(const EpicRtcVideoEncoderConfig& CodecSettings, FVideoEncoderConfig& VideoConfig)
	{
		VideoConfig.Preset = GetEnumFromCVar<EAVPreset>(UPixelStreaming2PluginSettings::CVarEncoderQualityPreset);
		VideoConfig.LatencyMode = GetEnumFromCVar<EAVLatencyMode>(UPixelStreaming2PluginSettings::CVarEncoderLatencyMode);
		VideoConfig.Width = CodecSettings._width;
		VideoConfig.Height = CodecSettings._height;
		VideoConfig.TargetFramerate = CodecSettings._maxFramerate;
		VideoConfig.TargetBitrate = CodecSettings._startBitrate;
		VideoConfig.MaxBitrate = CodecSettings._maxBitrate;
		VideoConfig.MinQuality = UPixelStreaming2PluginSettings::CVarEncoderMinQuality.GetValueOnAnyThread();
		VideoConfig.MaxQuality = UPixelStreaming2PluginSettings::CVarEncoderMaxQuality.GetValueOnAnyThread();
		VideoConfig.RateControlMode = ERateControlMode::CBR;
		VideoConfig.bFillData = false;
		VideoConfig.KeyframeInterval = UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval.GetValueOnAnyThread();
		VideoConfig.MultipassMode = EMultipassMode::Quarter; // NOTE we probably should allow this to be set in AVCodecs by the quality/latency presets by having an auto value
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcMediaResult TEpicRtcVideoEncoder<TVideoResource>::SetConfig(const EpicRtcVideoEncoderConfig& VideoEncoderConfig)
	{
		if (VideoEncoderConfig._numberOfSimulcastStreams > 1 && VideoEncoderConfig._codec != EpicRtcVideoCodec::VP9)
		{
			return EpicRtcMediaResult::ErrSimulcastParametersNotSupported;
		}

		// Acquire resources
		EpicRtcVideoEncoderConfig ConfigCopy = VideoEncoderConfig;
		if (ConfigCopy._simulcastStreams)
		{
			ConfigCopy._simulcastStreams->AddRef();
		}

		if (ConfigCopy._spatialLayers)
		{
			ConfigCopy._spatialLayers->AddRef();
		}

		// Release previous
		if (EncoderConfig._simulcastStreams)
		{
			EncoderConfig._simulcastStreams->Release();
		}

		if (EncoderConfig._spatialLayers)
		{
			EncoderConfig._spatialLayers->Release();
		}

		EpicRtcVideoEncoderConfig OldConfig = EncoderConfig;
		EncoderConfig = MoveTemp(ConfigCopy);

		if (Encoder)
		{
			// We're already initialized, so this SetConfig call is triggered by WebRTC's SetRates. Just update the rates

			// This call to SetConfig is triggered by a res change. In this case, we don't need to do anything because
			// the underlying encoder will handle the reconfiguration
			if (OldConfig._width != EncoderConfig._width || OldConfig._height != EncoderConfig._height)
			{
				UpdateConfig();
				return EpicRtcMediaResult::Ok;
			}

			for (size_t si = 0; si < Video::MaxSpatialLayers; ++si)
			{
				for (size_t ti = 0; ti < Video::MaxTemporalStreams; ++ti)
				{
					EpicRtcTargetBitrates[si][ti].Emplace(VideoEncoderConfig._rateControl._bitrate->GetBitrate(si, ti));
				}
			}

			EpicRtcTargetFramerate.Emplace(VideoEncoderConfig._rateControl._framerateFps);

			UpdateConfig();
			return EpicRtcMediaResult::Ok;
		}
		else
		{
			// We haven't initialized a encoder, so this SetConfig call is triggered by WebRTC's InitEncode
			Encoder = CreateEncoder(VideoEncoderConfig, CodecInfo->GetCodec());

			if (!Encoder)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "TEpicRtcVideoEncoder<{0}>::SetConfig: Unable to create {1} encoder!", UE_STRINGIZE(TVideoResource), ToString(CodecInfo->GetCodec()));
				return EpicRtcMediaResult::Error;
			}

			return EpicRtcMediaResult::Ok;
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcVideoEncoderInfo TEpicRtcVideoEncoder<TVideoResource>::GetInfo()
	{
		// HACK: EpicRtcVideoEncoderInfo is not a ref counted object (yet?) but it holds ref counted objects.
		// in order to keep the member ref counted objects alive when this function goes out of scope they are kept as members.
		EpicRtcVideoEncoderInfo VideoEncoderInfo = {
			._requestedResolutionAlignment = 1,
			._applyAlignmentToAllSimulcastLayers = false,
			._supportsNativeHandle = true,
			._codecInfo = CodecInfo,
			._resolutionBitrateLimits = ResolutionBitrateLimits,
			._supportsSimulcast = false,
			._preferredPixelFormats = PreferredPixelFormats
		};

		if constexpr (std::is_same_v<TVideoResource, FVideoResourceRHI>)
		{
			VideoEncoderInfo._isHardwareAccelerated = true;
		}
		else if constexpr (std::is_same_v<TVideoResource, FVideoResourceCPU>)
		{
			VideoEncoderInfo._isHardwareAccelerated = false;
		}

		return VideoEncoderInfo;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	EpicRtcMediaResult TEpicRtcVideoEncoder<TVideoResource>::Encode(const EpicRtcVideoFrame& VideoFrame, EpicRtcVideoFrameTypeArrayInterface* FrameTypes)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("TEpicRtcVideoEncoder::Encode", PixelStreaming2Channel);

		// Capture the Callback to ensure it is not released in a different thread.
		TRefCountPtr<EpicRtcVideoEncoderCallbackInterface> CallbackEncoded(VideoEncoderCallback);
		if (!CallbackEncoded)
		{
			return EpicRtcMediaResult::Uninitialized;
		}

		TRefCountPtr<EpicRtcVideoBufferInterface> InputBuffer(VideoFrame._buffer);
		if (!InputBuffer)
		{
			return EpicRtcMediaResult::Error;
		}

		FEpicRtcVideoBufferMultiFormatLayered* VideoBufferLayered = StaticCast<FEpicRtcVideoBufferMultiFormatLayered*>(InputBuffer.GetReference());

		TRefCountPtr<FEpicRtcVideoBufferMultiFormat> VideoBufferMultiFormat = VideoBufferLayered->GetLayer({ static_cast<int>(EncoderConfig._width), static_cast<int>(EncoderConfig._height) });

		if (!VideoBufferMultiFormat)
		{
			// No layer matches the expected encoder config
			return EpicRtcMediaResult::Error;
		}

		// Check whether the output frame is valid because null frames are passed to stream sharing encoders.
		IPixelCaptureOutputFrame* AdaptedLayer;
		if constexpr (std::is_same_v<TVideoResource, FVideoResourceRHI>)
		{
			AdaptedLayer = VideoBufferMultiFormat->RequestFormat(PixelCaptureBufferFormat::FORMAT_RHI);
		}
		else if constexpr (std::is_same_v<TVideoResource, FVideoResourceCPU>)
		{
			AdaptedLayer = VideoBufferMultiFormat->RequestFormat(PixelCaptureBufferFormat::FORMAT_I420);
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "VideoResource isn't a compatible type! Expected either a FVideoResourceRHI or FVideoResourceCPU. Received: {0}", UE_STRINGIZE(TVideoResource));
			return EpicRtcMediaResult::Error;
		}

		if (!AdaptedLayer)
		{
			// probably the first request which starts the adapt pipeline for this format
			return EpicRtcMediaResult::Ok;
		}

		const int Width = VideoBufferMultiFormat->GetWidth();
		const int Height = VideoBufferMultiFormat->GetHeight();

		// Update the encoding config using the incoming frame resolution (required for dynamic res support)

		UpdateFrameMetadataPreEncode(*AdaptedLayer);

		TSharedPtr<TVideoResource> VideoResource;
		if constexpr (std::is_same_v<TVideoResource, FVideoResourceRHI>)
		{
			const FPixelCaptureOutputFrameRHI& RHILayer = StaticCast<const FPixelCaptureOutputFrameRHI&>(*AdaptedLayer);
			// Ensure we have a texture. Some capturers (eg mediacapture), can return frames with no texture while it's initializing
			if (RHILayer.GetFrameTexture() == nullptr)
			{
				return EpicRtcMediaResult::Ok;
			}

			VideoResource = MakeShared<FVideoResourceRHI>(
				Encoder->GetDevice().ToSharedRef(),
				FVideoResourceRHI::FRawData{ RHILayer.GetFrameTexture(), nullptr, 0 });
		}
		else if constexpr (std::is_same_v<TVideoResource, FVideoResourceCPU>)
		{
			const FPixelCaptureOutputFrameI420& CPULayer = StaticCast<const FPixelCaptureOutputFrameI420&>(*AdaptedLayer);
			// Ensure we have a texture. Some capturers (eg mediacapture), can return frames with no texture while it's initializing
			if (CPULayer.GetI420Buffer() == nullptr)
			{
				return EpicRtcMediaResult::Ok;
			}

			VideoResource = MakeShared<FVideoResourceCPU>(
				Encoder->GetDevice().ToSharedRef(),
				MakeShareable<uint8>(CPULayer.GetI420Buffer()->GetMutableData(), TFakeDeleter<uint8>()),
				FAVLayout(CPULayer.GetI420Buffer()->GetStrideY(), 0, CPULayer.GetI420Buffer()->GetSize()),
				FVideoDescriptor(EVideoFormat::YUV420, Width, Height));
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "VideoResource isn't a compatible type! Expected either a FVideoResourceRHI or FVideoResourceCPU. Received: {0}", UE_STRINGIZE(TVideoResource));
			return EpicRtcMediaResult::Error;
		}

		const bool bKeyFrame = (FrameTypes && FrameTypes->Size() > 0 && (FrameTypes->Get()[0] == EpicRtcVideoFrameType::I));

		// Encode
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("TVideoEncoder::SendFrame", PixelStreaming2Channel);
			Encoder->SendFrame(VideoResource, VideoFrame._timestampUs, bKeyFrame);
		}

		UpdateFrameMetadataPostEncode(*AdaptedLayer);

		FVideoPacket Packet;
		while (Encoder->ReceivePacket(Packet))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("TVideoEncoder::ReceivePacket", PixelStreaming2Channel);
			
			TRefCountPtr<FEpicRtcEncodedVideoBuffer> EncodedBuffer = new FEpicRtcEncodedVideoBuffer(Packet.DataPtr.Get(), Packet.DataSize);

			EpicRtcEncodedVideoFrame EncodedFrame = {
				._width = Width,
				._height = Height,
				._timestampUs = VideoFrame._timestampUs,
				._timestampRtp = VideoFrame._timestampRtp,
				._frameType = Packet.bIsKeyframe ? EpicRtcVideoFrameType::I : EpicRtcVideoFrameType::P,
				._qp = static_cast<int>(Packet.QP),
				._buffer = EncodedBuffer,
				._spatialIndex = Packet.SpatialIndex.Get(0),
				._hasSpatialIndex = Packet.SpatialIndex.IsSet(),
				._temporalIndex = Packet.TemporalIndex.Get(0),
				._hasTemporalIndex = Packet.TemporalIndex.IsSet(),
			};

			EpicRtcCodecSpecificInfo CodecSpecificInfo = {
				._codec = CodecInfo->GetCodec()
			};

			if (CodecInfo->GetCodec() == EpicRtcVideoCodec::H264)
			{
				constexpr uint8_t kNoTemporalIdx = 0xff;
				CodecSpecificInfo._codecSpecific._h264 = {
					._isSingleNAL = false,
					._temporalIdx = kNoTemporalIdx,
					._baseLayerSync = false,
					._isIDR = static_cast<bool>(Packet.bIsKeyframe)
				};
			}
			else if (CodecInfo->GetCodec() == EpicRtcVideoCodec::VP8)
			{
				memset(&CodecSpecificInfo._codecSpecific._vp8, 0, sizeof(CodecSpecificInfo._codecSpecific._vp8));

				EpicRtcCodecSpecificInfoVP8& VP8Info = CodecSpecificInfo._codecSpecific._vp8;

				VP8Info._nonReference = Packet.CodecSpecificInfo.CodecSpecific.VP8.bNonReference;
				VP8Info._temporalIdx = Packet.CodecSpecificInfo.CodecSpecific.VP8.TemporalIdx;
				VP8Info._layerSync = Packet.CodecSpecificInfo.CodecSpecific.VP8.bLayerSync;
				VP8Info._keyIdx = Packet.CodecSpecificInfo.CodecSpecific.VP8.KeyIdx;
				VP8Info._useExplicitDependencies = Packet.CodecSpecificInfo.CodecSpecific.VP8.bUseExplicitDependencies;
				VP8Info._referencedBuffersCount = Packet.CodecSpecificInfo.CodecSpecific.VP8.ReferencedBuffersCount;
				VP8Info._updatedBuffersCount = Packet.CodecSpecificInfo.CodecSpecific.VP8.UpdatedBuffersCount;

				FMemory::Memcpy(VP8Info._referencedBuffers, Packet.CodecSpecificInfo.CodecSpecific.VP8.ReferencedBuffers, sizeof(size_t) * Packet.CodecSpecificInfo.CodecSpecific.VP8.BuffersCount);
				FMemory::Memcpy(VP8Info._updatedBuffers, Packet.CodecSpecificInfo.CodecSpecific.VP8.UpdatedBuffers, sizeof(size_t) * Packet.CodecSpecificInfo.CodecSpecific.VP8.BuffersCount);
			}
			else if (CodecInfo->GetCodec() == EpicRtcVideoCodec::VP9)
			{
				CodecSpecificInfo._endOfPicture = Packet.CodecSpecificInfo.bEndOfPicture;
				if (Packet.CodecSpecificInfo.GenericFrameInfo.IsSet())
				{
					CodecSpecificInfo._genericFrameInfo = new FEpicRtcGenericFrameInfo(Packet.CodecSpecificInfo.GenericFrameInfo.GetValue());
					CodecSpecificInfo._genericFrameInfo->AddRef();
					CodecSpecificInfo._hasGenericFrameInfo = true;
				}

				if (Packet.CodecSpecificInfo.TemplateStructure.IsSet())
				{
					CodecSpecificInfo._templateStructure = new FEpicRtcFrameDependencyStructure(Packet.CodecSpecificInfo.TemplateStructure.GetValue());
					CodecSpecificInfo._templateStructure->AddRef();
					CodecSpecificInfo._hasTemplateStructure = true;
				}

				memset(&CodecSpecificInfo._codecSpecific._vp9, 0, sizeof(CodecSpecificInfo._codecSpecific._vp9));

				EpicRtcCodecSpecificInfoVP9& VP9Info = CodecSpecificInfo._codecSpecific._vp9;

				VP9Info._firstFrameInPicture = Packet.CodecSpecificInfo.CodecSpecific.VP9.bFirstFrameInPicture;
				VP9Info._interPicPredicted = Packet.CodecSpecificInfo.CodecSpecific.VP9.bInterPicPredicted;
				VP9Info._flexibleMode = Packet.CodecSpecificInfo.CodecSpecific.VP9.bFlexibleMode;
				VP9Info._ssDataAvailable = Packet.CodecSpecificInfo.CodecSpecific.VP9.bSSDataAvailable;
				VP9Info._nonRefForInterLayerPred = Packet.CodecSpecificInfo.CodecSpecific.VP9.bNonRefForInterLayerPred;

				VP9Info._temporalIdx = Packet.CodecSpecificInfo.CodecSpecific.VP9.TemporalIdx;
				VP9Info._temporalUpSwitch = Packet.CodecSpecificInfo.CodecSpecific.VP9.bTemporalUpSwitch;
				VP9Info._interLayerPredicted = Packet.CodecSpecificInfo.CodecSpecific.VP9.bInterLayerPredicted;
				VP9Info._gofIdx = Packet.CodecSpecificInfo.CodecSpecific.VP9.GofIdx;

				VP9Info._numSpatialLayers = Packet.CodecSpecificInfo.CodecSpecific.VP9.NumSpatialLayers;
				VP9Info._firstActiveLayer = Packet.CodecSpecificInfo.CodecSpecific.VP9.FirstActiveLayer;
				VP9Info._spatialLayerResolutionPresent = Packet.CodecSpecificInfo.CodecSpecific.VP9.bSpatialLayerResolutionPresent;
				FMemory::Memcpy(VP9Info._width, Packet.CodecSpecificInfo.CodecSpecific.VP9.Width, sizeof(uint16) * UE::AVCodecCore::VP9::MaxNumberOfSpatialLayers);
				FMemory::Memcpy(VP9Info._height, Packet.CodecSpecificInfo.CodecSpecific.VP9.Height, sizeof(uint16) * UE::AVCodecCore::VP9::MaxNumberOfSpatialLayers);

				VP9Info._gof._numFramesInGof = Packet.CodecSpecificInfo.CodecSpecific.VP9.Gof.NumFramesInGof;
				for (size_t i = 0; i < VP9Info._gof._numFramesInGof; ++i)
				{
					VP9Info._gof._temporalIdx[i] = Packet.CodecSpecificInfo.CodecSpecific.VP9.Gof.TemporalIdx[i];
					VP9Info._gof._temporalUpSwitch[i] = Packet.CodecSpecificInfo.CodecSpecific.VP9.Gof.TemporalUpSwitch[i];
					VP9Info._gof._numRefPics[i] = Packet.CodecSpecificInfo.CodecSpecific.VP9.Gof.NumRefPics[i];
					for (uint8_t r = 0; r < VP9Info._gof._numRefPics[i]; ++r)
					{
						VP9Info._gof._pidDiff[i][r] = Packet.CodecSpecificInfo.CodecSpecific.VP9.Gof.PidDiff[i][r];
					}
				}
				VP9Info._numRefPics = Packet.CodecSpecificInfo.CodecSpecific.VP9.NumRefPics;

				FMemory::Memcpy(VP9Info._pDiff, Packet.CodecSpecificInfo.CodecSpecific.VP9.PDiff, sizeof(uint8) * UE::AVCodecCore::VP9::MaxRefPics);
			}

			MaybeDumpFrame(EncodedFrame);

			UpdateFrameMetadataPrePacketization(*AdaptedLayer);
			EpicRtcVideoEncodedResult Result = VideoEncoderCallback->Encoded(EncodedFrame, CodecSpecificInfo);
			UpdateFrameMetadataPostPacketization(*AdaptedLayer);

			// It is not possible to know if the stream has ended here so a frame may be pushed despite the streaming ending.
			// This causes the Result to return an error.
			// This section only prints if there was an error in the frame before current since that will be caused by an actual error.
			if (Result._error)
			{
				if (bDidLastEncodedFramePushFail)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "PixelStreamingVideoEncoder: Failed to push previous and current encoded frame.");
				}
				else
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Log, "PixelStreamingVideoEncoder: Failed to push encoded frame. This is expected when the stream is shutting down.");
				}
				bDidLastEncodedFramePushFail = true;
			}
			else
			{
				bDidLastEncodedFramePushFail = false;
			}
		}

		return EpicRtcMediaResult::Ok;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::RegisterCallback(EpicRtcVideoEncoderCallbackInterface* InCallback)
	{
		VideoEncoderCallback = InCallback;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::Reset()
	{
		// Do not reset our internal encoder here as we manage its lifecycle and resetting it when res/fps changes.
		// Resetting our encoder here would mean a reset everytime we send a null frame during "stream sharing", which is not what we want.
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::UpdateConfig()
	{
		if (!Encoder)
		{
			return;
		}

		// We're guaranteed to have a encoder by the time this is called. No need to check
		FVideoEncoderConfig	 VideoConfigMinimal = Encoder->GetMinimalConfig();
		FVideoEncoderConfig* VideoConfig = &VideoConfigMinimal;

		switch (CodecInfo->GetCodec())
		{
			case EpicRtcVideoCodec::H264:
			{
				if (Encoder->GetInstance()->template Has<FVideoEncoderConfigH264>())
				{
					FVideoEncoderConfigH264& VideoConfigH264 = Encoder->GetInstance()->template Edit<FVideoEncoderConfigH264>();
					VideoConfig = &VideoConfigH264;

					VideoConfigH264.Profile = GetEnumFromCVar<EH264Profile>(UPixelStreaming2PluginSettings::CVarEncoderH264Profile);
				}

				// Webrtc may not have updated bitrates for us yet. In that case, we want to check that the sum is greater
				// than 0 and only update the proposed value if > 0
				uint32_t BitrateSum = SumAndResetBitrates();
				// H264 doesn't support simulcast or SVC, so just sum the layer bitrates as the target bitrate
				if (BitrateSum > 0)
				{
					EpicRtcProposedTargetBitrate = BitrateSum;
				}
				break;
			}
			case EpicRtcVideoCodec::AV1:
			{
				if (Encoder->GetInstance()->template Has<FVideoEncoderConfigAV1>())
				{
					FVideoEncoderConfigAV1& VideoConfigAV1 = Encoder->GetInstance()->template Edit<FVideoEncoderConfigAV1>();
					VideoConfig = &VideoConfigAV1;
				}

				// Webrtc may not have updated bitrates for us yet. In that case, we want to check that the sum is greater
				// than 0 and only update the proposed value if > 0
				uint32_t BitrateSum = SumAndResetBitrates();
				// AV1 doesn't support simulcast or SVC, so just sum the layer bitrates as the target bitrate
				if (BitrateSum > 0)
				{
					EpicRtcProposedTargetBitrate = BitrateSum;
				}
				break;
			}
			case EpicRtcVideoCodec::VP8:
			{
				if (Encoder->GetInstance()->template Has<FVideoEncoderConfigVP8>())
				{
					FVideoEncoderConfigVP8& VideoConfigVP8 = Encoder->GetInstance()->template Edit<FVideoEncoderConfigVP8>();
					VideoConfig = &VideoConfigVP8;
				}

				// Webrtc may not have updated bitrates for us yet. In that case, we want to check that the sum is greater
				// than 0 and only update the proposed value if > 0
				uint32_t BitrateSum = SumAndResetBitrates();
				// VP8 doesn't support simulcast or SVC, so just sum the layer bitrates as the target bitrate
				if (BitrateSum > 0)
				{
					EpicRtcProposedTargetBitrate = BitrateSum;
				}
				break;
			}
			case EpicRtcVideoCodec::VP9:
			{
				if (Encoder->GetInstance()->template Has<FVideoEncoderConfigVP9>())
				{
					FVideoEncoderConfigVP9& VideoConfigVP9 = Encoder->GetInstance()->template Edit<FVideoEncoderConfigVP9>();
					VideoConfig = &VideoConfigVP9;
				}

				for (size_t si = 0; si < Video::MaxSpatialLayers; ++si)
				{
					for (size_t ti = 0; ti < Video::MaxTemporalStreams; ++ti)
					{
						if (EpicRtcTargetBitrates[si][ti].IsSet())
						{
							VideoConfig->Bitrates[si][ti] = EpicRtcTargetBitrates[si][ti].GetValue();
						}
						EpicRtcTargetBitrates[si][ti].Reset();
					}
				}

				// Update the SpatialLayer infomation as they may have changed as well (eg res)
				for (size_t i = 0; i < EncoderConfig._spatialLayers->Size(); i++)
				{
					EpicRtcSpatialLayer SpatialLayer = EncoderConfig._spatialLayers->Get()[i];
					VideoConfig->SpatialLayers[i] = FSpatialLayer{
						.Width = static_cast<uint32>(SpatialLayer._resolution._width),
						.Height = static_cast<uint32>(SpatialLayer._resolution._height),
						.Framerate = SpatialLayer._maxFramerate,
						.NumberOfTemporalLayers = SpatialLayer._numberOfTemporalLayers,
						.MaxBitrate = static_cast<int32>(SpatialLayer._maxBitrate),
						.TargetBitrate = static_cast<int32>(SpatialLayer._targetBitrate),
						.MinBitrate = static_cast<int32>(SpatialLayer._minBitrate),
						.MaxQP = static_cast<int32>(SpatialLayer._qpMax),
						.bActive = static_cast<bool>(SpatialLayer._active)
					};
				}
				break;
			}

			default:
				// We don't support encoders for other codecs
				checkNoEntry();
		}

		if (EpicRtcTargetFramerate.IsSet())
		{
			VideoConfig->TargetFramerate = EpicRtcTargetFramerate.GetValue();
			EpicRtcTargetFramerate.Reset();
		}

		// Change encoder settings through CVars
		const int32 TargetBitrateCVar = UPixelStreaming2PluginSettings::CVarEncoderTargetBitrate.GetValueOnAnyThread();

		VideoConfig->MinBitrate = UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread();
		VideoConfig->MaxBitrate = UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread();
		VideoConfig->TargetBitrate = TargetBitrateCVar > -1 ? TargetBitrateCVar : EpicRtcProposedTargetBitrate;
		VideoConfig->MinQuality = UPixelStreaming2PluginSettings::CVarEncoderMinQuality.GetValueOnAnyThread();
		VideoConfig->MaxQuality = UPixelStreaming2PluginSettings::CVarEncoderMaxQuality.GetValueOnAnyThread();
		VideoConfig->RateControlMode = ERateControlMode::CBR;
		VideoConfig->MultipassMode = EMultipassMode::Quarter; // Note we probably should add an EMultipassMode::Auto and let presets decide this
		VideoConfig->bFillData = false;
		VideoConfig->Width = EncoderConfig._width;
		VideoConfig->Height = EncoderConfig._height;

		Encoder->SetMinimalConfig(*VideoConfig);
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::OnEncoderDebugDumpFrameChanged(IConsoleVariable* Var)
	{
		if (Var->GetBool())
		{
			CreateDumpFile();
		}
		else
		{
			FileHandle = nullptr;
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::CreateDumpFile()
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FString		   TempFilePath = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), TEXT("encoded_frame"), TEXT(".raw"));
		FileHandle = PlatformFile.OpenWrite(*TempFilePath);
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::MaybeDumpFrame(EpicRtcEncodedVideoFrame const& EncodedImage)
	{
		// Dump encoded frames to file for debugging if CVar is turned on.
		if (!FileHandle)
		{
			return;
		}

		// Note: To examine individual frames from this dump: ffmpeg -i "encoded_frame78134A5047638BB99AE1D88471E5E513.raw" "frames/out-%04d.jpg"
		FileHandle->Write(EncodedImage._buffer->GetData(), EncodedImage._buffer->GetSize());
		FileHandle->Flush();
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	uint32_t TEpicRtcVideoEncoder<TVideoResource>::SumAndResetBitrates()
	{
		uint32_t SumBps = 0;
		for (size_t si = 0; si < Video::MaxSpatialLayers; ++si)
		{
			for (size_t ti = 0; ti < Video::MaxTemporalStreams; ++ti)
			{
				if (EpicRtcTargetBitrates[si][ti].IsSet())
				{
					SumBps += EpicRtcTargetBitrates[si][ti].GetValue();
				}
				EpicRtcTargetBitrates[si][ti].Reset();
			}
		}

		return SumBps;
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::UpdateFrameMetadataPreEncode(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.UseCount++;
		FrameMetadata.LastEncodeStartTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		if (FrameMetadata.UseCount == 1)
		{
			FrameMetadata.FirstEncodeStartTime = FrameMetadata.LastEncodeStartTime;
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::UpdateFrameMetadataPostEncode(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastEncodeEndTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		FStats::Get()->AddFrameTimingStats(FrameMetadata, { Frame.GetWidth(), Frame.GetHeight() });
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::UpdateFrameMetadataPrePacketization(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastPacketizationStartTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		if (FrameMetadata.UseCount == 1)
		{
			FrameMetadata.FirstPacketizationStartTime = FrameMetadata.LastPacketizationStartTime;
		}
	}

	template <std::derived_from<FVideoResource> TVideoResource>
	void TEpicRtcVideoEncoder<TVideoResource>::UpdateFrameMetadataPostPacketization(IPixelCaptureOutputFrame& Frame)
	{
		FPixelCaptureFrameMetadata& FrameMetadata = Frame.Metadata;
		FrameMetadata.LastPacketizationEndTime = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

		FStats::Get()->AddFrameTimingStats(FrameMetadata, { Frame.GetWidth(), Frame.GetHeight() });
	}

	// Explicit specialisation
	template class TEpicRtcVideoEncoder<FVideoResourceRHI>;
	template class TEpicRtcVideoEncoder<FVideoResourceCPU>;
} // namespace UE::PixelStreaming2
