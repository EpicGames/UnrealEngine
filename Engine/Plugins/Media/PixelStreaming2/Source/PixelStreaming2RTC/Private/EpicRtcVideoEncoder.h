// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVConstants.h"
#include "EpicRtcContainers.h"
#include "HAL/PlatformFileManager.h"
#include "IPixelCaptureOutputFrame.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP8.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP9.h"
#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Resources/VideoResourceRHI.h"
#include "Video/VideoEncoder.h"

#include "epic_rtc/core/video/video_encoder.h"
#include "epic_rtc_helper/memory/ref_count_impl_helper.h"

namespace UE::PixelStreaming2
{
	template <std::derived_from<FVideoResource> TVideoResource>
	class TEpicRtcVideoEncoder : public EpicRtcVideoEncoderInterface
	{
	public:
		TEpicRtcVideoEncoder(EpicRtcVideoCodecInfoInterface* InCodecInfo);
		virtual ~TEpicRtcVideoEncoder() override;

		// Begin EpicRtcVideoEncoderInterface
		virtual EpicRtcStringView		  GetName() const override;
		virtual EpicRtcVideoEncoderConfig GetConfig() const override;
		virtual EpicRtcMediaResult		  SetConfig(const EpicRtcVideoEncoderConfig& VideoEncoderConfig) override;
		virtual EpicRtcVideoEncoderInfo	  GetInfo() override;
		virtual EpicRtcMediaResult		  Encode(const EpicRtcVideoFrame& VideoFrame, EpicRtcVideoFrameTypeArrayInterface* FrameTypes) override;
		virtual void					  RegisterCallback(EpicRtcVideoEncoderCallbackInterface* InCallback) override;
		virtual void					  Reset() override;
		// End EpicRtcVideoEncoderInterface

	private:
		EpicRtcVideoEncoderConfig								EncoderConfig;
		TSharedPtr<TVideoEncoder<TVideoResource>>				Encoder;
		TRefCountPtr<EpicRtcVideoEncoderCallbackInterface>		VideoEncoderCallback;
		TRefCountPtr<EpicRtcVideoCodecInfoInterface>			CodecInfo;
		TRefCountPtr<FEpicRtcPixelFormatArray>					PreferredPixelFormats;
		TRefCountPtr<FEpicRtcVideoResolutionBitrateLimitsArray> ResolutionBitrateLimits;

		TOptional<uint32_t> EpicRtcTargetBitrates[Video::MaxSpatialLayers][Video::MaxTemporalStreams];
		TOptional<double>	EpicRtcTargetFramerate;

		// We store this so we can restore back to it if the user decides to use then stop using the PixelStreaming2.Encoder.TargetBitrate CVar.
		uint32_t EpicRtcProposedTargetBitrate = 5000000;

		// File handle used to save the encoded output to
		IFileHandle* FileHandle = nullptr;
		bool		 bDidLastEncodedFramePushFail = false;

	private:
		TSharedPtr<TVideoEncoder<TVideoResource>> CreateEncoder(const EpicRtcVideoEncoderConfig& Config, const EpicRtcVideoCodec Codec);
		void									  SetInitialSettings(const EpicRtcVideoEncoderConfig& CodecSettings, FVideoEncoderConfig& VideoConfig);
		void									  UpdateConfig();
		void									  MaybeDumpFrame(EpicRtcEncodedVideoFrame const& EncodedImage);
		uint32_t								  SumAndResetBitrates();

		void UpdateFrameMetadataPreEncode(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPostEncode(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPrePacketization(IPixelCaptureOutputFrame& Frame);
		void UpdateFrameMetadataPostPacketization(IPixelCaptureOutputFrame& Frame);

		void OnEncoderDebugDumpFrameChanged(IConsoleVariable* Var);
		void CreateDumpFile();

		FDelegateHandle DelegateHandle;

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface
	};
} // namespace UE::PixelStreaming2
