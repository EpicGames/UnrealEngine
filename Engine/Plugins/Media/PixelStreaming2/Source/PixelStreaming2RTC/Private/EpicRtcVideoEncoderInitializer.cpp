// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoEncoderInitializer.h"

#include "UtilsCoder.h"
#include "EpicRtcVideoEncoder.h"
#include "Logging.h"
#include "NvmlEncoder.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "UtilsAsync.h"
#include "UtilsString.h"
#include "UtilsCodecs.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP8.h"
#include "Video/Encoders/Configs/VideoEncoderConfigVP9.h"
#include "Video/Resources/VideoResourceCPU.h"
#include "Video/Resources/VideoResourceRHI.h"

namespace
{
	template <typename TConfig>
	EpicRtcVideoEncoderInterface* CreateEncoder(EpicRtcVideoCodecInfoInterface* CodecInfo)
	{
		EpicRtcVideoEncoderInterface* Encoder = nullptr;

		if (UE::PixelStreaming2::IsHardwareEncoderSupported<TConfig>())
		{
			Encoder = new UE::PixelStreaming2::TEpicRtcVideoEncoder<FVideoResourceRHI>(CodecInfo);
		}
		else if (UE::PixelStreaming2::IsSoftwareEncoderSupported<TConfig>())
		{
			Encoder = new UE::PixelStreaming2::TEpicRtcVideoEncoder<FVideoResourceCPU>(CodecInfo);
		}

		return Encoder;
	}

	// Explicit initialisation. We don't want this util namespace public
	template EpicRtcVideoEncoderInterface* CreateEncoder<FVideoEncoderConfigH264>(EpicRtcVideoCodecInfoInterface* CodecInfo);
	template EpicRtcVideoEncoderInterface* CreateEncoder<FVideoEncoderConfigAV1>(EpicRtcVideoCodecInfoInterface* CodecInfo);
	template EpicRtcVideoEncoderInterface* CreateEncoder<FVideoEncoderConfigVP8>(EpicRtcVideoCodecInfoInterface* CodecInfo);
	template EpicRtcVideoEncoderInterface* CreateEncoder<FVideoEncoderConfigVP9>(EpicRtcVideoCodecInfoInterface* CodecInfo);
} // namespace

namespace UE::PixelStreaming2
{
	void FEpicRtcVideoEncoderInitializer::CreateEncoder(EpicRtcVideoCodecInfoInterface* CodecInfo, EpicRtcVideoEncoderInterface** OutEncoder)
	{
		EpicRtcVideoEncoderInterface* Encoder = nullptr;

		switch (CodecInfo->GetCodec())
		{
			case EpicRtcVideoCodec::H264:
				Encoder = ::CreateEncoder<FVideoEncoderConfigH264>(CodecInfo);
				break;
			case EpicRtcVideoCodec::AV1:
				Encoder = ::CreateEncoder<FVideoEncoderConfigAV1>(CodecInfo);
				break;
			case EpicRtcVideoCodec::VP8:
				Encoder = ::CreateEncoder<FVideoEncoderConfigVP8>(CodecInfo);
				break;
			case EpicRtcVideoCodec::VP9:
				Encoder = ::CreateEncoder<FVideoEncoderConfigVP9>(CodecInfo);
				break;
			default:
				// Every codec should be accounted for
				checkNoEntry();
		}

		if (!Encoder)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create encoder!");
			return;
		}

		// Because the ptr was created with new, we need to call AddRef ourself (ms spec compliant)
		Encoder->AddRef();

		*OutEncoder = Encoder;
	}

	EpicRtcStringView FEpicRtcVideoEncoderInitializer::GetName()
	{
		static FUtf8String Name("PixelStreamingVideoEncoder");
		return UE::PixelStreaming2::ToEpicRtcStringView(Name);
	}

	// We want this method to return all the formats we have encoders for but the selected codecs formats should be first in the list.
	// There is some nuance to this though, we cannot simply return just the selected codec. The reason for this because when we receive
	// video from another pixel streaming source, for some reason WebRTC will query the encoder factory on the receiving end and if it
	// doesnt support the video we are receiving then transport_cc is not enabled which leads to very low bitrate streams.
	EpicRtcVideoCodecInfoArrayInterface* FEpicRtcVideoEncoderInitializer::GetSupportedCodecs()
	{
		// static so we dont create the list every time this is called since the list will not change during runtime.
		static const TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> SupportedCodecMap = CreateSupportedEncoderMap();

		EVideoCodec SelectedCodec = UE::PixelStreaming2::GetEnumFromString<EVideoCodec>(GSelectedCodec);
		const bool	bNegotiateCodecs = UPixelStreaming2PluginSettings::CVarWebRTCNegotiateCodecs.GetValueOnAnyThread();

		// This array of support codecs gets built up and returned in our preference order
		TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>> SupportedCodecs;
		// Codecs that have been denied (e.g. a codec may be denied if all HW encoder instances are in use)
		TSet<EVideoCodec> DenyListedCodecs;

#if PLATFORM_WINDOWS || PLATFORM_LINUX
		if ((SelectedCodec == EVideoCodec::H264 || SelectedCodec == EVideoCodec::AV1 || bNegotiateCodecs) && IsRHIDeviceNVIDIA())
		{
			int32 NumEncoderSessions = NvmlEncoder::GetEncoderSessionCount(0); // TODO we should probably actually figure out the GPU index rather than assume 0
			int32 MaxCVarAllowedSessions = UPixelStreaming2PluginSettings::CVarEncoderMaxSessions.GetValueOnAnyThread();
			bool  bCanCreateHardwareEncoder = true;

			if (MaxCVarAllowedSessions != -1 && NumEncoderSessions != -1)
			{
				// If our CVar is set and we receive a valid session count
				bCanCreateHardwareEncoder &= NumEncoderSessions < MaxCVarAllowedSessions;
			}
			else if (MaxCVarAllowedSessions == -1)
			{
				// If we receive a valid session count and our cvar isn't set
				bCanCreateHardwareEncoder &= NvmlEncoder::IsEncoderSessionAvailable(0); // TODO we should probably actually figure out the GPU index rather than assume 0
			}

			if (!bCanCreateHardwareEncoder)
			{
				// No more hardware encoder sessions available. Fallback to VP8
				// NOTE: CVars can only be set from game thread
				UE::PixelStreaming2::DoOnGameThread([]() {
					UPixelStreaming2PluginSettings::CVarEncoderCodec.AsVariable()->Set(*UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP8));
					if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
					{
						Delegates->OnFallbackToSoftwareEncoding.Broadcast();
						Delegates->OnFallbackToSoftwareEncodingNative.Broadcast();
					}
				});
				// Also update our local SelectedCodec to reflect what the state will be
				GSelectedCodec = TEXT("VP8");
				SelectedCodec = EVideoCodec::VP8;
				UE_LOGF(LogPixelStreaming2RTC, Warning, "No more HW encoders available. Falling back to software encoding");
				DenyListedCodecs.Add(EVideoCodec::H264);
				DenyListedCodecs.Add(EVideoCodec::AV1);
			}
		}
#endif // PLATFORM_WINDOWS || PLATFORM_LINUX

		// If we are not negotiating codecs simply return just the one codec that is selected in UE
		if (!bNegotiateCodecs)
		{
			if (DenyListedCodecs.Contains(SelectedCodec))
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Selected codec was denied - most like due to lack of hw encoder sessions.");
			}
			else if (SupportedCodecMap.Contains(SelectedCodec))
			{
				SupportedCodecs.Append(SupportedCodecMap[SelectedCodec]);
			}
			else
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Selected codec was not a supported codec.");
			}
		}
		else
		{
			if (UPixelStreaming2PluginSettings::CVarEncoderEnableSimulcast.GetValueOnAnyThread())
			{
				// Only H264 and VP8 support simulcast in the way we do it
				DenyListedCodecs.Add(EVideoCodec::VP9);
				DenyListedCodecs.Add(EVideoCodec::AV1);
				UE_LOGF(LogPixelStreaming2RTC, Warning, "Removing VP9 and AV1 from negotiable codecs due to simulcast being enabled");
			}

			for (auto& Codec : UPixelStreaming2PluginSettings::GetCodecPreferences())
			{
				if (SupportedCodecMap.Contains(Codec) && !DenyListedCodecs.Contains(Codec))
				{
					SupportedCodecs.Append(SupportedCodecMap[Codec]);
				}
			}
		}

		return new FVideoCodecInfoArray(SupportedCodecs);
	}

	TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> FEpicRtcVideoEncoderInitializer::CreateSupportedEncoderMap()
	{
		TMap<EVideoCodec, TArray<TRefCountPtr<EpicRtcVideoCodecInfoInterface>>> Codecs;
		for (auto& Codec : SupportedVideoCodecs)
		{
			Codecs.Add(Codec);
		}

		// UE doesn't support the automatic conversion of a RefCountPtr from a derived type to a base type.
		if (UE::PixelStreaming2::IsEncoderSupported<FVideoEncoderConfigVP8>())
		{
			Codecs[EVideoCodec::VP8].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::VP8,
				nullptr,
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
		}

		if (UE::PixelStreaming2::IsEncoderSupported<FVideoEncoderConfigVP9>())
		{
			using namespace UE::AVCodecCore::VP9;
			Codecs[EVideoCodec::VP9].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::VP9,
				UE::PixelStreaming2::CreateVP9Format(EProfile::Profile0),
				new FEpicRtcScalabilityModeArray(UE::PixelStreaming2::AllScalabilityModes)
			)));

			/**
			 * Only advertise profile 0 until EpicRtc provides us with a way to extract the negotiated profile in the encoder config
			 * 
			 * Codecs[EVideoCodec::VP9].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
			 *	EpicRtcVideoCodec::VP9,
			 *	UE::PixelStreaming2::CreateVP9Format(EProfile::Profile2),
			 *	new FEpicRtcScalabilityModeArray(AllScalabilityModes))));
			 */
		}

		if (UE::PixelStreaming2::IsEncoderSupported<FVideoEncoderConfigH264>())
		{
			using namespace UE::AVCodecCore::H264;
			Codecs[EVideoCodec::H264].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::H264,
				UE::PixelStreaming2::CreateH264Format(EH264Profile::ConstrainedBaseline, EH264Level::Level_3_1),
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
			Codecs[EVideoCodec::H264].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::H264,
				UE::PixelStreaming2::CreateH264Format(EH264Profile::Baseline, EH264Level::Level_3_1),
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
		}

		if (UE::PixelStreaming2::IsEncoderSupported<FVideoEncoderConfigAV1>())
		{
			Codecs[EVideoCodec::AV1].Add(TRefCountPtr<EpicRtcVideoCodecInfoInterface>(new FEpicRtcVideoCodecInfo(
				EpicRtcVideoCodec::AV1,
				nullptr,
				new FEpicRtcScalabilityModeArray({ EScalabilityMode::L1T1 })
			)));
		}

		return Codecs;
	}

} // namespace UE::PixelStreaming2