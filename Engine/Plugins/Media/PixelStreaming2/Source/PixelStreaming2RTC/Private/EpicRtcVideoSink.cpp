// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoSink.h"

#include "Async/Async.h"
#include "ColorConversion.h"
#include "EpicRtcVideoBufferI420.h"
#include "EpicRtcVideoBufferRHI.h"
#include "IPixelCaptureOutputFrame.h"
#include "Logging.h"
#include "PixelCaptureInputFrameI420.h"
#include "PixelCaptureInputFrameRHI.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelStreaming2Trace.h"
#include "RenderTargetPool.h"
#include "Stats.h"
#include "UtilsString.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FEpicRtcVideoSink> FEpicRtcVideoSink::Create(TRefCountPtr<EpicRtcVideoTrackInterface> InTrack)
	{
		TSharedPtr<FEpicRtcVideoSink> VideoSink = MakeShareable<FEpicRtcVideoSink>(new FEpicRtcVideoSink(InTrack));

		VideoSink->VideoCapturer->OnFrameCaptured.AddSP(VideoSink.ToSharedRef(), &FEpicRtcVideoSink::OnFrameCaptured);

		return VideoSink;
	}

	FEpicRtcVideoSink::FEpicRtcVideoSink(TRefCountPtr<EpicRtcVideoTrackInterface> InTrack)
		: TEpicRtcTrack(InTrack)
		, VideoCapturer(FVideoCapturer::Create(true))
	{
	}

	void FEpicRtcVideoSink::OnEpicRtcFrame(const EpicRtcVideoFrame& Frame)
	{
		if (!HasVideoConsumers() || bIsMuted || IsEngineExitRequested())
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("FEpicRtcVideoSink::OnEpicRtcFrame", PixelStreaming2Channel);

		int32_t Width = Frame._buffer->GetWidth();
		int32_t Height = Frame._buffer->GetHeight();

		EpicRtcPixelFormat BufferFormat = Frame._buffer->GetFormat();
		// Buffers that report an I420 format are backed by EpicRtc internal buffers and not PixelCapture buffers
		if (BufferFormat == EpicRtcPixelFormat::I420)
		{
			TSharedPtr<FPixelCaptureBufferI420> I420Buffer = MakeShared<FPixelCaptureBufferI420>(Width, Height);
			FMemory::Memcpy(I420Buffer->GetMutableDataY(), Frame._buffer->GetData(), I420Buffer->GetSize());

			TWeakPtr<FEpicRtcVideoSink> WeakSink = AsWeak();
			ENQUEUE_RENDER_COMMAND(CaptureDecodedFrameCommand)
			([WeakSink, I420Buffer](FRHICommandList& RHICmdList) {
				if (TSharedPtr<FEpicRtcVideoSink> PinnedSink = WeakSink.Pin())
				{
					PinnedSink->VideoCapturer->OnFrame(FPixelCaptureInputFrameI420(I420Buffer));
				}
			});
		}
		// Buffers that report a Native format are backed by PixelCapture buffers
		else if (BufferFormat == EpicRtcPixelFormat::Native)
		{
			FEpicRtcVideoBuffer* const BaseFrameBuffer = static_cast<FEpicRtcVideoBuffer*>(Frame._buffer);
			if (!BaseFrameBuffer)
			{
				return;
			}
			if (BaseFrameBuffer->GetBufferFormat() == PixelCaptureBufferFormat::FORMAT_RHI)
			{
				FEpicRtcVideoBufferRHI* const FrameBuffer = static_cast<FEpicRtcVideoBufferRHI*>(Frame._buffer);
				if (FrameBuffer == nullptr)
				{
					return;
				}
				TSharedPtr<FVideoResourceRHI, ESPMode::ThreadSafe> VideoResource = FrameBuffer->GetVideoResource();
				TWeakPtr<FEpicRtcVideoSink>						   WeakSink = AsWeak();
				ENQUEUE_RENDER_COMMAND(CaptureDecodedFrameCommand)
				([WeakSink, VideoResource, Width, Height](FRHICommandList& RHICmdList) {
					if (TSharedPtr<FEpicRtcVideoSink> PinnedSink = WeakSink.Pin())
					{
						TSharedPtr<FVideoResourceRHI, ESPMode::ThreadSafe> LocalVideoResource = VideoResource;
						if (LocalVideoResource->GetFormat() != EVideoFormat::BGRA)
						{
							LocalVideoResource = LocalVideoResource->TransformResource(FVideoDescriptor(EVideoFormat::BGRA, Width, Height));
						}
						auto& Raw = LocalVideoResource->GetRaw();
						PinnedSink->VideoCapturer->OnFrame(FPixelCaptureInputFrameRHI(Raw.Texture));
					}
				});
			}
			else if (BaseFrameBuffer->GetBufferFormat() == PixelCaptureBufferFormat::FORMAT_I420)
			{
				FEpicRtcVideoBufferI420* const FrameBuffer = static_cast<FEpicRtcVideoBufferI420*>(Frame._buffer);
				if (FrameBuffer == nullptr)
				{
					return;
				}
				TSharedPtr<FPixelCaptureBufferI420> I420Buffer = FrameBuffer->GetBuffer();
				TWeakPtr<FEpicRtcVideoSink>			WeakSink = AsWeak();
				ENQUEUE_RENDER_COMMAND(CaptureDecodedFrameCommand)
				([WeakSink, I420Buffer](FRHICommandList& RHICmdList) {
					if (TSharedPtr<FEpicRtcVideoSink> PinnedSink = WeakSink.Pin())
					{
						PinnedSink->VideoCapturer->OnFrame(FPixelCaptureInputFrameI420(I420Buffer));
					}
				});
			}
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Received an EpicRtcVideoFrame with an unhandled buffer format. Expected EpicRtcPixelFormat::I420 or EpicRtcPixelFormat::Native! Received: {0}", ToString(BufferFormat));
		}
	}

	void FEpicRtcVideoSink::OnFrameCaptured()
	{
		TWeakPtr<FEpicRtcVideoSink> WeakSink = AsWeak();
		ENQUEUE_RENDER_COMMAND(DisplayCapturedFrameCommand)
		([WeakSink](FRHICommandList& RHICmdList) {
			if (TSharedPtr<FEpicRtcVideoSink> PinnedSink = WeakSink.Pin())
			{
				TSharedPtr<IPixelCaptureOutputFrame> OutputFrame = PinnedSink->VideoCapturer->RequestFormat(PixelCaptureBufferFormat::FORMAT_RHI);
				if (!OutputFrame)
				{
					return;
				}

				TSharedPtr<FPixelCaptureOutputFrameRHI> RHIFrame = StaticCastSharedPtr<FPixelCaptureOutputFrameRHI>(OutputFrame);
				if (!RHIFrame->GetFrameTexture())
				{
					return;
				}

				OutputFrame->Metadata.UseCount++;
				if (OutputFrame->Metadata.UseCount == 1)
				{
					OutputFrame->Metadata.ProcessName = FString::Printf(TEXT("VideoSink %s"), *(OutputFrame->Metadata.ProcessName));
				}

				FStats::Get()->AddFrameTimingStats(OutputFrame->Metadata, { OutputFrame->GetWidth(), OutputFrame->GetHeight() });

				PinnedSink->OnVideoData(RHIFrame->GetFrameTexture());
			}
		});
	}
} // namespace UE::PixelStreaming2