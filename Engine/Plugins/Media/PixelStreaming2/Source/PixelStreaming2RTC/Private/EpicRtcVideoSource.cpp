// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoSource.h"

#include "Logging.h"
#include "PixelStreaming2Trace.h"
#include "VideoSourceGroup.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FEpicRtcVideoSource> FEpicRtcVideoSource::Create(TRefCountPtr<EpicRtcVideoTrackInterface> InVideoTrack, TSharedPtr<FEpicRtcVideoCapturer> InVideoCapturer, TSharedPtr<FVideoSourceGroup> InVideoSourceGroup)
	{
		TSharedPtr<FEpicRtcVideoSource> VideoSource = TSharedPtr<FEpicRtcVideoSource>(new FEpicRtcVideoSource(InVideoTrack, InVideoCapturer));

		InVideoSourceGroup->AddVideoSource(VideoSource);

		return VideoSource;
	}

	FEpicRtcVideoSource::FEpicRtcVideoSource(TRefCountPtr<EpicRtcVideoTrackInterface> InVideoTrack, TSharedPtr<FEpicRtcVideoCapturer> InVideoCapturer)
		: TEpicRtcTrack(InVideoTrack)
		, VideoCapturer(InVideoCapturer)
	{
	}

	TRefCountPtr<EpicRtcVideoTrackInterface> FEpicRtcVideoSource::GetVideoTrack()
	{
		return Track;
	}

	void FEpicRtcVideoSource::ForceKeyFrame()
	{
		Track->GenerateKeyFrame(EpicRtcStringViewSpan{ ._ptr = nullptr, ._size = 0 });
	}

	void FEpicRtcVideoSource::PushFrame()
	{
		if (!VideoCapturer->IsReady() || !Track || bIsMuted)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("PixelStreaming2 Push Video Frame", PixelStreaming2Channel);
		static uint16_t FrameId = 1;

		TRefCountPtr<EpicRtcVideoBufferInterface> FrameBuffer = VideoCapturer->GetFrameBuffer();
#if PLATFORM_ANDROID
		// Android uses the internal EpicRtc coders which require I420 frames
		TRefCountPtr<EpicRtcVideoBufferInterface> OldFrameBuffer = FrameBuffer;
		FrameBuffer = OldFrameBuffer->ToI420();
		if (!FrameBuffer)
		{
			return;
		}
#endif

		EpicRtcVideoFrame Frame = {
			._id = FrameId++,
			._timestampUs = static_cast<int64>(FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64()) * 1000),
			._timestampRtp = static_cast<int64>(0),
			._isBackedByWebRtc = false,
			._buffer = FrameBuffer
		};

		if (!Track->PushFrame(Frame))
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Unable to push Video frame id: %d", Frame._id);
		}
	}
} // namespace UE::PixelStreaming2
