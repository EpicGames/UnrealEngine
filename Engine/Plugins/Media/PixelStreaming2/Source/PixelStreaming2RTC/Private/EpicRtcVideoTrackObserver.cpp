// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoTrackObserver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EpicRtcVideoTrackObserver)

namespace UE::PixelStreaming2
{

	FEpicRtcVideoTrackObserver::FEpicRtcVideoTrackObserver(TObserverVariant<IPixelStreaming2VideoTrackObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	void FEpicRtcVideoTrackObserver::OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted)
	{
		if (UserObserver)
		{
			return UserObserver->OnVideoTrackMuted(VideoTrack, bIsMuted);
		}
	}

	void FEpicRtcVideoTrackObserver::OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame)
	{
		if (UserObserver)
		{
			return UserObserver->OnVideoTrackFrame(VideoTrack, Frame);
		}
	}

	void FEpicRtcVideoTrackObserver::OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State)
	{
		if (UserObserver)
		{
			return UserObserver->OnVideoTrackState(VideoTrack, State);
		}
	}

	void FEpicRtcVideoTrackObserver::OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame)
	{
		if (UserObserver)
		{
			return UserObserver->OnVideoTrackEncodedFrame(ParticipantId, VideoTrack, EncodedFrame);
		}
	}

	EpicRtcBool FEpicRtcVideoTrackObserver::Enabled() const
	{
		if (UserObserver)
		{
			return UserObserver->Enabled();
		}
		return false;
	}

} // namespace UE::PixelStreaming2
