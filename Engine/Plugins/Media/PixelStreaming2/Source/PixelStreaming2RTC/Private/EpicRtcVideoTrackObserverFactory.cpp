// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcVideoTrackObserverFactory.h"

#include "EpicRtcVideoTrackObserver.h"

namespace UE::PixelStreaming2
{
	FEpicRtcVideoTrackObserverFactory::FEpicRtcVideoTrackObserverFactory(TObserverVariant<IPixelStreaming2VideoTrackObserver> UserObserver)
		: UserObserver(UserObserver)
	{
	}

	EpicRtcErrorCode FEpicRtcVideoTrackObserverFactory::CreateVideoTrackObserver(const EpicRtcStringView ParticipantId, const EpicRtcStringView VideoTrackId, EpicRtcVideoTrackObserverInterface** OutVideoTrackObserver)
	{
		EpicRtcVideoTrackObserverInterface* VideoTrackObserver = new FEpicRtcVideoTrackObserver(UserObserver);
		// Because the ptr was created with new, we need to call AddRef ourself (ms spec compliant)
		VideoTrackObserver->AddRef();

		*OutVideoTrackObserver = VideoTrackObserver;
		return EpicRtcErrorCode::Ok;
	}

} // namespace UE::PixelStreaming2