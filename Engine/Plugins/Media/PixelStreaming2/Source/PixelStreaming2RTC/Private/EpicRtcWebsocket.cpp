// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcWebsocket.h"

#include "IPixelStreaming2Module.h"
#include "IWebSocket.h"
#include "PixelStreaming2PluginSettings.h"
#include "UtilsString.h"
#include "WebSocketsModule.h"

namespace UE::PixelStreaming2
{

	DECLARE_LOG_CATEGORY_EXTERN(LogEpicRtcWebsocket, Log, All);
	DEFINE_LOG_CATEGORY(LogEpicRtcWebsocket);

	FEpicRtcWebsocket::FEpicRtcWebsocket(bool bKeepAlive, TSharedPtr<IWebSocket> InWebSocket)
		: WebSocket(InWebSocket)
		, bSendKeepAlive(bKeepAlive)
	{
	}

	EpicRtcBool FEpicRtcWebsocket::Connect(EpicRtcStringView InUrl, EpicRtcWebsocketObserverInterface* InObserver)
	{
		if (WebSocket && WebSocket->IsConnected())
		{
			return false;
		}

		Observer = InObserver;
		Url = UE::PixelStreaming2::ToString(InUrl);

		if (!WebSocket)
		{
			WebSocket = FWebSocketsModule::Get().CreateWebSocket(Url, TEXT(""));
			verifyf(WebSocket, TEXT("FWebSocketsModule failed to create a valid Web Socket."));
		}

		OnConnectedHandle = WebSocket->OnConnected().AddLambda([this]() { OnConnected(); });
		OnConnectionErrorHandle = WebSocket->OnConnectionError().AddLambda([this](const FString& Error) { OnConnectionError(Error); });
		OnClosedHandle = WebSocket->OnClosed().AddLambda([this](int32 StatusCode, const FString& Reason, bool bWasClean) { OnClosed(StatusCode, Reason, bWasClean); });
		OnMessageHandle = WebSocket->OnMessage().AddLambda([this](const FString& Msg) { OnMessage(Msg); });
		OnBinaryMessageHandle = WebSocket->OnBinaryMessage().AddLambda([this](const void* Data, int32 Count, bool bIsLastFragment) { OnBinaryMessage((const uint8*)Data, Count, bIsLastFragment); });

		// Do the actual WS connection here
		WebSocket->Connect();

		return true;
	}

	void FEpicRtcWebsocket::Disconnect(const EpicRtcStringView InReason)
	{
		if (!WebSocket)
		{
			return;
		}

		WebSocket->OnConnected().Remove(OnConnectedHandle);
		WebSocket->OnConnectionError().Remove(OnConnectionErrorHandle);
		WebSocket->OnClosed().Remove(OnClosedHandle);
		WebSocket->OnMessage().Remove(OnMessageHandle);
		WebSocket->OnBinaryMessage().Remove(OnBinaryMessageHandle);

		if (WebSocket->IsConnected() && !bCloseRequested)
		{
			bCloseRequested = true;
			FString Reason;
			if (InReason._length)
			{
				Reason = ToString(InReason);
			}
			else
			{
				Reason = IsEngineExitRequested() ? TEXT("Pixel Streaming shutting down") : TEXT("Pixel Streaming closed WS under normal conditions.");
			}

			UE_LOGF(LogEpicRtcWebsocket, Log, "Closing websocket to %ls", *Url);
			WebSocket->Close(1000, Reason);

			// Because we've onbound ourselves from the existing WS message, we need to manually trigger OnClosed
			OnClosed(1000, Reason, true);
		}
	}

	void FEpicRtcWebsocket::Send(EpicRtcStringView Message)
	{
		if (!WebSocket || !WebSocket->IsConnected())
		{
			return;
		}

		FString MessageString = FString{ (int32)Message._length, Message._ptr };

		// Hijacking the offer message is a bit cheeky and should be removed once RTCP-7055 is closed.
		TSharedPtr<FJsonObject>	  JsonObject = MakeShareable(new FJsonObject);
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(MessageString);
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			FString MessageType;
			JsonObject->TryGetStringField(TEXT("type"), MessageType);

			if (MessageType == TEXT("offer"))
			{
				EScalabilityMode			 ScalabilityMode = UE::PixelStreaming2::GetEnumFromCVar<EScalabilityMode>(UPixelStreaming2PluginSettings::CVarEncoderScalabilityMode);
				FString						 ScalabilityModeString = UE::PixelStreaming2::GetCVarStringFromEnum<EScalabilityMode>(ScalabilityMode);
				TSharedRef<FJsonValueString> JsonValueObject = MakeShareable(new FJsonValueString(ScalabilityModeString));

				JsonObject->SetField(TEXT("scalabilityMode"), JsonValueObject);
				TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&MessageString);
				FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter);
			}
		}

		WebSocket->Send(MessageString);
	}

	void FEpicRtcWebsocket::OnConnected()
	{
		UE_LOGF(LogEpicRtcWebsocket, Log, "Websocket connection made to: %ls", *Url);
		bCloseRequested = false;
		LastKeepAliveCycles = FPlatformTime::Cycles64();
		Observer->OnOpen();
	}

	void FEpicRtcWebsocket::OnConnectionError(const FString& Error)
	{
		// In this case with we were already connected and got an error OR we have disabled reconnection
		UE_LOGF(LogEpicRtcWebsocket, Log, "Failed to connect to %ls - signalling server may not be up yet. Message: \"%ls\"", *Url, *Error);
		Observer->OnClosed();
	}

	void FEpicRtcWebsocket::OnClosed(int32 StatusCode, const FString& Reason, bool bWasClean)
	{
		UE_LOGF(LogEpicRtcWebsocket, Log, "Closed connection to %ls - \n\tstatus %d\n\treason: %ls\n\twas clean: %ls", *Url, StatusCode, *Reason, bWasClean ? TEXT("true") : TEXT("false"));
		Observer->OnClosed();
	}

	void FEpicRtcWebsocket::OnMessage(const FString& Msg)
	{
		// Hijacking the answer message is a bit cheeky and should be removed once RTCP-7130 is closed.
		TSharedPtr<FJsonObject>	  JsonObject = MakeShareable(new FJsonObject);
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(Msg);
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			FString MessageType;
			JsonObject->TryGetStringField(TEXT("type"), MessageType);

			if (MessageType == TEXT("answer"))
			{
				FString PlayerId;
				if (JsonObject->TryGetStringField(TEXT("playerId"), PlayerId))
				{
					int		   MinBitrate;
					int		   MaxBitrate;
					const bool bGotMinBitrate = JsonObject->TryGetNumberField(TEXT("minBitrateBps"), MinBitrate);
					const bool bGotMaxBitrate = JsonObject->TryGetNumberField(TEXT("maxBitrateBps"), MaxBitrate);

					if (bGotMinBitrate && bGotMaxBitrate && MinBitrate > 0 && MaxBitrate > 0)
					{
						IPixelStreaming2Module::Get().ForEachStreamer([PlayerId, MinBitrate, MaxBitrate](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
							Streamer->PlayerRequestsBitrate(PlayerId, MinBitrate, MaxBitrate);
						});
					}
				}
			}
		}

		FUtf8String Message(Msg);
		Observer->OnMessage(UE::PixelStreaming2::ToEpicRtcStringView(Message));
	}

	void FEpicRtcWebsocket::OnBinaryMessage(const uint8* Data, int32 Length, bool bIsLastFragment)
	{
		FUtf8String Utf8String = FUtf8String::ConstructFromPtrSize(reinterpret_cast<const char*>(Data), Length);

		FString Msg = *Utf8String;
		OnMessage(Msg);
	}

	void FEpicRtcWebsocket::Tick(float DeltaTime)
	{
		if (IsEngineExitRequested())
		{
			return;
		}

		if (bSendKeepAlive)
		{
			KeepAlive();
		}
	}

	void FEpicRtcWebsocket::KeepAlive()
	{
		if (!WebSocket)
		{
			return;
		}

		if (!WebSocket->IsConnected())
		{
			return;
		}

		float KeepAliveIntervalSeconds = UPixelStreaming2PluginSettings::CVarSignalingKeepAliveInterval.GetValueOnAnyThread();

		if (KeepAliveIntervalSeconds <= 0.0f)
		{
			return;
		}

		uint64 CyclesNow = FPlatformTime::Cycles64();
		uint64 DeltaCycles = CyclesNow - LastKeepAliveCycles;
		float  DeltaSeconds = FPlatformTime::ToSeconds(DeltaCycles);

		// If enough time has elapsed, try a keepalive
		if (DeltaSeconds >= KeepAliveIntervalSeconds)
		{
			TSharedPtr<FJsonObject> Json = MakeShared<FJsonObject>();
			const double			UnixTime = FDateTime::UtcNow().ToUnixTimestamp();
			Json->SetStringField(TEXT("type"), TEXT("ping"));
			Json->SetNumberField(TEXT("time"), UnixTime);
			WebSocket->Send(UE::PixelStreaming2::ToString(Json, false));
			LastKeepAliveCycles = CyclesNow;
		}
	}
} // namespace UE::PixelStreaming2
