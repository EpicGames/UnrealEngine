// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcWebsocketFactory.h"

namespace UE::PixelStreaming2
{

	EpicRtcErrorCode FEpicRtcWebsocketFactory::CreateWebsocket(EpicRtcWebsocketInterface** OutWebsocket)
	{
		EpicRtcWebsocketInterface* Websocket = new FEpicRtcWebsocket(bSendKeepAlive, nullptr);
		// Because the ptr was created with new, we need to call AddRef ourself (ms spec compliant)
		Websocket->AddRef();

		*OutWebsocket = Websocket;
		return EpicRtcErrorCode::Ok;
	}

} // namespace UE::PixelStreaming2
