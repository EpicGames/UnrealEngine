// Copyright Epic Games, Inc. All Rights Reserved.

#include "FreezeFrame.h"

#include "ColorConversion.h"
#include "EngineModule.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Logging.h"
#include "MediaShaders.h"
#include "Modules/ModuleManager.h"
#include "PixelCaptureBufferI420.h"
#include "PixelCaptureBufferFormat.h"
#include "PixelCaptureOutputFrameI420.h"
#include "PixelCaptureOutputFrameRHI.h"
#include "PixelStreaming2PluginSettings.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIFwd.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "TextureResource.h"
#include "UtilsCommon.h"

namespace UE::PixelStreaming2
{
	TSharedPtr<FFreezeFrame> FFreezeFrame::Create(TWeakPtr<TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>> InPlayers, TWeakPtr<FVideoCapturer> InVideoCapturer, TWeakPtr<IPixelStreaming2InputHandler> InInputHandler)
	{
		TSharedPtr<FFreezeFrame> FreezeFrame = TSharedPtr<FFreezeFrame>(new FFreezeFrame(InPlayers, InVideoCapturer, InInputHandler));
		return FreezeFrame;
	}

	FFreezeFrame::FFreezeFrame(TWeakPtr<TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>> WeakPlayers, TWeakPtr<FVideoCapturer> WeakVideoCapturer, TWeakPtr<IPixelStreaming2InputHandler> WeakInputHandler)
		: WeakPlayers(WeakPlayers)
		, VideoCapturer(WeakVideoCapturer)
		, InputHandler(WeakInputHandler)
	{
	}

	FFreezeFrame::~FFreezeFrame()
	{
		RemoveFreezeFrameBinding();
	}

	/**
	 * Add the commands to the RHI command list to copy a texture from source to dest - even if the format is different.
	 * Assumes SourceTexture is in ERHIAccess::CopySrc and DestTexture is in ERHIAccess::CopyDest
	 */
	void CopyTexture(FRHICommandList& RHICmdList, FTextureRHIRef SourceTexture, FTextureRHIRef DestTexture)
	{
		if (SourceTexture->GetDesc().Format == DestTexture->GetDesc().Format
			&& SourceTexture->GetDesc().Extent.X == DestTexture->GetDesc().Extent.X
			&& SourceTexture->GetDesc().Extent.Y == DestTexture->GetDesc().Extent.Y)
		{

			RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::CopySrc));
			RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest));

			// source and dest are the same. simple copy
			RHICmdList.CopyTexture(SourceTexture, DestTexture, {});
		}
		else
		{
			IRendererModule* RendererModule = &FModuleManager::GetModuleChecked<IRendererModule>("Renderer");

			RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

			// source and destination are different. rendered copy
			FRHIRenderPassInfo RPInfo(DestTexture, ERenderTargetActions::Load_Store);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("PixelStreaming2::CopyTexture"));
			{
				FGlobalShaderMap*		 ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
				TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

				RHICmdList.SetViewport(0, 0, 0.0f, DestTexture->GetDesc().Extent.X, DestTexture->GetDesc().Extent.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), SourceTexture);

				FIntPoint TargetBufferSize(DestTexture->GetDesc().Extent.X, DestTexture->GetDesc().Extent.Y);
				RendererModule->DrawRectangle(RHICmdList, 0, 0, // Dest X, Y
					DestTexture->GetDesc().Extent.X,			// Dest Width
					DestTexture->GetDesc().Extent.Y,			// Dest Height
					0, 0,										// Source U, V
					1, 1,										// Source USize, VSize
					TargetBufferSize,							// Target buffer size
					FIntPoint(1, 1),							// Source texture size
					VertexShader, EDRF_Default);
			}

			RHICmdList.EndRenderPass();

			RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
			RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::RTV, ERHIAccess::CopyDest));
		}
	}

	void FFreezeFrame::StartFreeze(UTexture2D* Texture)
	{
		if (Texture)
		{
			ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)
			([this, Texture](FRHICommandListImmediate& RHICmdList) {
				// A frame is supplied so immediately read its data and send as a JPEG.
				FTextureRHIRef TextureRHI = Texture->GetResource() ? Texture->GetResource()->TextureRHI : nullptr;
				if (!TextureRHI)
				{
					UE_LOGF(LogPixelStreaming2RTC, Error, "Attempting freeze frame with texture %ls with no texture RHI", *Texture->GetName());
					return;
				}
				uint32 Width = TextureRHI->GetDesc().Extent.X;
				uint32 Height = TextureRHI->GetDesc().Extent.Y;

				FRHITextureCreateDesc TextureDesc =
					FRHITextureCreateDesc::Create2D(TEXT("PixelStreaming2BlankTexture"), Width, Height, EPixelFormat::PF_B8G8R8A8)
						.SetClearValue(FClearValueBinding::None)
						.SetFlags(ETextureCreateFlags::RenderTargetable)
						.SetInitialState(ERHIAccess::Present)
						.DetermineInititialState();

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(TextureDesc);

				// Copy freeze frame texture to empty texture
				CopyTexture(RHICmdList, TextureRHI, DestTexture);

				TArray<FColor> Data;
				FIntRect	   Rect(0, 0, Width, Height);
				// This `ReadSurfaceData` makes a blocking call from CPU -> GPU -> CPU
				// WHich is how on the very next line we are able to copy the data out and send it.
				RHICmdList.ReadSurfaceData(DestTexture, Rect, Data, FReadSurfaceDataFlags());
				SendFreezeFrame(MoveTemp(Data), Rect);
			});
		}
		else
		{
			// A frame is not supplied, so we need to get it from the video input
			// at the next opportunity and send as a JPEG.
			SetupFreezeFrameCapture();
		}
	}

	void FFreezeFrame::StopFreeze()
	{
		TSharedPtr<TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>> Players = WeakPlayers.Pin();

		if (!Players)
		{
			return;
		}

		TWeakPtr<IPixelStreaming2InputHandler> WeakHandler = InputHandler;

		Players->Apply([WeakHandler](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
			if (!Participant->DataTrack)
			{
				return;
			}
			if (TSharedPtr<IPixelStreaming2InputHandler> Handler = WeakHandler.Pin())
			{
				Participant->DataTrack->SendMessage(EPixelStreaming2FromStreamerMessage::UnfreezeFrame);
			}
		});

		CachedJpegBytes.Empty();
	}

	void FFreezeFrame::SendCachedFreezeFrameTo(FString PlayerId) const
	{
		TSharedPtr<TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>> Players = WeakPlayers.Pin();

		if (!Players)
		{
			return;
		}

		if (CachedJpegBytes.Num() > 0)
		{
			if (TSharedPtr<FPlayerContext> Participant = Players->FindRef(PlayerId); Participant.IsValid())
			{
				if (!Participant->DataTrack)
				{
					return;
				}
				if (TSharedPtr<IPixelStreaming2InputHandler> Handler = InputHandler.Pin())
				{
					Participant->DataTrack->SendArbitraryData(EPixelStreaming2FromStreamerMessage::FreezeFrame, CachedJpegBytes);
				}
			}
		}
	}

	void FFreezeFrame::SendFreezeFrame(TArray<FColor> RawData, const FIntRect& Rect)
	{
		TSharedPtr<TThreadSafeMap<FString, TSharedPtr<FPlayerContext>>> Players = WeakPlayers.Pin();

		if (!Players)
		{
			return;
		}

		IImageWrapperModule&	  ImageWrapperModule = FModuleManager::GetModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::JPEG);
		bool					  bSuccess = ImageWrapper->SetRaw(RawData.GetData(), RawData.Num() * sizeof(FColor), Rect.Width(), Rect.Height(), ERGBFormat::BGRA, 8);
		if (bSuccess)
		{
			TWeakPtr<IPixelStreaming2InputHandler> WeakHandler = InputHandler;
			// Compress to a JPEG of the maximum possible quality.
			const TArray64<uint8>& JpegBytes = ImageWrapper->GetCompressed(100);
			Players->Apply([&JpegBytes, WeakHandler](FString PlayerId, TSharedPtr<FPlayerContext> Participant) {
				if (!Participant->DataTrack)
				{
					return;
				}
				if (TSharedPtr<IPixelStreaming2InputHandler> Handler = WeakHandler.Pin())
				{
					Participant->DataTrack->SendArbitraryData(EPixelStreaming2FromStreamerMessage::FreezeFrame, JpegBytes);
				}
			});
			CachedJpegBytes = JpegBytes;
		}
		else
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "JPEG image wrapper failed to accept frame data");
		}
	}

	void FFreezeFrame::SetupFreezeFrameCapture()
	{
		// Remove any existing binding
		RemoveFreezeFrameBinding();

		if (TSharedPtr<FVideoCapturer> ConcreteVideoCapturer = VideoCapturer.Pin())
		{
			OnFrameCapturedForFreezeFrameHandle = ConcreteVideoCapturer->OnFrameCaptured.AddSP(this, &FFreezeFrame::FreezeFrameCapture);
		}
	}

	void FFreezeFrame::RemoveFreezeFrameBinding()
	{
		if (!OnFrameCapturedForFreezeFrameHandle)
		{
			return;
		}

		if (TSharedPtr<FVideoCapturer> ConcreteVideoCapturer = VideoCapturer.Pin())
		{
			ConcreteVideoCapturer->OnFrameCaptured.Remove(OnFrameCapturedForFreezeFrameHandle.GetValue());
			OnFrameCapturedForFreezeFrameHandle.Reset();
		}
	}

	void FFreezeFrame::FreezeFrameCapture()
	{
		TSharedPtr<FVideoCapturer> Input = VideoCapturer.Pin();

		if (!Input)
		{
			return;
		}

		// HACK we probably should check if we are outputing a CPU texture rather than checking like this.
		EVideoCodec CurrentCodec = GetEnumFromCVar<EVideoCodec>(UPixelStreaming2PluginSettings::CVarEncoderCodec);
		if (CurrentCodec == EVideoCodec::VP8 || CurrentCodec == EVideoCodec::VP9)
		{
			// Request output format is I420 for VPX
			TSharedPtr<IPixelCaptureOutputFrame> OutputFrame = Input->WaitForFormat(PixelCaptureBufferFormat::FORMAT_I420);
			if (OutputFrame)
			{
				// Can remove binding now we have got the output in the format we need to send a FF
				RemoveFreezeFrameBinding();

				FPixelCaptureOutputFrameI420*		I420Frame = StaticCast<FPixelCaptureOutputFrameI420*>(OutputFrame.Get());
				TSharedPtr<FPixelCaptureBufferI420> I420Buffer = I420Frame->GetI420Buffer();

				const size_t   NumBytes = CalcBufferSizeArgb(I420Frame->GetWidth(), I420Frame->GetHeight());
				const uint32_t NumPixels = I420Frame->GetWidth() * I420Frame->GetHeight();
				uint8_t*	   ARGBBuffer = new uint8_t[NumBytes];

				ConvertI420ToArgb(I420Buffer->GetDataY(), I420Buffer->GetStrideY(),
					I420Buffer->GetDataU(), I420Buffer->GetStrideUV(),
					I420Buffer->GetDataV(), I420Buffer->GetStrideUV(),
					ARGBBuffer, 0,
					I420Buffer->GetWidth(), I420Buffer->GetHeight());

				// We assume FColor packing is same ordering as the Buffer are copying from
				TArray<FColor> PixelArr((FColor*)ARGBBuffer, NumPixels);
				FIntRect	   Rect(0, 0, I420Frame->GetWidth(), I420Frame->GetHeight());
				SendFreezeFrame(MoveTemp(PixelArr), Rect);
			}
		}
		else
		{
			TSharedPtr<IPixelCaptureOutputFrame> OutputFrame = Input->WaitForFormat(PixelCaptureBufferFormat::FORMAT_RHI);
			if (OutputFrame)
			{
				// Can remove binding now we have got the output in the format we need to send a FF
				RemoveFreezeFrameBinding();

				TWeakPtr<FFreezeFrame> WeakSelf = AsShared();

				ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)
				([WeakSelf, OutputFrame](FRHICommandListImmediate& RHICmdList) {
					if (auto ThisPtr = WeakSelf.Pin())
					{
						FPixelCaptureOutputFrameRHI* RHISourceFrame = StaticCast<FPixelCaptureOutputFrameRHI*>(OutputFrame.Get());

						// Read the data out of the back buffer and send as a JPEG.
						FIntRect	   Rect(0, 0, RHISourceFrame->GetWidth(), RHISourceFrame->GetHeight());
						TArray<FColor> Data;

						RHICmdList.ReadSurfaceData(RHISourceFrame->GetFrameTexture(), Rect, Data, FReadSurfaceDataFlags());
						ThisPtr->SendFreezeFrame(MoveTemp(Data), Rect);
					}
				});
			}
		}
	}

} // namespace UE::PixelStreaming2