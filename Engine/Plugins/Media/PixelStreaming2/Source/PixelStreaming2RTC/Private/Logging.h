// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPixelStreaming2RTC, Log, All);