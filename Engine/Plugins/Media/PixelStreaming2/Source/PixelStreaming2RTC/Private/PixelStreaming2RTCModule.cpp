// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2RTCModule.h"

#include "CoreMinimal.h"

#include "EpicRtcAllocator.h"
#include "EpicRtcAudioCapturer.h"
#include "EpicRtcCallstack.h"
#include "EpicRtcLogging.h"
#include "EpicRtcPlatform.h"
#include "EpicRtcTrace.h"
#include "EpicRtcVideoEncoderInitializer.h"
#include "EpicRtcVideoDecoderInitializer.h"
#include "EpicRtcWebsocketFactory.h"
#include "IMediaModule.h"
#include "IPixelStreaming2Module.h"
#include "Logging.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2RTCPlayer.h"
#include "PixelStreaming2Utils.h"
#include "RendererInterface.h"
#include "Stats.h"
#include "UtilsCommon.h"
#include "UtilsCoder.h"
#include "UtilsCore.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "WebSocketsModule.h"

namespace UE::PixelStreaming2
{
	FPixelStreaming2RTCModule* FPixelStreaming2RTCModule::PixelStreaming2Module = nullptr;

	FUtf8String FPixelStreaming2RTCModule::EpicRtcConferenceName("pixel_streaming_conference_instance");

	/**
	 * Stats logger - as turned on/off by CVarPixelStreaming2LogStats
	 */
	void ConsumeStat(FString PlayerId, FName StatName, float StatValue)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "[{0}]({1}) = {2}", PlayerId, StatName.ToString(), StatValue);
	}

#if PLATFORM_ANDROID
	/**
	 * As we use Android WebRTC through a static library as opposed to the aar (.so) typically used, we have to manually "call" these JNI methods
	 * to ensure that the symbols are exported by libUnreal.so and therefore available when libwebrtc.a tries to use them.
	 */
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_SoftwareVideoEncoderFactory_nativeCreateFactory(JNIEnv*, jclass);
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_SoftwareVideoDecoderFactory_nativeCreateFactory(JNIEnv*, jclass);
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_VideoEncoderFallback_nativeCreateEncoder(JNIEnv*, jclass, jobject, jobject);
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_VideoDecoderFallback_nativeCreateDecoder(JNIEnv*, jclass, jobject, jobject);
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_JniCommon_nativeAllocateByteBuffer(JNIEnv*, jclass, jint);
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_YuvHelper_nativeCopyPlane(JNIEnv*, jclass, jobject, jint, jobject, jint, jint, jint);
	extern "C" JNIEXPORT jlong JNICALL Java_org_webrtc_H264Utils_nativeIsSameH264Profile(JNIEnv*, jclass, jobject, jobject);

	// We must disable optimizations here to prevent the compiler from optimizing out these function calls and not linking the required symbols
	UE_DISABLE_OPTIMIZATION_SHIP
	void ForceWebRtcSymbolLink()
	{
		Java_org_webrtc_SoftwareVideoEncoderFactory_nativeCreateFactory(nullptr, nullptr);
		Java_org_webrtc_SoftwareVideoDecoderFactory_nativeCreateFactory(nullptr, nullptr);
		Java_org_webrtc_VideoEncoderFallback_nativeCreateEncoder(nullptr, nullptr, nullptr, nullptr);
		Java_org_webrtc_VideoDecoderFallback_nativeCreateDecoder(nullptr, nullptr, nullptr, nullptr);
		Java_org_webrtc_JniCommon_nativeAllocateByteBuffer(nullptr, nullptr, 0);
		Java_org_webrtc_YuvHelper_nativeCopyPlane(nullptr, nullptr, nullptr, 0, nullptr, 0, 0, 0);
		Java_org_webrtc_H264Utils_nativeIsSameH264Profile(nullptr, nullptr, nullptr, nullptr);
	}
	UE_ENABLE_OPTIMIZATION_SHIP
#endif

	/**
	 * IModuleInterface implementation
	 */
	void FPixelStreaming2RTCModule::StartupModule()
	{
#if UE_SERVER
		// Hack to no-op the rest of the module so Blueprints can still work
		return;
#else
		if (!IsStreamingSupported())
		{
			return;
		}

		if (!FSlateApplication::IsInitialized())
		{
			return;
		}

		const ERHIInterfaceType RHIType = GDynamicRHI ? RHIGetInterfaceType() : ERHIInterfaceType::Hidden;
		// Validate RHI against what PS2 supports
		if (!(
			RHIType == ERHIInterfaceType::D3D11 
			|| RHIType == ERHIInterfaceType::D3D12 
			|| RHIType == ERHIInterfaceType::Vulkan 
			|| RHIType == ERHIInterfaceType::Metal
#if PLATFORM_ANDROID
			|| RHIType == ERHIInterfaceType::OpenGL
#endif
			))
		{
	#if !WITH_DEV_AUTOMATION_TESTS
			UE_LOGF(LogPixelStreaming2RTC, Warning, "Only D3D11/D3D12/Vulkan/Metal/OpenGL(Android only) Dynamic RHI is supported. Detected %ls", GDynamicRHI != nullptr ? GDynamicRHI->GetName() : TEXT("[null]"));
	#endif
			return;
		}

		FString LogFilterString = UPixelStreaming2PluginSettings::CVarEpicRtcLogFilter.GetValueOnAnyThread() + TEXT("//\\bTicking audio (?:too|to) late\\b");
		UPixelStreaming2PluginSettings::CVarEpicRtcLogFilter->Set(*LogFilterString, ECVF_SetByHotfix);

		// By calling InitDefaultStreamer post engine init we can use pixel streaming in standalone editor mode
		IPixelStreaming2Module::Get().OnReady().AddLambda([this](IPixelStreaming2Module& CoreModule) {
	#if !PLATFORM_ANDROID
			// Android uses the EpicRtc internal coders so no need to check AVCodecs compatibility
			const EVideoCodec SelectedCodec = UE::PixelStreaming2::GetEnumFromCVar<EVideoCodec>(UPixelStreaming2PluginSettings::CVarEncoderCodec);
			if ((SelectedCodec == EVideoCodec::H264 && !IsHardwareEncoderSupported<FVideoEncoderConfigH264>())
				|| (SelectedCodec == EVideoCodec::AV1 && !IsHardwareEncoderSupported<FVideoEncoderConfigAV1>()))
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Could not setup hardware encoder. This is usually a driver issue or hardware limitation, try reinstalling your drivers.");
				UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Falling back to VP8 software video encoding.");
				UPixelStreaming2PluginSettings::CVarEncoderCodec.AsVariable()->SetWithCurrentPriority(*UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP8));
				if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
				{
					Delegates->OnFallbackToSoftwareEncoding.Broadcast();
					Delegates->OnFallbackToSoftwareEncodingNative.Broadcast();
				}
			}
	#endif

			// Need to initialize after other modules have initialized such as NVCodec.
			if (!InitializeEpicRtc())
			{
				return;
			}

			if (!ensure(GEngine != nullptr))
			{
				return;
			}

			StreamerFactory.Reset(new FRTCStreamerFactory(EpicRtcConference));

			// Ensure we have ImageWrapper loaded, used in Freezeframes
			verify(FModuleManager::Get().LoadModule(FName("ImageWrapper")));

			bModuleReady = true;
			ReadyEvent.Broadcast(*this);
		});

		FModuleManager::LoadModuleChecked<FWebSocketsModule>("WebSockets");

		// Call these to initialize their singletons
		FStats::Get();

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnLogStatsChanged.AddLambda([this](IConsoleVariable* Var) {
				bool					   bLogStats = Var->GetBool();
				UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
				if (!Delegates)
				{
					return;
				}
				if (bLogStats)
				{
					LogStatsHandle = Delegates->OnStatChangedNative.AddStatic(&ConsumeStat);
				}
				else
				{
					Delegates->OnStatChangedNative.Remove(LogStatsHandle);
				}
			});

			Delegates->OnWebRTCFpsChanged.AddLambda([](IConsoleVariable*) {
				IPixelStreaming2Module::Get().ForEachStreamer([](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
					Streamer->RefreshStreamBitrate();
				});
			});

			Delegates->OnWebRTCBitrateChanged.AddLambda([](IConsoleVariable*) {
				IPixelStreaming2Module::Get().ForEachStreamer([](TSharedPtr<IPixelStreaming2Streamer> Streamer) {
					Streamer->RefreshStreamBitrate();
				});
			});
			Delegates->OnWebRTCDisableStatsChanged.AddLambda([this](IConsoleVariable* Var) {
				TRefCountPtr<EpicRtcConferenceInterface> Conference(EpicRtcConference);
				if (Conference)
				{
					if (Var->GetBool())
					{
						Conference->DisableStats();
					}
					else
					{
						Conference->EnableStats();
					}
				}
			});
		}

		if (IMediaModule* MediaModulePtr = FModuleManager::LoadModulePtr<IMediaModule>("Media"); MediaModulePtr != nullptr)
		{
			MediaModulePtr->RegisterPlayerFactory(*this);
		}

		bStartupCompleted = true;
#endif // UE_SERVER
	}

	void FPixelStreaming2RTCModule::ShutdownModule()
	{
		if (!IsStreamingSupported())
		{
			return;
		}

		if (!bStartupCompleted)
		{
			return;
		}

		if (IMediaModule* MediaModulePtr = FModuleManager::LoadModulePtr<IMediaModule>("Media"); MediaModulePtr != nullptr)
		{
			MediaModulePtr->UnregisterPlayerFactory(*this);
		}

		TickableTasks.Reset();
		StreamerFactory.Reset();

		if (!EpicRtcPlatform)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "EpicRtcPlatform does not exist during shutdown when it is expected to exist");
		}
		else
		{
			EpicRtcPlatform->ReleaseConference(ToEpicRtcStringView(EpicRtcConferenceName));
		}

		bStartupCompleted = false;
	}

	/**
	 * End IModuleInterface implementation
	 */

	FPixelStreaming2RTCModule* FPixelStreaming2RTCModule::GetModule()
	{
		if (!PixelStreaming2Module)
		{
			PixelStreaming2Module = FModuleManager::Get().LoadModulePtr<FPixelStreaming2RTCModule>("PixelStreaming2RTC");
		}

		return PixelStreaming2Module;
	}

	/**
	 * IPixelStreaming2RTCModule implementation
	 */
	IPixelStreaming2RTCModule::FReadyEvent& FPixelStreaming2RTCModule::OnReady()
	{
		return ReadyEvent;
	}

	bool FPixelStreaming2RTCModule::IsReady()
	{
		return bModuleReady;
	}
	/**
	 * End IPixelStreaming2RTCModule implementation
	 */

	/**
	 * IMediaPlayerFactory implementation
	 */
	TSharedPtr<IMediaPlayer> FPixelStreaming2RTCModule::CreatePlayer(IMediaEventSink& EventSink)
	{
		return MakeShared<FPixelStreaming2RTCStreamPlayer>();
	}

	FName FPixelStreaming2RTCModule::GetPlayerName() const
	{
		return "PixelStreaming2RTC";
	}

	FText FPixelStreaming2RTCModule::GetDisplayName() const
	{
		return NSLOCTEXT("PixelStreaming2", "MediaPlayerFactory", "PixelStreaming2 RTC Stream Player");
	}

	bool FPixelStreaming2RTCModule::CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
	{
		return Url.StartsWith(TEXT("ws://")) || Url.StartsWith(TEXT("wss://"));
	}

	bool FPixelStreaming2RTCModule::SupportsFeature(EMediaFeature Feature) const
	{
		return Feature == EMediaFeature::VideoSamples || Feature == EMediaFeature::AudioSamples;
	}

	FGuid FPixelStreaming2RTCModule::GetPlayerPluginGUID() const
	{
		static FGuid PlayerPluginGUID = FGuid::NewGuid();
		return PlayerPluginGUID;
	}

	const TArray<FString>& FPixelStreaming2RTCModule::GetSupportedPlatforms() const
	{
		static TArray<FString> SupportedPlatforms = { TEXT("Windows"), TEXT("Linux"), TEXT("Mac") };
		return SupportedPlatforms;
	}
	/**
	 * End IMediaPlayerFactory implementation
	 */

	TSharedPtr<FSharedTickableTasks> FPixelStreaming2RTCModule::GetSharedTickableTasks()
	{
		TSharedPtr<FSharedTickableTasks> PinnedTickableTasks = TickableTasks.Pin();
		if (!PinnedTickableTasks)
		{
			PinnedTickableTasks = MakeShared<FSharedTickableTasks>(FPixelStreamingTickableTask::Create<FEpicRtcTickConferenceTask>(EpicRtcConference, TEXT("PixelStreaming2Module TickConferenceTask")));
			TickableTasks = PinnedTickableTasks;
		}
		return PinnedTickableTasks;
	}

	FString FPixelStreaming2RTCModule::GetFieldTrials()
	{
		FString FieldTrials = UPixelStreaming2PluginSettings::CVarWebRTCFieldTrials.GetValueOnAnyThread();

		// Set the WebRTC-FrameDropper/Disabled/ if the CVar is set
		if (UPixelStreaming2PluginSettings::CVarWebRTCDisableFrameDropper.GetValueOnAnyThread())
		{
			FieldTrials += TEXT("WebRTC-FrameDropper/Disabled/");
		}

		if (UPixelStreaming2PluginSettings::CVarWebRTCEnableFlexFec.GetValueOnAnyThread())
		{
			FieldTrials += TEXT("WebRTC-FlexFEC-03-Advertised/Enabled/WebRTC-FlexFEC-03/Enabled/");
		}

		// Parse "WebRTC-Video-Pacing/" field trial
		{
			float PacingFactor = UPixelStreaming2PluginSettings::CVarWebRTCVideoPacingFactor.GetValueOnAnyThread();
			float PacingMaxDelayMs = UPixelStreaming2PluginSettings::CVarWebRTCVideoPacingMaxDelay.GetValueOnAnyThread();

			if (PacingFactor >= 0.0f || PacingMaxDelayMs >= 0.0f)
			{
				FString VideoPacingFieldTrialStr = TEXT("WebRTC-Video-Pacing/");
				bool	bHasPacingFactor = PacingFactor >= 0.0f;
				if (bHasPacingFactor)
				{
					VideoPacingFieldTrialStr += FString::Printf(TEXT("factor:%.1f"), PacingFactor);
				}
				bool bHasMaxDelay = PacingMaxDelayMs >= 0.0f;
				if (bHasMaxDelay)
				{
					VideoPacingFieldTrialStr += bHasPacingFactor ? TEXT(",") : TEXT("");
					VideoPacingFieldTrialStr += FString::Printf(TEXT("max_delay:%.0f"), PacingMaxDelayMs);
				}
				VideoPacingFieldTrialStr += TEXT("/");
				FieldTrials += VideoPacingFieldTrialStr;
			}
		}

		return FieldTrials;
	}

	bool FPixelStreaming2RTCModule::InitializeEpicRtc()
	{
		EpicRtcVideoEncoderInitializers = { new FEpicRtcVideoEncoderInitializer() };
		EpicRtcVideoDecoderInitializers = { new FEpicRtcVideoDecoderInitializer() };

		EpicRtcPlatformConfig PlatformConfig{
			._memory = new FEpicRtcAllocator()
		};
#if ENABLE_EPICRTC_TRACING
		PlatformConfig._callstack = new FEpicRtcCallstack();
#endif
#if PLATFORM_ANDROID
		PlatformConfig._android = new FEpicRtcAndroidPlatform();
#endif

		EpicRtcErrorCode Result = GetOrCreatePlatform(PlatformConfig, EpicRtcPlatform.GetInitReference());
		if (Result != EpicRtcErrorCode::Ok && Result != EpicRtcErrorCode::FoundExistingPlatform)
		{
			UE_LOGF(LogPixelStreaming2RTC, Warning, "Unable to create EpicRtc Platform. GetOrCreatePlatform returned %ls", *ToString(Result));
			return false;
		}

		FUtf8String EpicRtcFieldTrials(GetFieldTrials());

		WebsocketFactory = MakeRefCount<FEpicRtcWebsocketFactory>();

		StatsCollector = MakeRefCount<FEpicRtcStatsCollector>();

		// clang-format off
		EpicRtcConfig ConferenceConfig = {
			._websocketFactory = WebsocketFactory.GetReference(),
			._signallingType = EpicRtcSignallingType::PixelStreaming,
			._signingPlugin = nullptr,
			._migrationPlugin = nullptr,
			._audioDevicePlugin = nullptr,
			._audioConfig = {
				._tickAdm = true,
				._audioEncoderInitializers = {}, // Not needed because we use the inbuilt audio codecs
				._audioDecoderInitializers = {}, // Not needed because we use the inbuilt audio codecs
				._enableBuiltInAudioCodecs = true,
			},
			._videoConfig = {
#if PLATFORM_ANDROID
				// Android uses the inbuilt video codecs as AVCodecs doesn't support hardware acceleration on Android
				._videoEncoderInitializers = {
					._ptr = nullptr,
					._size = 0
				},
				._videoDecoderInitializers = {
					._ptr = nullptr,
					._size = 0
				},
				._enableBuiltInVideoCodecs = true
#else
				._videoEncoderInitializers = {
					._ptr = const_cast<const EpicRtcVideoEncoderInitializerInterface**>(EpicRtcVideoEncoderInitializers.GetData()),
					._size = static_cast<uint64_t>(EpicRtcVideoEncoderInitializers.Num())
				},
				._videoDecoderInitializers = {
					._ptr = const_cast<const EpicRtcVideoDecoderInitializerInterface**>(EpicRtcVideoDecoderInitializers.GetData()),
					._size = static_cast<uint64_t>(EpicRtcVideoDecoderInitializers.Num())
				},
				._enableBuiltInVideoCodecs = false
#endif
			},
			._fieldTrials = {
				._fieldTrials = ToEpicRtcStringView(EpicRtcFieldTrials),
				._isGlobal = 0
			},
			._logging = {
				._logger = new FEpicRtcLogsRedirector(MakeShared<FEpicRtcLogFilter>()),
#if !NO_LOGGING // When building WITH_SHIPPING by default .GetVerbosity() does not exist
				._level = UnrealLogToEpicRtcCategoryMap[LogPixelStreaming2EpicRtc.GetVerbosity()],
				._levelWebRtc = UnrealLogToEpicRtcCategoryMap[LogPixelStreaming2WebRtc.GetVerbosity()]
#endif
			},
			._stats = {
				._statsCollectorCallback = StatsCollector.GetReference(),
				._statsCollectorInterval = 1000,
				._jsonFormatOnly = false
			}
		};
		// clang-format on

		Result = EpicRtcPlatform->CreateConference(ToEpicRtcStringView(EpicRtcConferenceName), ConferenceConfig, EpicRtcConference.GetInitReference());
		if (Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGF(LogPixelStreaming2RTC, Warning, "Unable to create EpicRtc Conference: CreateConference returned %ls", *ToString(Result));
			return false;
		}

		return true;
	}

	/**
	 * End own methods
	 */
} // namespace UE::PixelStreaming2

IMPLEMENT_MODULE(UE::PixelStreaming2::FPixelStreaming2RTCModule, PixelStreaming2RTC)
