// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcConferenceUtils.h"
#include "EpicRtcStatsCollector.h"
#include "EpicRtcStreamer.h"
#include "IMediaPlayerFactory.h"
#include "IPixelStreaming2RTCModule.h"

#include "epic_rtc/core/platform.h"
#include "epic_rtc/plugins/signalling/signalling_type.h"

namespace UE::PixelStreaming2
{
	class FEpicRtcWebsocketFactory;

	/**
	 * This plugin allows the back buffer to be sent as a compressed video across a network.
	 */
	class FPixelStreaming2RTCModule : public IPixelStreaming2RTCModule, public IMediaPlayerFactory
	{
	public:
		static FPixelStreaming2RTCModule* GetModule();

		virtual ~FPixelStreaming2RTCModule() = default;

		// Begin IPixelStreaming2RTCModule
		virtual FReadyEvent& OnReady() override;
		virtual bool		 IsReady() override;
		// End IPixelStreaming2RTCModule

		// Begin IMediaPlayerFactory
		virtual TSharedPtr<IMediaPlayer> CreatePlayer(IMediaEventSink& EventSink) override;
		virtual FName					 GetPlayerName() const override;
		virtual FText					 GetDisplayName() const override;
		virtual bool					 CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
		virtual bool					 SupportsFeature(EMediaFeature Feature) const override;
		virtual FGuid					 GetPlayerPluginGUID() const override;
		virtual const TArray<FString>&	 GetSupportedPlatforms() const override;
		// End IMediaPlayerFactory

		TSharedPtr<FSharedTickableTasks>		  GetSharedTickableTasks();
		TRefCountPtr<EpicRtcConferenceInterface>& GetEpicRtcConference() { return EpicRtcConference; }
		TRefCountPtr<FEpicRtcStatsCollector>&	  GetStatsCollector() { return StatsCollector; }

	private:
		// Begin IModuleInterface
		void StartupModule() override;
		void ShutdownModule() override;
		// End IModuleInterface

		FString GetFieldTrials();
		bool	InitializeEpicRtc();

	private:
		bool							  bModuleReady = false;
		bool							  bStartupCompleted = false;
		static FPixelStreaming2RTCModule* PixelStreaming2Module;

		FReadyEvent		ReadyEvent;
		FDelegateHandle LogStatsHandle;

	private:
		TRefCountPtr<EpicRtcPlatformInterface>	 EpicRtcPlatform;
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
		TRefCountPtr<FEpicRtcStatsCollector>	 StatsCollector;

		TRefCountPtr<FEpicRtcWebsocketFactory> WebsocketFactory;
		TWeakPtr<FSharedTickableTasks>		   TickableTasks;

		TArray<EpicRtcVideoEncoderInitializerInterface*> EpicRtcVideoEncoderInitializers;
		TArray<EpicRtcVideoDecoderInitializerInterface*> EpicRtcVideoDecoderInitializers;

		static FUtf8String EpicRtcConferenceName;

		TUniquePtr<FRTCStreamerFactory> StreamerFactory;
	};
} // namespace UE::PixelStreaming2
