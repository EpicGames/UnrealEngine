// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2RTCPlayer.h"

#include "GenericPlatform/GenericPlatformHttp.h"
#include "IMediaAudioSample.h"
#include "IMediaTextureSample.h"
#include "MediaObjectPool.h"
#include "Logging.h"
#include "UtilsString.h"
#include "PixelStreaming2RTCModule.h"
#include "Templates/PointerVariants.h"

namespace UE::PixelStreaming2
{
	class FPixelStreaming2TextureSample : public IMediaTextureSample, public IMediaPoolable
	{
	public:
		FPixelStreaming2TextureSample()
			: Time(FTimespan::Zero())
		{
		}

		void Initialize(TRefCountPtr<FRHITexture> InTexture, FIntPoint InDisplaySize, FIntPoint InTotalSize, FTimespan InTime, FTimespan InDuration)
		{
			Texture = InTexture;
			Time = InTime;
			DisplaySize = InDisplaySize;
			TotalSize = InTotalSize;
			Duration = InDuration;
		}

		//~ IMediaTextureSample interface
		virtual const void* GetBuffer() override
		{
			return nullptr;
		}

		virtual FIntPoint GetDim() const override
		{
			return TotalSize;
		}

		virtual FTimespan GetDuration() const override
		{
			return Duration;
		}

		virtual EMediaTextureSampleFormat GetFormat() const override
		{
			return EMediaTextureSampleFormat::CharBGRA;
		}

		virtual FIntPoint GetOutputDim() const override
		{
			return DisplaySize;
		}

		virtual uint32 GetStride() const override
		{
			if (!Texture.IsValid())
			{
				return 0;
			}
			return Texture->GetSizeX();
		}

		virtual FRHITexture* GetTexture() const override
		{
			return Texture.GetReference();
		}

		virtual FMediaTimeStamp GetTime() const override
		{
			return FMediaTimeStamp(Time);
		}

		virtual bool IsCacheable() const override
		{
			return true;
		}

		virtual bool IsOutputSrgb() const override
		{
			return true;
		}

	public:
		//~ IMediaPoolable interface
		virtual void ShutdownPoolable() override
		{
			// Drop reference to the texture. It should be released by the outside system.
			Texture = nullptr;
			Time = FTimespan::Zero();
		}

	private:
		TRefCountPtr<FRHITexture> Texture;
		FTimespan				  Time;
		FTimespan				  Duration;
		FIntPoint				  TotalSize;
		FIntPoint				  DisplaySize;
	};

	class FPixelStreaming2TextureSamplePool : public TMediaObjectPool<FPixelStreaming2TextureSample>
	{
	};

	class FPixelStreaming2AudioSample : public IMediaAudioSample, public IMediaPoolable
	{
	public:
		FPixelStreaming2AudioSample()
			: Channels(0)
			, Duration(FTimespan::Zero())
			, SampleRate(0)
			, Time(FTimespan::Zero())
		{
		}
		virtual ~FPixelStreaming2AudioSample() = default;

		void Initialize(
			const uint8* InBuffer,
			uint32		 InSize,
			uint32		 InChannels,
			uint32		 InSampleRate,
			FTimespan	 InTime,
			FTimespan	 InDuration)
		{
			check(InBuffer && InSize > 0);

			Buffer.Reset(InSize);
			Buffer.Append(InBuffer, InSize);

			Channels = InChannels;
			Duration = InDuration;
			SampleRate = InSampleRate;
			Time = InTime;
		}

		// Begin IMediaAudioSample
		virtual const void* GetBuffer() override
		{
			return Buffer.GetData();
		}

		virtual uint32 GetChannels() const override
		{
			return Channels;
		}

		virtual FTimespan GetDuration() const override
		{
			return Duration;
		}

		virtual EMediaAudioSampleFormat GetFormat() const override
		{
			return EMediaAudioSampleFormat::Int16;
		}

		virtual uint32 GetFrames() const override
		{
			return Buffer.Num() / (Channels * sizeof(int16));
		}

		virtual uint32 GetSampleRate() const override
		{
			return SampleRate;
		}

		virtual FMediaTimeStamp GetTime() const override
		{
			return FMediaTimeStamp(Time);
		}

		const TArray<uint8>& GetDataBuffer() const
		{
			return Buffer;
		}
		// End IMediaAudioSample

	private:
		/** The sample's data buffer. */
		TArray<uint8> Buffer;

		/** Number of audio channels. */
		uint32 Channels;

		/** The duration for which the sample is valid. */
		FTimespan Duration;

		/** Audio sample rate (in samples per second). */
		uint32 SampleRate;

		/** Presentation time for which the sample was generated. */
		FTimespan Time;
	};

	class FPixelStreaming2AudioSamplePool : public TMediaObjectPool<FPixelStreaming2AudioSample>
	{
	};

	uint32_t FPixelStreaming2RTCStreamPlayer::PlayerId = 0;

	FPixelStreaming2RTCStreamPlayer::FPixelStreaming2RTCStreamPlayer()
		: SelectedAudioTrack(INDEX_NONE)
		, SelectedVideoTrack(INDEX_NONE)
		, Samples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
		, AudioSamplePool(new FPixelStreaming2AudioSamplePool)
		, VideoSamplePool(new FPixelStreaming2TextureSamplePool)
	{
		PlayerName = FString::Printf(TEXT("FPixelStreaming2RTCStreamPlayer%d"), PlayerId++);
		OnStatsReadyHandle = FPixelStreaming2RTCModule::GetModule()->GetStatsCollector()->OnStatsReady.AddRaw(this, &FPixelStreaming2RTCStreamPlayer::OnStatsReady);
	}

	FPixelStreaming2RTCStreamPlayer::~FPixelStreaming2RTCStreamPlayer()
	{
		FPixelStreaming2RTCModule::GetModule()->GetStatsCollector()->OnStatsReady.Remove(OnStatsReadyHandle);
	}

	bool FPixelStreaming2RTCStreamPlayer::Open(const FString& Url, const IMediaOptions* Options)
	{
		StreamURL = Url;

		TOptional<FString> UrlStreamerId = FGenericPlatformHttp::GetUrlParameter(Url, TEXT("StreamerId"));
		if (UrlStreamerId.IsSet())
		{
			TargetStreamerId = *UrlStreamerId;
		}

		TSharedPtr<FPixelStreaming2RTCStreamPlayer> Player = AsShared();
		if (!SessionObserver)
		{
			SessionObserver = MakeRefCount<FEpicRtcSessionObserver>(TObserver<IPixelStreaming2SessionObserver>(Player));
		}

		if (!RoomObserver)
		{
			RoomObserver = MakeRefCount<FEpicRtcRoomObserver>(TObserver<IPixelStreaming2RoomObserver>(Player));
		}

		if (!AudioTrackObserverFactory)
		{
			AudioTrackObserverFactory = MakeRefCount<FEpicRtcAudioTrackObserverFactory>(TObserver<IPixelStreaming2AudioTrackObserver>(Player));
		}

		if (!VideoTrackObserverFactory)
		{
			VideoTrackObserverFactory = MakeRefCount<FEpicRtcVideoTrackObserverFactory>(TObserver<IPixelStreaming2VideoTrackObserver>(Player));
		}

		if (!DataTrackObserverFactory)
		{
			DataTrackObserverFactory = MakeRefCount<FEpicRtcDataTrackObserverFactory>(TObserver<IPixelStreaming2DataTrackObserver>(Player));
		}

		if (!EpicRtcConference)
		{
			EpicRtcConference = FPixelStreaming2RTCModule::GetModule()->GetEpicRtcConference();
		}

		FUtf8String Utf8PlayerName = *PlayerName;
		FUtf8String Utf8Url = *StreamURL;
		FUtf8String Utf8ConnectionUrl = Utf8Url + (Utf8Url.Contains(TEXT("?")) ? TEXT("&") : TEXT("?")) + TEXT("isStreamer=false");

		EpicRtcSessionConfig SessionConfig = {
			._id = { ._ptr = reinterpret_cast<const char*>(*Utf8PlayerName), ._length = static_cast<uint64_t>(Utf8PlayerName.Len()) },
			._url = { ._ptr = reinterpret_cast<const char*>(*Utf8ConnectionUrl), ._length = static_cast<uint64_t>(Utf8ConnectionUrl.Len()) },
			._observer = SessionObserver.GetReference()
		};

		if (TRefCountPtr<EpicRtcConferenceInterface> SafeConference = EpicRtcConference; SafeConference)
		{
			if (const EpicRtcErrorCode Result = SafeConference->CreateSession(SessionConfig, EpicRtcSession.GetInitReference()); Result != EpicRtcErrorCode::Ok)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create EpicRtc session, CreateSession returned: {0}", ToString(Result));
				return false;
			}

			if (const EpicRtcErrorCode Result = EpicRtcSession->Connect(); Result != EpicRtcErrorCode::Ok)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to connect EpicRtcSession. Connect returned: {0}", ToString(Result));
				return false;
			}
		}
		else
		{
			return false;
		}

		return true;
	}

	bool FPixelStreaming2RTCStreamPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options)
	{
		// TODO! We don't support this Open method
		return false;
	}

	void FPixelStreaming2RTCStreamPlayer::Close()
	{
		TRefCountPtr<EpicRtcSessionInterface> SafeSession = EpicRtcSession;
		if (!SafeSession)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to close. EpicRtcSession does not exist!");
			return;
		}

		// NOTE: It is imperative we null out the video sink before we remove room and session. If the sink is still alive
		// during session destruction, webrtc will flush incoming frames (leading to OnVideoTrackFrame) and the engine will
		// lock inside AVCodecs on a RHI fence. With the sink nulled, OnVideoTrackFrame will early exit
		if (VideoSink)
		{
			if (TSharedPtr<FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
			{
				SafeVideoSink->RemoveVideoConsumer(TWeakPtrVariant<IPixelStreaming2VideoConsumer>(this));
			}

			VideoSink = nullptr;
			RemoteVideoTrack = 0;
		}

		if (AudioSink)
		{
			if (TSharedPtr<FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
			{
				SafeAudioSink->RemoveAudioConsumer(TWeakPtrVariant<IPixelStreaming2AudioConsumer>(this));
			}

			AudioSink = nullptr;
			RemoteAudioTrack = 0;
		}

		if (TRefCountPtr<EpicRtcRoomInterface> SafeRoom = EpicRtcRoom; SafeRoom)
		{
			FUtf8String Utf8TargetStreamerId = *TargetStreamerId;

			SafeRoom->Leave();

			SafeSession->RemoveRoom({ ._ptr = reinterpret_cast<const char*>(*Utf8TargetStreamerId), ._length = static_cast<uint64_t>(Utf8TargetStreamerId.Len()) });
		}

		FUtf8String Reason = TEXT("FPixelStreaming2RTCStreamPlayer::Close");
		if (const EpicRtcErrorCode Result = SafeSession->Disconnect(ToEpicRtcStringView(Reason)); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to disconnect EpicRtcSession. Disconnect returned {0}", ToString(Result));
			return;
		}

		if (TRefCountPtr<EpicRtcConferenceInterface> SafeConference = EpicRtcConference; SafeConference)
		{
			FUtf8String Utf8PlayerName = *PlayerName;
			EpicRtcConference->RemoveSession({ ._ptr = reinterpret_cast<const char*>(*Utf8PlayerName), ._length = static_cast<uint64_t>(Utf8PlayerName.Len()) });
		}
	}

	FString FPixelStreaming2RTCStreamPlayer::GetUrl() const
	{
		return StreamURL;
	}

	IMediaSamples& FPixelStreaming2RTCStreamPlayer::GetSamples()
	{
		return *Samples.Get();
	}

	IMediaTracks& FPixelStreaming2RTCStreamPlayer::GetTracks()
	{
		return *this;
	}

	IMediaCache& FPixelStreaming2RTCStreamPlayer::GetCache()
	{
		return *this;
	}

	IMediaControls& FPixelStreaming2RTCStreamPlayer::GetControls()
	{
		return *this;
	}

	FString FPixelStreaming2RTCStreamPlayer::GetInfo() const
	{
		return TEXT("FPixelStreaming2RTCStreamPlayer information not implemented yet");
	}

	FGuid FPixelStreaming2RTCStreamPlayer::GetPlayerPluginGUID() const
	{
		FPixelStreaming2RTCModule* ModulePtr = FPixelStreaming2RTCModule::GetModule();
		check(ModulePtr);

		return ModulePtr->GetPlayerPluginGUID();
	}

	FString FPixelStreaming2RTCStreamPlayer::GetStats() const
	{
		return TEXT("FPixelStreaming2RTCStreamPlayer stats information not implemented yet");
	}

	IMediaView& FPixelStreaming2RTCStreamPlayer::GetView()
	{
		return *this;
	}

	void FPixelStreaming2RTCStreamPlayer::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnSessionStateUpdate(const EpicRtcSessionState StateUpdate)
	{
		switch (StateUpdate)
		{
			case EpicRtcSessionState::New:
			case EpicRtcSessionState::Pending:
			case EpicRtcSessionState::Connected:
			case EpicRtcSessionState::Disconnected:
			case EpicRtcSessionState::Failed:
			case EpicRtcSessionState::Exiting:
				SessionState = StateUpdate;
				break;
			default:
				UE_LOGFMT(LogPixelStreaming2RTC, Warning, "OnSessionStateUpdate received unknown EpicRtcSessionState: {0}", static_cast<int>(StateUpdate));
				break;
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList)
	{
		TArray<FString> Streamers;
		for (uint64_t i = 0; i < RoomsList->Size(); i++)
		{
			EpicRtcStringInterface* RoomName = RoomsList->Get()[i];
			Streamers.Add(FString{ RoomName->Get(), static_cast<int32>(RoomName->Length()) });
		}

		if (Streamers.IsEmpty())
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Streamer list empty!");
			return;
		}

		// No specific streamer passed in so subscribe to the first one
		if (TargetStreamerId.IsEmpty())
		{
			TargetStreamerId = Streamers[0];
		}
		else
		{
			if (!Streamers.Contains(TargetStreamerId))
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Streamer list [{0}] doesn't contain target streamer [{1}]!", FString::Join(Streamers, TEXT(",")), TargetStreamerId);
				return;
			}
		}

		if (SessionState != EpicRtcSessionState::Connected)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create subscribe to streamer. EpicRtc session isn't connected");
			return;
		}

		EpicRtcConnectionConfig ConnectionConfig = {
			// TODO (Migration): RTCP-7032 This info usually comes from the OnConfig signalling message
			._iceServers = EpicRtcIceServerSpan{ ._ptr = nullptr, ._size = 0 },
			._iceConnectionPolicy = EpicRtcIcePolicy::All,
			._disableTcpCandidates = false
		};

		FUtf8String		  Utf8TargetStreamerId = *TargetStreamerId;
		EpicRtcRoomConfig RoomConfig = {
			._id = EpicRtcStringView{ ._ptr = reinterpret_cast<const char*>(*Utf8TargetStreamerId), ._length = static_cast<uint64_t>(Utf8TargetStreamerId.Len()) },
			._connectionConfig = ConnectionConfig,
			._ticket = EpicRtcStringView{ ._ptr = nullptr, ._length = 0 },
			._joinParameters = EpicRtcSignallingJoinParameters{
				._options = nullptr,
				._reservedAudioStreamsMaxPoolSize = 0,
				._audioOnly = false
			},
			._observer = RoomObserver,
			._audioTrackObserverFactory = AudioTrackObserverFactory,
			._dataTrackObserverFactory = DataTrackObserverFactory,
			._videoTrackObserverFactory = VideoTrackObserverFactory
		};

		if (const EpicRtcErrorCode Result = EpicRtcSession->CreateRoom(RoomConfig, EpicRtcRoom.GetInitReference()); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create EpicRtc room. CreateRoom returned: {0}", ToString(Result));
			return;
		}

		EpicRtcRoom->Join();

		// Create a stats collector so we can receive stats from the subscribed streamer
		StatsCollector = FRTCStatsCollector::Create(*Utf8TargetStreamerId);
	}
	void FPixelStreaming2RTCStreamPlayer::OnRoomStateUpdate(const EpicRtcRoomState State)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant)
	{
		FString ParticipantId = ToString(Participant->GetId());
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "Player ({0}) joined", ParticipantId);
	}

	void FPixelStreaming2RTCStreamPlayer::OnRoomLeftUpdate(const EpicRtcStringView ParticipantId)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* InAudioTrack)
	{
		FString ParticipantId = ToString(Participant->GetId());
		FString AudioTrackId = ToString(InAudioTrack->GetId());

		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FPixelStreaming2RTCStreamPlayer::OnAudioTrackUpdate(Participant [{0}], AudioTrack [{1}])", ParticipantId, AudioTrackId);
		if (InAudioTrack->IsRemote())
		{
			// We received a remote track. We should now create a sink to handle receiving the frames.
			// NOTE: We pass in nullptr as the track because if we store the track on the sink, EpicRtc will be unable to destroy it
			// and webrtc will try to flush remaining frames during session removal.
			AudioSink = FEpicRtcAudioSink::Create(nullptr);
			RemoteAudioTrack = reinterpret_cast<uintptr_t>(InAudioTrack); // Keep track of which remote track we're receiving audio from as we only support display one track
			// As EpicRtc work can happen on different threads, we always need to call functions on a copy of the AudioSink to ensure it doesn't get deleted out from underneath us
			if (TSharedPtr<FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
			{
				SafeAudioSink->AddAudioConsumer(TWeakPtrVariant<IPixelStreaming2AudioConsumer>(this));
			}
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* InVideoTrack)
	{

		FString ParticipantId = ToString(Participant->GetId());
		FString VideoTrackId = ToString(InVideoTrack->GetId());

		if (VideoTrackId == TEXT("probator"))
		{
			// probator is fake sfu news
			return;
		}

		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FPixelStreaming2RTCStreamPlayer::OnVideoTrackUpdate(Participant [{0}], VideoTrack [{1}])", ParticipantId, VideoTrackId);
		if (InVideoTrack->IsRemote())
		{
			// We received a remote track. We should now create a sink to handle receiving the frames.
			// NOTE: We pass in nullptr as the track because if we store the track on the sink, EpicRtc will be unable to destroy it
			// and webrtc will try to flush remaining frames during session removal.
			VideoSink = FEpicRtcVideoSink::Create(nullptr);
			RemoteVideoTrack = reinterpret_cast<uintptr_t>(InVideoTrack); // Keep track of which remote track we're receiving video from as we only support display one track
			// As EpicRtc work can happen on different threads, we always need to call functions on a copy of the VideoSink to ensure it doesn't get deleted out from underneath us
			if (TSharedPtr<FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
			{
				SafeVideoSink->AddVideoConsumer(TWeakPtrVariant<IPixelStreaming2VideoConsumer>(this));
			}
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack)
	{
	}

	EpicRtcSdpInterface* FPixelStreaming2RTCStreamPlayer::OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		return Sdp;
	}

	EpicRtcSdpInterface* FPixelStreaming2RTCStreamPlayer::OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		return Sdp;
	}

	void FPixelStreaming2RTCStreamPlayer::OnRoomErrorUpdate(const EpicRtcErrorCode Error)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnAudioTrackMuted(EpicRtcAudioTrackInterface* InAudioTrack, EpicRtcBool bIsMuted)
	{
		if (!AudioSink || RemoteAudioTrack != reinterpret_cast<uintptr_t>(InAudioTrack))
		{
			return;
		}

		if (TSharedPtr<FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
		{
			SafeAudioSink->SetMuted(static_cast<bool>(bIsMuted));
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnAudioTrackFrame(EpicRtcAudioTrackInterface* InAudioTrack, const EpicRtcAudioFrame& Frame)
	{
		if (!AudioSink || RemoteAudioTrack != reinterpret_cast<uintptr_t>(InAudioTrack))
		{
			return;
		}

		if (TSharedPtr<FEpicRtcAudioSink> SafeAudioSink = AudioSink; SafeAudioSink)
		{
			SafeAudioSink->OnAudioData(Frame._data, Frame._length, Frame._format._numChannels, Frame._format._sampleRate);
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnAudioTrackState(EpicRtcAudioTrackInterface* InAudioTrack, const EpicRtcTrackState State)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoTrackMuted(EpicRtcVideoTrackInterface* InVideoTrack, EpicRtcBool bIsMuted)
	{
		if (!VideoSink || RemoteVideoTrack != reinterpret_cast<uintptr_t>(InVideoTrack))
		{
			return;
		}

		if (TSharedPtr<FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
		{
			SafeVideoSink->SetMuted(static_cast<bool>(bIsMuted));
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoTrackFrame(EpicRtcVideoTrackInterface* InVideoTrack, const EpicRtcVideoFrame& Frame)
	{
		if (!VideoSink || RemoteVideoTrack != reinterpret_cast<uintptr_t>(InVideoTrack))
		{
			return;
		}

		if (TSharedPtr<FEpicRtcVideoSink> SafeVideoSink = VideoSink; SafeVideoSink)
		{
			SafeVideoSink->OnEpicRtcFrame(Frame);
		}
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoTrackState(EpicRtcVideoTrackInterface* InVideoTrack, const EpicRtcTrackState State)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoTrackEncodedFrame(EpicRtcStringView InParticipantId, EpicRtcVideoTrackInterface* InVideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame)
	{
	}

	EpicRtcBool FPixelStreaming2RTCStreamPlayer::Enabled() const
	{
		return true;
	}

	void FPixelStreaming2RTCStreamPlayer::OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error)
	{
	}

	void FPixelStreaming2RTCStreamPlayer::ConsumeFrame(FTextureRHIRef Frame)
	{
		TSharedRef<FPixelStreaming2TextureSample, ESPMode::ThreadSafe> VideoSample = VideoSamplePool->AcquireShared();
		// Fake timestamp so we always display the frame
		VideoSample->Initialize(Frame, Frame->GetSizeXY(), Frame->GetSizeXY(), FTimespan::Zero(), FTimespan(1));

		Samples->AddVideo(VideoSample);
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoConsumerAdded()
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnVideoConsumerRemoved()
	{
	}

	void FPixelStreaming2RTCStreamPlayer::ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames)
	{
		if (AudioData == nullptr || SampleRate <= 0 || NumChannels == 0 || NumFrames == 0)
		{
			return;
		}

		TSharedRef<FPixelStreaming2AudioSample, ESPMode::ThreadSafe> AudioSample = AudioSamplePool->AcquireShared();

		TArray<uint8> AudioBuffer;

		size_t NumSamples = NumFrames * NumChannels;
		size_t DataSize = NumSamples * sizeof(int16);
		AudioBuffer.SetNumUninitialized(DataSize);

		FMemory::Memcpy(AudioBuffer.GetData(), AudioData, DataSize);

		FTimespan Duration = (NumSamples * ETimespan::TicksPerSecond) / SampleRate;
		AudioSample->Initialize(AudioBuffer.GetData(), DataSize, NumChannels, SampleRate, FTimespan::Zero(), Duration);

		Samples->AddAudio(AudioSample);
	}

	void FPixelStreaming2RTCStreamPlayer::OnAudioConsumerAdded()
	{
	}

	void FPixelStreaming2RTCStreamPlayer::OnAudioConsumerRemoved()
	{
	}

	bool FPixelStreaming2RTCStreamPlayer::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
	{
		return false;
	}

	int32 FPixelStreaming2RTCStreamPlayer::GetNumTracks(EMediaTrackType TrackType) const
	{
		// We support only video and audio tracks
		return (TrackType == EMediaTrackType::Audio || TrackType == EMediaTrackType::Video) ? 1 : 0;
	}

	int32 FPixelStreaming2RTCStreamPlayer::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
	{
		return 1;
	}

	int32 FPixelStreaming2RTCStreamPlayer::GetSelectedTrack(EMediaTrackType TrackType) const
	{
		if (TrackType == EMediaTrackType::Audio)
		{
			return SelectedAudioTrack;
		}
		else if (TrackType == EMediaTrackType::Video)
		{
			return SelectedVideoTrack;
		}
		else
		{
			return INDEX_NONE;
		}
	}

	FText FPixelStreaming2RTCStreamPlayer::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
	{
		if (TrackType == EMediaTrackType::Audio)
		{
			return FText::FromString(TEXT("PixelStreaming2Audio"));
		}
		else if (TrackType == EMediaTrackType::Video)
		{
			return FText::FromString(TEXT("PixelStreaming2Video"));
		}
		else
		{
			return FText::FromString(TEXT("Unknown"));
		}
	}

	int32 FPixelStreaming2RTCStreamPlayer::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
	{
		return 0;
	}

	FString FPixelStreaming2RTCStreamPlayer::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
	{
		static FString Language(TEXT("Default"));
		return Language;
	}

	FString FPixelStreaming2RTCStreamPlayer::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
	{
		if (TrackType == EMediaTrackType::Audio)
		{
			return TEXT("PixelStreaming2Audio");
		}
		else if (TrackType == EMediaTrackType::Video)
		{
			return TEXT("PixelStreaming2Video");
		}
		else
		{
			return TEXT("Unknown");
		}
	}

	bool FPixelStreaming2RTCStreamPlayer::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
	{
		return false;
	}

	bool FPixelStreaming2RTCStreamPlayer::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
	{
		bool bHasChanged = false;

		if (TrackType == EMediaTrackType::Audio)
		{
			bHasChanged = SelectedAudioTrack != TrackIndex;
			SelectedAudioTrack = TrackIndex;
		}
		else if (TrackType == EMediaTrackType::Video)
		{
			bHasChanged = SelectedVideoTrack != TrackIndex;
			SelectedVideoTrack = TrackIndex;
		}

		return true;
	}

	bool FPixelStreaming2RTCStreamPlayer::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
	{
		if (FormatIndex == 0)
		{
			return true;
		}
		else
		{
			// We only support one track format
			return false;
		}
	}

	bool FPixelStreaming2RTCStreamPlayer::CanControl(EMediaControl Control) const
	{
		// We don't support any media controls
		return false;
	}

	FTimespan FPixelStreaming2RTCStreamPlayer::GetDuration() const
	{
		return FTimespan::Zero();
	}

	float FPixelStreaming2RTCStreamPlayer::GetRate() const
	{
		// We only support 1x playback
		return 1.f;
	}

	EMediaState FPixelStreaming2RTCStreamPlayer::GetState() const
	{
		// As we don't support media controls, we're always playing
		return EMediaState::Playing;
	}

	EMediaStatus FPixelStreaming2RTCStreamPlayer::GetStatus() const
	{
		return EMediaStatus::None;
	}

	TRangeSet<float> FPixelStreaming2RTCStreamPlayer::GetSupportedRates(EMediaRateThinning Thinning) const
	{
		TRangeSet<float> Result;
		// We only support 1x playback
		Result.Add(TRange<float>(1.0f));
		return Result;
	}

	FTimespan FPixelStreaming2RTCStreamPlayer::GetTime() const
	{
		return FTimespan(FPlatformTime::Cycles64());
	}

	bool FPixelStreaming2RTCStreamPlayer::IsLooping() const
	{
		return false;
	}

	bool FPixelStreaming2RTCStreamPlayer::Seek(const FTimespan& Time)
	{
		// We don't support seeking of the stream
		return false;
	}

	bool FPixelStreaming2RTCStreamPlayer::SetLooping(bool Looping)
	{
		return false;
	}

	bool FPixelStreaming2RTCStreamPlayer::SetRate(float Rate)
	{
		// We don't support setting any rates of the stream but must return true to succeed with initialization
		return Rate == 1.f;
	}

	void FPixelStreaming2RTCStreamPlayer::OnStatsReady(const FString& PeerId, const EpicRtcConnectionStats& ConnectionStats)
	{
		if (!StatsCollector)
		{
			return;
		}

		if (PeerId != TargetStreamerId)
		{
			return;
		}

		StatsCollector->Process(ConnectionStats);
	}
} // namespace UE::PixelStreaming2