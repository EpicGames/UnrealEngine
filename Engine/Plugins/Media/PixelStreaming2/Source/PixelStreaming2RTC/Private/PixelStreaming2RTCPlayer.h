// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "Containers/UnrealString.h"
#include "EpicRtcAudioSink.h"
#include "EpicRtcAudioTrackObserver.h"
#include "EpicRtcAudioTrackObserverFactory.h"
#include "EpicRtcDataTrackObserver.h"
#include "EpicRtcDataTrackObserverFactory.h"
#include "EpicRtcVideoSink.h"
#include "EpicRtcVideoTrackObserver.h"
#include "EpicRtcVideoTrackObserverFactory.h"
#include "EpicRtcRoomObserver.h"
#include "EpicRtcSessionObserver.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaOptions.h"
#include "IMediaPlayer.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "IPixelStreaming2AudioConsumer.h"
#include "IPixelStreaming2VideoConsumer.h"
#include "MediaSamples.h"
#include "MediaSource.h"
#include "RTCStatsCollector.h"

#include "epic_rtc/core/conference.h"
#include "epic_rtc/core/room.h"
#include "epic_rtc/core/session.h"

namespace UE::PixelStreaming2
{
	class FPixelStreaming2RTCStreamPlayer :
		public IMediaPlayer,
		public IMediaTracks,
		public IMediaCache,
		public IMediaControls,
		public IMediaView,
		public IPixelStreaming2SessionObserver,
		public IPixelStreaming2RoomObserver,
		public IPixelStreaming2AudioTrackObserver,
		public IPixelStreaming2DataTrackObserver,
		public IPixelStreaming2VideoTrackObserver,
		public IPixelStreaming2AudioConsumer,
		public IPixelStreaming2VideoConsumer,
		public TSharedFromThis<FPixelStreaming2RTCStreamPlayer>
	{
	public:
		FPixelStreaming2RTCStreamPlayer();
		virtual ~FPixelStreaming2RTCStreamPlayer();

		// Begin IMediaPlayer
		virtual bool			Open(const FString& Url, const IMediaOptions* Options) override;
		virtual bool			Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
		virtual void			Close() override;
		virtual FString			GetUrl() const override;
		virtual IMediaSamples&	GetSamples() override;
		virtual IMediaTracks&	GetTracks() override;
		virtual IMediaCache&	GetCache() override;
		virtual IMediaControls& GetControls() override;
		virtual FString			GetInfo() const override;
		virtual FGuid			GetPlayerPluginGUID() const override;
		virtual FString			GetStats() const override;
		virtual IMediaView&		GetView() override;
		virtual void			TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
		// End IMediaPlayer

		// Begin IMediaTracks
		virtual bool	GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
		virtual int32	GetNumTracks(EMediaTrackType TrackType) const override;
		virtual int32	GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
		virtual int32	GetSelectedTrack(EMediaTrackType TrackType) const override;
		virtual FText	GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
		virtual int32	GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
		virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
		virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
		virtual bool	GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
		virtual bool	SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
		virtual bool	SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;
		// End IMediaTracks

		// Begin IMediaControls
		virtual bool			 CanControl(EMediaControl Control) const override;
		virtual FTimespan		 GetDuration() const override;
		virtual float			 GetRate() const override;
		virtual EMediaState		 GetState() const override;
		virtual EMediaStatus	 GetStatus() const override;
		virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
		virtual FTimespan		 GetTime() const override;
		virtual bool			 IsLooping() const override;
		virtual bool			 Seek(const FTimespan& Time) override;
		virtual bool			 SetLooping(bool Looping) override;
		virtual bool			 SetRate(float Rate) override;
		// End IMediaControls

	private:
		// Incremented Id used to construct the PlayerName
		static uint32_t PlayerId;
		// A unique identifier for the player instance. Used to create unique EpicRtc sessions per player instance
		FString PlayerName;
		// The URL of the stream to connect to
		FString StreamURL;
		// The id of the streamer to subscribe to. Optionally passed in through the stream URL. If not present in URL
		// the first streamer id returned in the streamer list will be the streamer that is subscribed to
		FString TargetStreamerId;

		// The audio sink used to process remote audio
		TSharedPtr<FEpicRtcAudioSink> AudioSink;
		// The remote audio track. We only support one track at this time so we make sure we only receive frames for that track
		uintptr_t RemoteAudioTrack;

		// The video sink used to process remote audio
		TSharedPtr<FEpicRtcVideoSink> VideoSink;
		// The remote video track. We only support one track at this time so we make sure we only receive frames for that track
		uintptr_t RemoteVideoTrack;

		TSharedPtr<FRTCStatsCollector> StatsCollector;
		FDelegateHandle				   OnStatsReadyHandle;

		int32 SelectedAudioTrack;
		int32 SelectedVideoTrack;

		TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;

		TSharedPtr<class FPixelStreaming2AudioSamplePool>	AudioSamplePool;
		TSharedPtr<class FPixelStreaming2TextureSamplePool> VideoSamplePool;

	public:
		// Begin IPixelStreaming2SessionObserver
		virtual void OnSessionStateUpdate(const EpicRtcSessionState StateUpdate) override;
		virtual void OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate) override;
		virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) override;
		// End IPixelStreaming2SessionObserver

		// Begin IPixelStreaming2RoomObserver
		virtual void							   OnRoomStateUpdate(const EpicRtcRoomState State) override;
		virtual void							   OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) override;
		virtual void							   OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) override;
		virtual void							   OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) override;
		virtual void							   OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) override;
		virtual void							   OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) override;
		[[nodiscard]] virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		[[nodiscard]] virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		virtual void							   OnRoomErrorUpdate(const EpicRtcErrorCode Error) override;
		// End IPixelStreaming2RoomObserver

		// Begin IPixelStreaming2AudioTrackObserver
		virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) override;
		virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) override;
		virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) override;
		// End IPixelStreaming2AudioTrackObserver

		// Begin IPixelStreaming2VideoTrackObserver
		virtual void		OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) override;
		virtual void		OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) override;
		virtual void		OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) override;
		virtual void		OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) override;
		virtual EpicRtcBool Enabled() const override;
		// End IPixelStreaming2VideoTrackObserver

		// Begin IPixelStreaming2DataTrackObserver
		virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) override;
		virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) override;
		virtual void OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error) override;
		// End IPixelStreaming2DataTrackObserver

		// Begin IPixelStreaming2VideoConsumer
		virtual void ConsumeFrame(FTextureRHIRef Frame) override;
		virtual void OnVideoConsumerAdded() override;
		virtual void OnVideoConsumerRemoved() override;
		// End IPixelStreaming2VideoConsumer

		// Begin IPixelStreaming2AudioConsumer
		virtual void ConsumeRawPCM(const int16_t* AudioData, int SampleRate, size_t NumChannels, size_t NumFrames) override;
		virtual void OnAudioConsumerAdded() override;
		virtual void OnAudioConsumerRemoved() override;
		// End IPixelStreaming2AudioConsumer

		void OnStatsReady(const FString& PeerId, const EpicRtcConnectionStats& ConnectionStats);

	private:
		EpicRtcSessionState SessionState = EpicRtcSessionState::Disconnected;

		// Begin EpicRtc Classes
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
		TRefCountPtr<EpicRtcSessionInterface>	 EpicRtcSession;
		TRefCountPtr<EpicRtcRoomInterface>		 EpicRtcRoom;
		// End EpicRtc Classes

		// Begin EpicRtc Observers
		TRefCountPtr<FEpicRtcSessionObserver>			SessionObserver;
		TRefCountPtr<FEpicRtcRoomObserver>				RoomObserver;
		TRefCountPtr<FEpicRtcAudioTrackObserverFactory> AudioTrackObserverFactory;
		TRefCountPtr<FEpicRtcVideoTrackObserverFactory> VideoTrackObserverFactory;
		TRefCountPtr<FEpicRtcDataTrackObserverFactory>	DataTrackObserverFactory;
		// End EpicRtc Observers
	};
} // namespace UE::PixelStreaming2