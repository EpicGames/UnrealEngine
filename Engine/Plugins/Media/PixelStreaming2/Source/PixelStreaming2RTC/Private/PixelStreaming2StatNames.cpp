// Copyright Epic Games, Inc. All Rights Reserved.
#include "PixelStreaming2StatNames.h"

namespace PixelStreaming2StatNames
{
	// Calculated stats
	const FName Bitrate = FName(TEXT("bitrate"));
	const FName BitrateMegabits = FName(TEXT("bitrateMbits"));
	const FName TargetBitrateMegabits = FName(TEXT("targetBitrateMbits"));
	const FName MeanSendDelay = FName(TEXT("captureToSend"));
	const FName SourceFps = FName(TEXT("captureFps"));
	const FName Fps = FName(TEXT("captureFps"));
	const FName MeanEncodeTime = FName(TEXT("encodeTime"));
	const FName EncodedFramesPerSecond = FName(TEXT("encodeFps"));
	const FName DecodedFramesPerSecond = FName(TEXT("decodeFps"));
	const FName MeanQPPerSecond = FName(TEXT("qp"));
	const FName FramesSentPerSecond = FName(TEXT("transmitFps"));
	const FName FramesReceivedPerSecond = FName(TEXT("receiveFps"));

	// WebRTC stats
	const FName JitterBufferDelay = FName(TEXT("jitterBufferDelay"));
	const FName FramesSent = FName(TEXT("framesSent"));
	const FName FramesReceived = FName(TEXT("framesReceived"));
	const FName FramesPerSecond = FName(TEXT("framesPerSecond"));
	const FName FramesDecoded = FName(TEXT("framesDecoded"));
	const FName FramesDropped = FName(TEXT("framesDropped"));
	const FName FramesCorrupted = FName(TEXT("framesCorrupted"));
	const FName PartialFramesLost = FName(TEXT("partialFramesLost"));
	const FName FullFramesLost = FName(TEXT("fullFramesLost"));
	const FName HugeFramesSent = FName(TEXT("hugeFramesSent"));
	const FName JitterBufferTargetDelay = FName(TEXT("jitterBufferTargetDelay"));
	const FName InterruptionCount = FName(TEXT("interruptionCount"));
	const FName TotalInterruptionDuration = FName(TEXT("totalInterruptionDuration"));
	const FName FreezeCount = FName(TEXT("freezeCount"));
	const FName PauseCount = FName(TEXT("pauseCount"));
	const FName TotalFreezesDuration = FName(TEXT("totalFreezesDuration"));
	const FName TotalPausesDuration = FName(TEXT("totalPausesDuration"));
	const FName FirCount = FName(TEXT("firCount"));
	const FName PliCount = FName(TEXT("pliCount"));
	const FName NackCount = FName(TEXT("nackCount"));
	const FName RetransmittedBytesSent = FName(TEXT("retransmittedBytesSent"));
	const FName TargetBitrate = FName(TEXT("targetBitrate"));
	const FName TotalEncodeBytesTarget = FName(TEXT("totalEncodedBytesTarget"));
	const FName KeyFramesEncoded = FName(TEXT("keyFramesEncoded"));
	const FName FrameWidth = FName(TEXT("frameWidth"));
	const FName FrameHeight = FName(TEXT("frameHeight"));
	const FName BytesSent = FName(TEXT("bytesSent"));
	const FName BytesReceived = FName(TEXT("bytesReceived"));
	const FName QPSum = FName(TEXT("qpSum"));
	const FName TotalEncodeTime = FName(TEXT("totalEncodeTime"));
	const FName TotalPacketSendDelay = FName(TEXT("totalPacketSendDelay"));
	const FName FramesEncoded = FName(TEXT("framesEncoded"));
	const FName AvgSendDelay = FName(TEXT("packetSendDelay"));
	const FName MessagesSent = FName(TEXT("messagesSent"));
	const FName MessagesReceived = FName(TEXT("messagesReceived"));

	const FName PacketsLost = FName(TEXT("packetsLost"));
	const FName Jitter = FName(TEXT("jitter"));
	const FName RoundTripTime = FName(TEXT("roundTripTime"));
	const FName KeyFramesDecoded = FName(TEXT("keyFramesDecoded"));
	const FName AudioLevel = FName(TEXT("audioLevel"));
	const FName TotalSamplesDuration = FName(TEXT("totalSamplesDuration"));
	const FName AvailableOutgoingBitrate = FName(TEXT("availableOutgoingBitrate"));
	const FName AvailableIncomingBitrate = FName(TEXT("availableIncomingBitrate"));
	const FName RetransmittedBytesReceived = FName(TEXT("retransmittedBytesReceived"));
	const FName RetransmittedPacketsReceived = FName(TEXT("retransmittedPacketsReceived"));
	const FName PacketsSent = FName(TEXT("packetsSent"));
	const FName EncoderImplementation = FName(TEXT("encoderImplementation"));
	const FName DecoderImplementation = FName(TEXT("decoderImplementation"));
	const FName MimeType = FName(TEXT("mimeType"));
	const FName Channels = FName(TEXT("channels"));
	const FName ClockRate = FName(TEXT("clockRate"));
	const FName QualityLimitationReason = FName(TEXT("qualityLimitationReason"));
	const FName Rid = FName(TEXT("rid"));

	// Data-channel stats
	const FName DataChannelBytesSent = FName(TEXT("data-channel-bytesSent"));
	const FName DataChannelBytesReceived = FName(TEXT("data-channel-bytesReceived"));
	const FName DataChannelMessagesSent = FName(TEXT("data-channel-messagesSent"));
	const FName DataChannelMessagesReceived = FName(TEXT("data-channel-messagesReceived"));

	// Pixel Streaming specific stats
	const FName InputController = FName(TEXT("inputController"));

} // namespace PixelStreaming2StatNames
