// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AtomicOnceFlag.h"
#include "EpicRtcAudioSink.h"
#include "EpicRtcAudioSource.h"
#include "EpicRtcDataTrack.h"
#include "EpicRtcVideoSink.h"
#include "EpicRtcVideoSource.h"
#include "RTCStatsCollector.h"
#include "UtilsCommon.h"

#include "epic_rtc/core/audio/audio_track.h"
#include "epic_rtc/core/video/video_track.h"
#include "epic_rtc/core/data_track.h"
#include "epic_rtc/core/participant.h"

namespace UE::PixelStreaming2
{
	/**
	 * NOTE: As all of these members are either ref counted, or shared pointers, usage MUST look something like
	 * 
	 * TSharedPtr<TargetClass> Member; <-- Target member declared outside scope
	 * if(TSharedPtr<FPlayerContext> Participant = Particpants->FindRef(ParticipantId); Participant.IsValid())
	 * {
	 * 		Member = Participant->Member;
	 * }
	 * 
	 * if (Member)
	 * {
	 * 		Member->DoFunc();
	 * }
	 * 
	 * Using this pattern means that we won't be keeping all members of the player context alive longer than required
	 */
	struct FPlayerContext
	{
		TRefCountPtr<EpicRtcParticipantInterface> ParticipantInterface;

		TSharedPtr<FEpicRtcAudioSink> AudioSink;
		TSharedPtr<FEpicRtcVideoSink> VideoSink;
		TSharedPtr<FEpicRtcDataTrack> DataTrack;

		// One shot flags to keep track of if the delegates for this player has already been broadcast
		// to prevent multiple broadcasts
		FAtomicOnceFlag BroadcastLocalAudioTrackClosed;
		FAtomicOnceFlag BroadcastLocalVideoTrackClosed;
		FAtomicOnceFlag BroadcastRemoteAudioTrackClosed;
		FAtomicOnceFlag BroadcastRemoteVideoTrackClosed;

		TSharedPtr<FRTCStatsCollector> StatsCollector;
	};

} // namespace UE::PixelStreaming2