// Copyright Epic Games, Inc. All Rights Reserved.

#include "RTCInputHandler.h"

#include "Engine/Engine.h"
#include "Engine/GameEngine.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "PixelStreaming2InputStructures.h"
#include "IPixelStreaming2HMDModule.h"
#include "Logging.h"
#include "PixelStreaming2DefaultDataProtocol.h"
#include "PixelStreaming2JavaScriptKeyCodes.inl"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2Utils.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/SMultiLineEditableText.h"

namespace UE::PixelStreaming2
{
	using namespace UE::PixelStreaming2Input;

	TSharedPtr<FRTCInputHandler> FRTCInputHandler::Create()
	{
		return MakeShareable<FRTCInputHandler>(new FRTCInputHandler());
	}

	FRTCInputHandler::FRTCInputHandler()
	{
		// RTC module uses the default protocol
		ToStreamerProtocol = UE::PixelStreaming2Input::GetDefaultToStreamerProtocol();
		FromStreamerProtocol = UE::PixelStreaming2Input::GetDefaultFromStreamerProtocol();

		RegisterMessageHandler("TouchStart", [this](FString SourceId, FMemoryReader Ar) { HandleOnTouchStarted(Ar); });
		RegisterMessageHandler("TouchMove", [this](FString SourceId, FMemoryReader Ar) { HandleOnTouchMoved(Ar); });
		RegisterMessageHandler("TouchEnd", [this](FString SourceId, FMemoryReader Ar) { HandleOnTouchEnded(Ar); });

		RegisterMessageHandler("KeyPress", [this](FString SourceId, FMemoryReader Ar) { HandleOnKeyChar(Ar); });
		RegisterMessageHandler("KeyUp", [this](FString SourceId, FMemoryReader Ar) { HandleOnKeyUp(Ar); });
		RegisterMessageHandler("KeyDown", [this](FString SourceId, FMemoryReader Ar) { HandleOnKeyDown(Ar); });

		RegisterMessageHandler("MouseEnter", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseEnter(Ar); });
		RegisterMessageHandler("MouseLeave", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseLeave(Ar); });
		RegisterMessageHandler("MouseUp", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseUp(Ar); });
		RegisterMessageHandler("MouseDown", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseDown(Ar); });
		RegisterMessageHandler("MouseMove", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseMove(Ar); });
		RegisterMessageHandler("MouseWheel", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseWheel(Ar); });
		RegisterMessageHandler("MouseDouble", [this](FString SourceId, FMemoryReader Ar) { HandleOnMouseDoubleClick(Ar); });

		RegisterMessageHandler("GamepadConnected", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerConnected(Ar); });
		RegisterMessageHandler("GamepadAnalog", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerAnalog(Ar); });
		RegisterMessageHandler("GamepadButtonPressed", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerButtonPressed(Ar); });
		RegisterMessageHandler("GamepadButtonReleased", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerButtonReleased(Ar); });
		RegisterMessageHandler("GamepadDisconnected", [this](FString SourceId, FMemoryReader Ar) { HandleOnControllerDisconnected(Ar); });

		RegisterMessageHandler("XREyeViews", [this](FString SourceId, FMemoryReader Ar) { HandleOnXREyeViews(Ar); });
		RegisterMessageHandler("XRHMDTransform", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRHMDTransform(Ar); });
		RegisterMessageHandler("XRControllerTransform", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRControllerTransform(Ar); });
		RegisterMessageHandler("XRButtonPressed", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonPressed(Ar); });
		RegisterMessageHandler("XRButtonTouched", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonTouched(Ar); });
		RegisterMessageHandler("XRButtonTouchReleased", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonTouchReleased(Ar); });
		RegisterMessageHandler("XRButtonReleased", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRButtonReleased(Ar); });
		RegisterMessageHandler("XRAnalog", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRAnalog(Ar); });
		RegisterMessageHandler("XRSystem", [this](FString SourceId, FMemoryReader Ar) { HandleOnXRSystem(Ar); });

		RegisterMessageHandler("Command", [this](FString SourceId, FMemoryReader Ar) { HandleOnCommand(SourceId, Ar); });
		RegisterMessageHandler("UIInteraction", [this](FString SourceId, FMemoryReader Ar) { HandleUIInteraction(Ar); });
		RegisterMessageHandler("TextboxEntry", [this](FString SourceId, FMemoryReader Ar) { HandleOnTextboxEntry(Ar); });

		// RequestQualityControl has been removed. We no-op this handler to prevent warnings about unregistered message types from older front-end versions
		RegisterMessageHandler("RequestQualityControl", [this](FString SourceId, FMemoryReader Ar) { /* Do nothing */ });

		// Populate map
		// Button indices found in: https://github.com/immersive-web/webxr-input-profiles/tree/master/packages/registry/profiles
		// HTC Vive - Left Hand
		// Buttons
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 0, EPixelStreaming2InputAction::Click), EKeys::Vive_Left_Trigger_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 0, EPixelStreaming2InputAction::Axis), EKeys::Vive_Left_Trigger_Axis);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 1, EPixelStreaming2InputAction::Click), EKeys::Vive_Left_Grip_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 2, EPixelStreaming2InputAction::Click), EKeys::Vive_Left_Trackpad_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 2, EPixelStreaming2InputAction::Touch), EKeys::Vive_Left_Trackpad_Touch);
		// Axes
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 0, EPixelStreaming2InputAction::X), EKeys::Vive_Left_Trackpad_X);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Left, 1, EPixelStreaming2InputAction::Y), EKeys::Vive_Left_Trackpad_Y);
		// HTC Vive - Right Hand
		// Buttons
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 0, EPixelStreaming2InputAction::Click), EKeys::Vive_Right_Trigger_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 0, EPixelStreaming2InputAction::Axis), EKeys::Vive_Right_Trigger_Axis);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 1, EPixelStreaming2InputAction::Click), EKeys::Vive_Right_Grip_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 2, EPixelStreaming2InputAction::Click), EKeys::Vive_Right_Trackpad_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 2, EPixelStreaming2InputAction::Touch), EKeys::Vive_Right_Trackpad_Touch);
		// Axes
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 0, EPixelStreaming2InputAction::X), EKeys::Vive_Right_Trackpad_X);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::HTCVive, EControllerHand::Right, 1, EPixelStreaming2InputAction::Y), EKeys::Vive_Right_Trackpad_Y);

		// Quest - Left Hand
		// Buttons
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 0, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Left_Trigger_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 0, EPixelStreaming2InputAction::Axis), EKeys::OculusTouch_Left_Trigger_Axis);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 0, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Left_Trigger_Touch);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 1, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Left_Grip_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 1, EPixelStreaming2InputAction::Axis), EKeys::OculusTouch_Left_Grip_Axis);
		// Index 1 (grip) touch not supported in UE
		// Index 2 not supported by WebXR
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 3, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Left_Thumbstick_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 3, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Left_Thumbstick_Touch);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 4, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Left_X_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 4, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Left_X_Touch);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 5, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Left_Y_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 5, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Left_Y_Touch);
		// Index 6 (thumbrest) not supported in UE

		// Axes
		// Indices 0 and 1 not supported in WebXR
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 2, EPixelStreaming2InputAction::X), EKeys::OculusTouch_Left_Thumbstick_X);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Left, 3, EPixelStreaming2InputAction::Y), EKeys::OculusTouch_Left_Thumbstick_Y);

		// Quest - Right Hand
		// Buttons
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 0, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Right_Trigger_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 0, EPixelStreaming2InputAction::Axis), EKeys::OculusTouch_Right_Trigger_Axis);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 0, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Right_Trigger_Touch);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 1, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Right_Grip_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 1, EPixelStreaming2InputAction::Axis), EKeys::OculusTouch_Right_Grip_Axis);
		// Index 1 (grip) touch not supported in UE
		// Index 2 not supported by WebXR
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 3, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Right_Thumbstick_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 3, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Right_Thumbstick_Touch);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 4, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Right_A_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 4, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Right_A_Touch);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 5, EPixelStreaming2InputAction::Click), EKeys::OculusTouch_Right_B_Click);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 5, EPixelStreaming2InputAction::Touch), EKeys::OculusTouch_Right_B_Touch);
		// Index 6 (thumbrest) not supported in UE

		// Axes
		// Indices 0 and 1 not supported in WebXR
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 2, EPixelStreaming2InputAction::X), EKeys::OculusTouch_Right_Thumbstick_X);
		XRInputToFKey.Add(MakeTuple(EPixelStreaming2XRSystem::Quest, EControllerHand::Right, 3, EPixelStreaming2InputAction::Y), EKeys::OculusTouch_Right_Thumbstick_Y);

		// Gamepad Axes
		GamepadInputToFKey.Add(MakeTuple(1, EPixelStreaming2InputAction::Axis), EKeys::Gamepad_LeftX);
		GamepadInputToFKey.Add(MakeTuple(2, EPixelStreaming2InputAction::Axis), EKeys::Gamepad_LeftY);
		GamepadInputToFKey.Add(MakeTuple(3, EPixelStreaming2InputAction::Axis), EKeys::Gamepad_RightX);
		GamepadInputToFKey.Add(MakeTuple(4, EPixelStreaming2InputAction::Axis), EKeys::Gamepad_RightY);
		GamepadInputToFKey.Add(MakeTuple(5, EPixelStreaming2InputAction::Axis), EKeys::Gamepad_LeftTriggerAxis);
		GamepadInputToFKey.Add(MakeTuple(6, EPixelStreaming2InputAction::Axis), EKeys::Gamepad_RightTriggerAxis);
		// Gamepad Buttons
		GamepadInputToFKey.Add(MakeTuple(0, EPixelStreaming2InputAction::Click), EKeys::Gamepad_FaceButton_Bottom);
		GamepadInputToFKey.Add(MakeTuple(1, EPixelStreaming2InputAction::Click), EKeys::Gamepad_FaceButton_Right);
		GamepadInputToFKey.Add(MakeTuple(2, EPixelStreaming2InputAction::Click), EKeys::Gamepad_FaceButton_Left);
		GamepadInputToFKey.Add(MakeTuple(3, EPixelStreaming2InputAction::Click), EKeys::Gamepad_FaceButton_Top);
		GamepadInputToFKey.Add(MakeTuple(4, EPixelStreaming2InputAction::Click), EKeys::Gamepad_LeftShoulder);
		GamepadInputToFKey.Add(MakeTuple(5, EPixelStreaming2InputAction::Click), EKeys::Gamepad_RightShoulder);
		GamepadInputToFKey.Add(MakeTuple(6, EPixelStreaming2InputAction::Click), EKeys::Gamepad_LeftTrigger);
		GamepadInputToFKey.Add(MakeTuple(7, EPixelStreaming2InputAction::Click), EKeys::Gamepad_RightTrigger);
		GamepadInputToFKey.Add(MakeTuple(8, EPixelStreaming2InputAction::Click), EKeys::Gamepad_Special_Left);
		GamepadInputToFKey.Add(MakeTuple(9, EPixelStreaming2InputAction::Click), EKeys::Gamepad_Special_Right);
		GamepadInputToFKey.Add(MakeTuple(10, EPixelStreaming2InputAction::Click), EKeys::Gamepad_LeftThumbstick);
		GamepadInputToFKey.Add(MakeTuple(11, EPixelStreaming2InputAction::Click), EKeys::Gamepad_RightThumbstick);
		GamepadInputToFKey.Add(MakeTuple(12, EPixelStreaming2InputAction::Click), EKeys::Gamepad_DPad_Up);
		GamepadInputToFKey.Add(MakeTuple(13, EPixelStreaming2InputAction::Click), EKeys::Gamepad_DPad_Down);
		GamepadInputToFKey.Add(MakeTuple(14, EPixelStreaming2InputAction::Click), EKeys::Gamepad_DPad_Left);
		GamepadInputToFKey.Add(MakeTuple(15, EPixelStreaming2InputAction::Click), EKeys::Gamepad_DPad_Right);

		PopulateDefaultCommandHandlers();

		// default to the scene viewport if we have a game engine
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine); GameEngine != nullptr)
		{
			TargetWindow = GameEngine->GameViewport ? GameEngine->GameViewport->GetWindow() : nullptr;
			if (!TargetWindow.IsValid())
			{
				TargetWindow = GameEngine->GameViewportWindow;
			}
			
			if (!TargetWindow.IsValid())
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "Cannot set target window - target window is not valid.");
			}
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Cannot set target window - GEngine is not valid.");
		}
	}

	FRTCInputHandler::~FRTCInputHandler()
	{
	}

	void FRTCInputHandler::HandleOnKeyChar(FMemoryReader Ar)
	{
		TPayload<TCHAR> Payload(Ar);
		OnKeyChar(Payload.Get<0>());
	}

	void FRTCInputHandler::HandleOnKeyDown(FMemoryReader Ar)
	{
		TPayload<uint8, uint8> Payload(Ar);

		bool		bIsRepeat = Payload.Get<1>() != 0;
		const FKey* AgnosticKey = UE::PixelStreaming2Input::JavaScriptKeyCodeToFKey[Payload.Get<0>()];
		if (FilterKey(*AgnosticKey))
		{
			OnKeyDown(*AgnosticKey, bIsRepeat);
		}
	}

	void FRTCInputHandler::HandleOnKeyUp(FMemoryReader Ar)
	{
		TPayload<uint8> Payload(Ar);
		const FKey*		AgnosticKey = UE::PixelStreaming2Input::JavaScriptKeyCodeToFKey[Payload.Get<0>()];
		if (FilterKey(*AgnosticKey))
		{
			OnKeyUp(*AgnosticKey);
		}
	}

	void FRTCInputHandler::HandleOnMouseEnter(FMemoryReader Ar)
	{
		OnMouseEnter();
	}

	void FRTCInputHandler::HandleOnMouseLeave(FMemoryReader Ar)
	{
		OnMouseLeave();
	}

	void FRTCInputHandler::HandleOnMouseDown(FMemoryReader Ar)
	{
		TPayload<uint8, uint16, uint16> Payload(Ar);

		EMouseButtons::Type Button = static_cast<EMouseButtons::Type>(Payload.Get<0>());
		//                                                                           convert range from 0,65536 -> 0,1
		FIntPoint ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Get<1>() / uint16_MAX, Payload.Get<2>() / uint16_MAX));

		OnMouseDown(Button, ScreenLocation);
	}

	void FRTCInputHandler::HandleOnMouseUp(FMemoryReader Ar)
	{
		TPayload<uint8, uint16, uint16> Payload(Ar);

		EMouseButtons::Type Button = static_cast<EMouseButtons::Type>(Payload.Get<0>());
		OnMouseUp(Button);
	}

	void FRTCInputHandler::HandleOnMouseMove(FMemoryReader Ar)
	{
		TPayload<uint16, uint16, int16, int16> Payload(Ar);

		//                                                                           convert range from 0,65536 -> 0,1
		FIntPoint ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Get<0>() / uint16_MAX, Payload.Get<1>() / uint16_MAX));
		//                                                                 convert range from -32,768 to 32,767 -> -1,1
		FIntPoint Delta = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Get<2>() / int16_MAX, Payload.Get<3>() / int16_MAX), false);

		OnMouseMove(ScreenLocation, Delta);
	}

	void FRTCInputHandler::HandleOnMouseWheel(FMemoryReader Ar)
	{
		TPayload<int16, uint16, uint16> Payload(Ar);
		//                                                                           convert range from 0,65536 -> 0,1
		FIntPoint	ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Get<1>() / uint16_MAX, Payload.Get<2>() / uint16_MAX));
		const float SpinFactor = 1 / 120.0f;

		OnMouseWheel(ScreenLocation, Payload.Get<0>() * SpinFactor);
	}

	void FRTCInputHandler::HandleOnMouseDoubleClick(FMemoryReader Ar)
	{
		TPayload<uint8, uint16, uint16> Payload(Ar);
		EMouseButtons::Type				Button = static_cast<EMouseButtons::Type>(Payload.Get<0>());

		//                                                                           convert range from 0,65536 -> 0,1
		FIntPoint ScreenLocation = ConvertFromNormalizedScreenLocation(FVector2D(Payload.Get<1>() / uint16_MAX, Payload.Get<2>() / uint16_MAX));

		OnMouseDoubleClick(Button, ScreenLocation);
	}

	void FRTCInputHandler::HandleOnTouchStarted(FMemoryReader Ar)
	{
		TPayload<uint8> Payload(Ar);

		uint8 NumTouches = Payload.Get<0>();
		for (uint8 TouchIdx = 0; TouchIdx < NumTouches; TouchIdx++)
		{
			//       PosX    PoxY    IDX   Force  Valid
			TPayload<uint16, uint16, uint8, uint8, uint8> Touch(Ar);
			// If Touch is valid
			if (Touch.Get<4>() != 0)
			{
				//                                                                           convert range from 0,65536 -> 0,1
				FIntPoint	TouchLocation = ConvertFromNormalizedScreenLocation(FVector2D(Touch.Get<0>() / uint16_MAX, Touch.Get<1>() / uint16_MAX));
				const int32 TouchIndex = Touch.Get<2>();
				const float TouchForce = Touch.Get<3>() / 255.0f;

				OnTouchStarted(TouchLocation, TouchIndex, TouchForce);
			}
		}
	}

	void FRTCInputHandler::HandleOnTouchMoved(FMemoryReader Ar)
	{
		TPayload<uint8> Payload(Ar);

		uint8 NumTouches = Payload.Get<0>();
		for (uint8 TouchIdx = 0; TouchIdx < NumTouches; TouchIdx++)
		{
			//       PosX    PoxY    IDX   Force  Valid
			TPayload<uint16, uint16, uint8, uint8, uint8> Touch(Ar);
			// If Touch is valid
			if (Touch.Get<4>() != 0)
			{
				//                                                                           convert range from 0,65536 -> 0,1
				FIntPoint	TouchLocation = ConvertFromNormalizedScreenLocation(FVector2D(Touch.Get<0>() / uint16_MAX, Touch.Get<1>() / uint16_MAX));
				const int32 TouchIndex = Touch.Get<2>();
				const float TouchForce = Touch.Get<3>() / 255.0f;

				OnTouchMoved(TouchLocation, TouchIndex, TouchForce);
			}
		}
	}

	void FRTCInputHandler::HandleOnTouchEnded(FMemoryReader Ar)
	{
		TPayload<uint8> Payload(Ar);

		uint8 NumTouches = Payload.Get<0>();
		for (uint8 TouchIdx = 0; TouchIdx < NumTouches; TouchIdx++)
		{
			//       PosX    PoxY    IDX   Force  Valid
			TPayload<uint16, uint16, uint8, uint8, uint8> Touch(Ar);
			// Always allowing the "up" events regardless of in or outside the valid region so
			// states aren't stuck "down". Might want to uncomment this if it causes other issues.
			// if (Touch.Get<4>() != 0)
			{
				//                                                                           convert range from 0,65536 -> 0,1
				FIntPoint	TouchLocation = ConvertFromNormalizedScreenLocation(FVector2D(Touch.Get<0>() / uint16_MAX, Touch.Get<1>() / uint16_MAX));
				const int32 TouchIndex = Touch.Get<2>();

				OnTouchEnded(TouchLocation, TouchIndex);
			}
		}
	}

	void FRTCInputHandler::HandleOnControllerConnected(FMemoryReader Ar)
	{
		OnControllerConnected();
	}

	void FRTCInputHandler::HandleOnControllerAnalog(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, double> Payload(Ar);

		uint8  ControllerIndex = Payload.Get<0>();
		uint8  KeyId = Payload.Get<1>();
		double AxisValue = Payload.Get<2>();

		FKey* KeyPtr = GamepadInputToFKey.Find(MakeTuple(KeyId, EPixelStreaming2InputAction::Axis));
		if (!KeyPtr)
		{
			return;
		}

		OnControllerAnalog(ControllerIndex, *KeyPtr, AxisValue);
	}

	void FRTCInputHandler::HandleOnControllerButtonPressed(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, uint8> Payload(Ar);

		uint8 ControllerIndex = Payload.Get<0>();
		uint8 KeyId = Payload.Get<1>();
		bool  bIsRepeat = Payload.Get<2>() != 0;

		FKey* KeyPtr = GamepadInputToFKey.Find(MakeTuple(KeyId, EPixelStreaming2InputAction::Click));
		if (KeyPtr == nullptr)
		{
			return;
		}

		OnControllerButtonPressed(ControllerIndex, *KeyPtr, bIsRepeat);
	}

	void FRTCInputHandler::HandleOnControllerButtonReleased(FMemoryReader Ar)
	{
		TPayload<uint8, uint8> Payload(Ar);

		uint8 ControllerIndex = Payload.Get<0>();
		uint8 KeyId = Payload.Get<1>();

		FKey* KeyPtr = GamepadInputToFKey.Find(MakeTuple(KeyId, EPixelStreaming2InputAction::Click));
		if (KeyPtr == nullptr)
		{
			return;
		}

		OnControllerButtonReleased(ControllerIndex, *KeyPtr);
	}

	void FRTCInputHandler::HandleOnControllerDisconnected(FMemoryReader Ar)
	{
		TPayload<uint8> Payload(Ar);

		OnControllerDisconnected(Payload.Get<0>());
	}

	void FRTCInputHandler::HandleOnXREyeViews(FMemoryReader Ar)
	{
		// The `Ar` buffer contains the left eye transform matrix stored as 16 floats
		FTransform LeftEyeTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));
		// The `Ar` buffer contains the left eye projection matrix stored as 16 floats
		FMatrix LeftEyeProjectionMatrix = ExtractWebXRMatrix(Ar);
		// The `Ar` buffer contains the right eye transform matrix stored as 16 floats
		FTransform RightEyeTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));
		// The `Ar` buffer contains the right eye projection matrix stored as 16 floats
		FMatrix RightEyeProjectionMatrix = ExtractWebXRMatrix(Ar);

		// The `Ar` buffer contains the right eye projection matrix stored as 16 floats
		FTransform HMDTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		OnXREyeViews(LeftEyeTransform, LeftEyeProjectionMatrix, RightEyeTransform, RightEyeProjectionMatrix, HMDTransform);
	}

	void FRTCInputHandler::HandleOnXRHMDTransform(FMemoryReader Ar)
	{
		// The `Ar` buffer contains the transform matrix stored as 16 floats
		FTransform HMDTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		OnXRHMDTransform(HMDTransform);
	}

	void FRTCInputHandler::HandleOnXRControllerTransform(FMemoryReader Ar)
	{
		// The `Ar` buffer contains the transform matrix stored as 16 floats
		FTransform ControllerTransform = WebXRMatrixToUETransform(ExtractWebXRMatrix(Ar));

		// The `Ar` buffer contains a UInt8 for the handedness
		EControllerHand Handedness = EControllerHand::Left;
		Ar << Handedness;

		OnXRControllerTransform(ControllerTransform, Handedness);
	}

	void FRTCInputHandler::HandleOnXRButtonTouched(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, uint8> Payload(Ar);

		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Get<0>());
		uint8			ButtonIdx = Payload.Get<1>();
		bool			bIsRepeat = Payload.Get<2>() != 0;

		EPixelStreaming2XRSystem System = IPixelStreaming2HMDModule::Get().GetActiveXRSystem();

		FKey* KeyPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, EPixelStreaming2InputAction::Touch));
		if (KeyPtr == nullptr)
		{
			return;
		}

		OnXRButtonTouched(Handedness, *KeyPtr, bIsRepeat);
	}

	void FRTCInputHandler::HandleOnXRButtonTouchReleased(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, uint8> Payload(Ar);

		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Get<0>());
		uint8			ButtonIdx = Payload.Get<1>();

		EPixelStreaming2XRSystem System = IPixelStreaming2HMDModule::Get().GetActiveXRSystem();

		FKey* KeyPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, EPixelStreaming2InputAction::Touch));
		if (KeyPtr == nullptr)
		{
			return;
		}

		OnXRButtonTouchReleased(Handedness, *KeyPtr);
	}

	void FRTCInputHandler::HandleOnXRButtonPressed(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, uint8, double> Payload(Ar);

		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Get<0>());
		uint8			ButtonIdx = Payload.Get<1>();
		bool			bIsRepeat = Payload.Get<2>() != 0;
		double			AnalogValue = Payload.Get<3>();

		EPixelStreaming2XRSystem System = IPixelStreaming2HMDModule::Get().GetActiveXRSystem();

		FKey* ButtonPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, EPixelStreaming2InputAction::Click));
		if (ButtonPtr)
		{
			OnXRButtonPressed(Handedness, *ButtonPtr, bIsRepeat);
		}

		// Try and see if there is an axis associated with this button (usually the case for triggers)
		FKey* AxisPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, EPixelStreaming2InputAction::Axis));
		// If we have axis associate with this press then set axis value to the button press value
		if (AxisPtr)
		{
			OnXRAnalog(Handedness, *AxisPtr, AnalogValue);
		}
	}

	void FRTCInputHandler::HandleOnXRButtonReleased(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, uint8> Payload(Ar);

		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Get<0>());
		uint8			ButtonIdx = Payload.Get<1>();

		EPixelStreaming2XRSystem System = IPixelStreaming2HMDModule::Get().GetActiveXRSystem();

		// Try and see if there is an axis associated with this button (usually the case for triggers)
		FKey* AxisPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, EPixelStreaming2InputAction::Axis));
		// If we have axis associate with this press then set axis value to 0.0
		if (AxisPtr)
		{
			OnXRAnalog(Handedness, *AxisPtr, 0.0f);
		}

		// Do the actual release after the analog trigger, as the release can cancel any further inputs
		FKey* ButtonPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, ButtonIdx, EPixelStreaming2InputAction::Click));
		if (ButtonPtr)
		{
			OnXRButtonReleased(Handedness, *ButtonPtr);
		}
	}

	void FRTCInputHandler::HandleOnXRAnalog(FMemoryReader Ar)
	{
		TPayload<uint8, uint8, double> Payload(Ar);

		EControllerHand Handedness = static_cast<EControllerHand>(Payload.Get<0>());
		uint8			AxisIndex = Payload.Get<1>();
		double			AnalogValue = Payload.Get<2>();

		EPixelStreaming2XRSystem	System = IPixelStreaming2HMDModule::Get().GetActiveXRSystem();
		EPixelStreaming2InputAction InputAction = static_cast<EPixelStreaming2InputAction>(AxisIndex % 2);

		FKey* KeyPtr = XRInputToFKey.Find(MakeTuple(System, Handedness, AxisIndex, InputAction));
		if (KeyPtr == nullptr)
		{
			return;
		}

		OnXRAnalog(Handedness, *KeyPtr, AnalogValue);
	}

	void FRTCInputHandler::HandleOnXRSystem(FMemoryReader Ar)
	{
		TPayload<uint8> Payload(Ar);

		EPixelStreaming2XRSystem System = static_cast<EPixelStreaming2XRSystem>(Payload.Get<0>());

		OnXRSystem(System);
	}

	/**
	 * Command handling
	 */
	void FRTCInputHandler::HandleOnCommand(FString SourceId, FMemoryReader Ar)
	{
		if (Ar.TotalSize() < 2)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FRTCInputHandler::HandleOnTextboxEntry: Malformed message. Expected to contain at least 2 bytes representing the descriptor length in characters");
			return;
		}

		uint16 StringLength = 0;
		Ar << StringLength;

		if (Ar.TotalSize() < static_cast<int64>(2 + StringLength * sizeof(TCHAR)))
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FRTCInputHandler::HandleOnTextboxEntry: Malformed message. Descriptor length of {0} characters exceeds remaining message size", StringLength);
			return;
		}

		FString Descriptor;
		Descriptor.GetCharArray().SetNumUninitialized(StringLength + 1 /* +1 for null terminator */);
		// Read archive into the string's char array
		Ar.Serialize(Descriptor.GetCharArray().GetData(), StringLength * sizeof(TCHAR));
		// Null terminate
		Descriptor.GetCharArray()[StringLength] = TEXT('\0');

		UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "Command: {0}", Descriptor);

		// Iterate each command handler and see if the command we got matches any of the bound command names.
		for (auto& CommandHandlersPair : CommandHandlers)
		{
			FString CommandValue;
			bool	bSuccess = false;
			ExtractJsonFromDescriptor(Descriptor, CommandHandlersPair.Key, CommandValue, bSuccess);
			if (bSuccess)
			{
				// Execute bound command handler with descriptor and parsed command value
				CommandHandlersPair.Value(SourceId, Descriptor, CommandValue);
				return;
			}
		}
	}

	/**
	 * UI Interaction handling
	 */
	void FRTCInputHandler::HandleUIInteraction(FMemoryReader Ar)
	{
		// FPixelStreaming2Module overwrites this handler
	}

	/**
	 * Textbox Entry handling
	 */
	void FRTCInputHandler::HandleOnTextboxEntry(FMemoryReader Ar)
	{
		if (Ar.TotalSize() < 2)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FRTCInputHandler::HandleOnTextboxEntry: Malformed message. Expected to contain at least 2 bytes representing the descriptor length in characters");
			return;
		}

		uint16 StringLength = 0;
		Ar << StringLength;

		if (Ar.TotalSize() < static_cast<int64>(2 + StringLength * sizeof(TCHAR)))
		{
		UE_LOGFMT(LogPixelStreaming2RTC, Warning, "FRTCInputHandler::HandleOnTextboxEntry: Malformed message. Descriptor length of {0} characters exceeds remaining message size", StringLength);
			return;
		}

		FString Text;
		Text.GetCharArray().SetNumUninitialized(StringLength + 1 /* +1 for null terminator */);
		// Read archive into the string's char array
		Ar.Serialize(Text.GetCharArray().GetData(), StringLength * sizeof(TCHAR));
		// Null terminate
		Text.GetCharArray()[StringLength] = TEXT('\0');

		FSlateApplication::Get().ForEachUser([this, Text](FSlateUser& User) {
			TSharedPtr<SWidget> FocusedWidget = User.GetFocusedWidget();
			if (!FocusedWidget)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Verbose, "FRTCInputHandler::HandleOnTextboxEntry: Focused widget is invalid");
				return;
			}

			bool bIsEditableTextType = FocusedWidget->GetType() == TEXT("SEditableText");
			bool bIsMultiLineEditableTextType = FocusedWidget->GetType() == TEXT("SMultiLineEditableText");
			bool bEditable = FocusedWidget && (bIsEditableTextType || bIsMultiLineEditableTextType);

			if (bEditable)
			{
				if (bIsEditableTextType)
				{
					SEditableText* TextBox = static_cast<SEditableText*>(FocusedWidget.Get());
					TextBox->SetText(FText::FromString(Text));
				}
				else if (bIsMultiLineEditableTextType)
				{
					SMultiLineEditableText* TextBox = static_cast<SMultiLineEditableText*>(FocusedWidget.Get());
					TextBox->SetText(FText::FromString(Text));
				}

				// We need to manually trigger an Enter key press so that the OnTextCommitted delegate gets fired
				const uint32* KeyPtr = nullptr;
				const uint32* CharacterPtr = nullptr;
				FInputKeyManager::Get().GetCodesFromKey(EKeys::Enter, KeyPtr, CharacterPtr);
				uint32 Key = KeyPtr ? *KeyPtr : 0;
				uint32 Character = CharacterPtr ? *CharacterPtr : 0;
				if (Key != 0 || Character != 0)
				{
					MessageHandler->OnKeyDown((int32)Key, (int32)Character, false);
					MessageHandler->OnKeyUp((int32)Key, (int32)Character, false);
				}
			}
		});
	}

	void FRTCInputHandler::PopulateDefaultCommandHandlers()
	{
		// Execute console commands if passed "ConsoleCommand" and -PixelStreaming2AllowConsoleCommands is on.
		CommandHandlers.Add(TEXT("ConsoleCommand"), [this](FString SourceId, FString Descriptor, FString ConsoleCommand) {
			if (!UPixelStreaming2PluginSettings::CVarInputAllowConsoleCommands.GetValueOnAnyThread()
				|| !IsElevated(SourceId))
			{
				return;
			}
			GEngine->Exec(GEngine->GetWorld(), *ConsoleCommand);
		});

		// Change width/height if sent { "Resolution.Width": 1920, "Resolution.Height": 1080 }
		CommandHandlers.Add(TEXT("Resolution.Width"), [this](FString SourceId, FString Descriptor, FString WidthString) {
			bool	bSuccess = false;
			FString HeightString;
			ExtractJsonFromDescriptor(Descriptor, TEXT("Resolution.Height"), HeightString, bSuccess);
			if (bSuccess && IsElevated(SourceId))
			{
				int Width = FCString::Atoi(*WidthString);
				int Height = FCString::Atoi(*HeightString);
				if (Width < 1 || Height < 1)
				{
					return;
				}

				FString ChangeResCommand = FString::Printf(TEXT("r.SetRes %dx%d"), Width, Height);
				GEngine->Exec(GEngine->GetWorld(), *ChangeResCommand);
			}
		});

		// Response to "Stat.FPS" by calling "stat fps"
		CommandHandlers.Add(TEXT("Stat.FPS"), [](FString SourceId, FString Descriptor, FString FPSCommand) {
			FString StatFPSCommand = FString::Printf(TEXT("stat fps"));
			GEngine->Exec(GEngine->GetWorld(), *StatFPSCommand);
		});
	}

	FMatrix FRTCInputHandler::ExtractWebXRMatrix(FMemoryReader& Ar)
	{
		FMatrix OutMat;
		for (int32 Row = 0; Row < 4; ++Row)

		{
			float Col0 = 0.0f, Col1 = 0.0f, Col2 = 0.0f, Col3 = 0.0f;
			Ar << Col0 << Col1 << Col2 << Col3;
			OutMat.M[Row][0] = Col0;
			OutMat.M[Row][1] = Col1;
			OutMat.M[Row][2] = Col2;
			OutMat.M[Row][3] = Col3;
		}
		OutMat.DiagnosticCheckNaN();
		return OutMat;
	}

	FTransform FRTCInputHandler::WebXRMatrixToUETransform(FMatrix Mat)
	{
		// Rows and columns are swapped between raw mat and FMat
		FMatrix UEMatrix = FMatrix(
			FPlane(Mat.M[0][0], Mat.M[1][0], Mat.M[2][0], Mat.M[3][0]),
			FPlane(Mat.M[0][1], Mat.M[1][1], Mat.M[2][1], Mat.M[3][1]),
			FPlane(Mat.M[0][2], Mat.M[1][2], Mat.M[2][2], Mat.M[3][2]),
			FPlane(Mat.M[0][3], Mat.M[1][3], Mat.M[2][3], Mat.M[3][3]));

		// Extract scale vector and reorder coordinates to be UE coordinate system.
		FVector ScaleVectorRaw = UEMatrix.GetScaleVector();
		// Note: We do not invert Z scaling here because we already handle that when we rebuild translation/rot below.
		FVector ScaleVector = FVector(ScaleVectorRaw.Z, ScaleVectorRaw.X, ScaleVectorRaw.Y);

		// Temporarily remove scaling component as we need rotation axes to be unit length for proper quat conversion
		UEMatrix.RemoveScaling();

		// Extract & convert translation component to UE coordinate syste,
		FVector Translation = FVector(-UEMatrix.M[3][2], UEMatrix.M[3][0], UEMatrix.M[3][1]) * 100.0f;

		// Extract & convert rotation component to UE coordinate system
		FQuat RawRotation(UEMatrix);
		FQuat Rotation(-RawRotation.Z, RawRotation.X, RawRotation.Y, -RawRotation.W);
		return FTransform(Rotation, Translation, ScaleVector);
	}
} // namespace UE::PixelStreaming2