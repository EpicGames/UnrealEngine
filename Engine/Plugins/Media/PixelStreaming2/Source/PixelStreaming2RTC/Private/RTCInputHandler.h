// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputHandler.h"

namespace UE::PixelStreaming2
{
	class FRTCInputHandler : public UE::PixelStreaming2Input::FPixelStreaming2InputHandler
	{
	public:
		static TSharedPtr<FRTCInputHandler> Create();

		virtual ~FRTCInputHandler();

	protected:
		FRTCInputHandler();
		/**
		 * Touch handling
		 */
		void HandleOnTouchStarted(FMemoryReader Ar);
		void HandleOnTouchMoved(FMemoryReader Ar);
		void HandleOnTouchEnded(FMemoryReader Ar);
		/**
		 * Key press handling
		 */
		void HandleOnKeyChar(FMemoryReader Ar);
		void HandleOnKeyDown(FMemoryReader Ar);
		void HandleOnKeyUp(FMemoryReader Ar);
		/**
		 * Mouse handling
		 */
		void HandleOnMouseEnter(FMemoryReader Ar);
		void HandleOnMouseLeave(FMemoryReader Ar);
		void HandleOnMouseDown(FMemoryReader Ar);
		void HandleOnMouseUp(FMemoryReader Ar);
		void HandleOnMouseMove(FMemoryReader Ar);
		void HandleOnMouseWheel(FMemoryReader Ar);
		void HandleOnMouseDoubleClick(FMemoryReader Ar);
		/**
		 * Controller handling
		 */
		void HandleOnControllerConnected(FMemoryReader Ar);
		void HandleOnControllerAnalog(FMemoryReader Ar);
		void HandleOnControllerButtonPressed(FMemoryReader Ar);
		void HandleOnControllerButtonReleased(FMemoryReader Ar);
		void HandleOnControllerDisconnected(FMemoryReader Ar);
		/**
		 * XR handling
		 */
		void HandleOnXREyeViews(FMemoryReader Ar);
		void HandleOnXRHMDTransform(FMemoryReader Ar);
		void HandleOnXRControllerTransform(FMemoryReader Ar);
		void HandleOnXRButtonTouched(FMemoryReader Ar);
		void HandleOnXRButtonTouchReleased(FMemoryReader Ar);
		void HandleOnXRButtonPressed(FMemoryReader Ar);
		void HandleOnXRButtonReleased(FMemoryReader Ar);
		void HandleOnXRAnalog(FMemoryReader Ar);
		void HandleOnXRSystem(FMemoryReader Ar);
		/**
		 * Command handling
		 */
		void HandleOnCommand(FString SourceId, FMemoryReader Ar);
		/**
		 * UI Interaction handling
		 */
		void HandleUIInteraction(FMemoryReader Ar);
		/**
		 * Textbox Entry handling
		 */
		void HandleOnTextboxEntry(FMemoryReader Ar);

	private:
		/**
		 * Populate default command handlers for data channel messages sent with "{ type: "Command" }".
		 */
		void PopulateDefaultCommandHandlers();

		/**
		 * Extract 4x4 WebXR ordered matrix and convert to FMatrix.
		 */
		FMatrix ExtractWebXRMatrix(FMemoryReader& Ar);

		/**
		 * Converts the 'Y up' 'right handed' WebXR coordinate system transform to Unreal's 'Z up'
		 * 'left handed' coordinate system. Note: Ignores scale.
		 * Assumes WebXR conforms to the following: (https://developer.mozilla.org/en-US/docs/Web/API/WebXR_Device_API/Geometry)
		 * @return A 4x4 z-up transform matrix for use with UE.
		 */
		FTransform WebXRMatrixToUETransform(FMatrix Mat);

		TMap<TTuple<EPixelStreaming2XRSystem, EControllerHand, uint8, EPixelStreaming2InputAction>, FKey> XRInputToFKey;
		TMap<TTuple<uint8, EPixelStreaming2InputAction>, FKey>											  GamepadInputToFKey;
	};
} // namespace UE::PixelStreaming2