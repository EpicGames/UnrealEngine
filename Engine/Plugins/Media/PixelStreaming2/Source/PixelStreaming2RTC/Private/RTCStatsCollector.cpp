// Copyright Epic Games, Inc. All Rights Reserved.

#include "RTCStatsCollector.h"

#include "Logging.h"
#include "PixelStreaming2PluginSettings.h"
#include "PixelStreaming2StatNames.h"
#include "EpicRtcStreamer.h"
#include "UtilsString.h"
#include "Stats.h"

namespace UE::PixelStreaming2
{
	/**
	 * ---------- FStat ----------
	 */

	FStat::FStat(FStatConfig Config, double InitialValue, int NDecimalPlacesToPrint, bool bSmooth)
		: Name(Config.Name)
		, DisplayFlags(Config.DisplayFlags)
		, Alias(Config.Alias)
		, NDecimalPlacesToPrint(NDecimalPlacesToPrint)
		, bSmooth(bSmooth)
		, StatVariant(TInPlaceType<double>(), InitialValue)
	{
	}

	FStat::FStat(FStatConfig Config, FString InitialValue)
		: Name(Config.Name)
		, DisplayFlags(Config.DisplayFlags)
		, Alias(Config.Alias)
		, StatVariant(TInPlaceType<FString>(), InitialValue)
	{
		checkf(!(Config.DisplayFlags & EDisplayFlags::GRAPH), TEXT("Text based stats cannot be graphed"));
	}

	FStat::FStat(FStatConfig Config, bool bInitialValue)
		: Name(Config.Name)
		, DisplayFlags(Config.DisplayFlags)
		, Alias(Config.Alias)
		, StatVariant(TInPlaceType<bool>(), bInitialValue)
	{
		checkf(!(Config.DisplayFlags & EDisplayFlags::GRAPH), TEXT("Boolean based stats cannot be graphed"));
	}

	FStat::FStat(const FStat& Other)
		: Name(Other.Name)
		, DisplayFlags(Other.DisplayFlags)
		, Alias(Other.Alias)
		, NDecimalPlacesToPrint(Other.NDecimalPlacesToPrint)
		, bSmooth(Other.bSmooth)
		, StatVariant(Other.StatVariant)
	{
	}

	bool FStat::IsNumeric() const
	{
		return StatVariant.GetIndex() == FStatVariant::IndexOfType<double>();
	}

	bool FStat::IsTextual() const
	{
		return StatVariant.GetIndex() == FStatVariant::IndexOfType<FString>();
	}

	bool FStat::IsBoolean() const
	{
		return StatVariant.GetIndex() == FStatVariant::IndexOfType<bool>();
	}

	FString FStat::ToString()
	{
		switch (StatVariant.GetIndex())
		{
			case FStatVariant::IndexOfType<FString>():
			{
				return StatVariant.Get<FString>();
			}
			case FStatVariant::IndexOfType<double>():
			{
				return FString::Printf(TEXT("%.*f"), NDecimalPlacesToPrint, StatVariant.Get<double>());
			}
			case FStatVariant::IndexOfType<bool>():
			{
				return FString::Printf(TEXT("%s"), StatVariant.Get<bool>() ? TEXT("true") : TEXT("false"));
			}
			default:
				checkNoEntry();
				return TEXT("");
		}
	}

	bool FStat::SetValue(FStatVariant ValueVariant)
	{
		if (ValueVariant.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
		{
			return false;
		}

		if (ValueVariant.GetIndex() != StatVariant.GetIndex())
		{
			FString ValueVariantType = TEXT("TYPE_OF_NULLPTR");
			switch (ValueVariant.GetIndex())
			{
				case FStatVariant::IndexOfType<FString>():
				{
					ValueVariantType = TEXT("FString");
					break;
				}
				case FStatVariant::IndexOfType<double>():
				{
					ValueVariantType = TEXT("double");
					break;
				}
				case FStatVariant::IndexOfType<bool>():
				{
					ValueVariantType = TEXT("bool");
					break;
				}
				default:
					checkNoEntry();
					break;
			}

			FString StatVariantType = TEXT("TYPE_OF_NULLPTR");
			switch (StatVariant.GetIndex())
			{
				case FStatVariant::IndexOfType<FString>():
				{
					StatVariantType = TEXT("FString");
					break;
				}
				case FStatVariant::IndexOfType<double>():
				{
					StatVariantType = TEXT("double");
					break;
				}
				case FStatVariant::IndexOfType<bool>():
				{
					StatVariantType = TEXT("bool");
					break;
				}
				default:
					checkNoEntry();
					break;
			}

			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Attempted to assign a {0} to a {1} stat!. The operation wasn't successful!", ValueVariantType, StatVariantType);
			return false;
		}

		PrevStatVariant = StatVariant;

		switch (ValueVariant.GetIndex())
		{
			case FStatVariant::IndexOfType<FString>():
			{
				FString PrevValue = StatVariant.Get<FString>();
				FString NewValue = ValueVariant.Get<FString>();

				StatVariant.Set<FString>(NewValue);

				return PrevValue != NewValue;
			}
			case FStatVariant::IndexOfType<double>():
			{
				double PrevValue = StatVariant.Get<double>();
				double NewValue = ValueVariant.Get<double>();

				if (bSmooth)
				{
					const int MaxSamples = 60;
					const int NumSamplesToUse = FGenericPlatformMath::Min(MaxSamples, NumSamples + 1);

					if (NumSamplesToUse < MaxSamples)
					{
						NewValue = CalcMA(PrevValue, NumSamples - 1, NewValue);
					}
					else
					{
						NewValue = CalcEMA(PrevValue, NumSamples - 1, NewValue);
					}
					NumSamples++;
				}

				StatVariant.Set<double>(NewValue);

				return PrevValue != NewValue;
			}
			case FStatVariant::IndexOfType<bool>():
			{
				bool PrevValue = StatVariant.Get<bool>();
				bool NewValue = ValueVariant.Get<bool>();

				StatVariant.Set<bool>(NewValue);

				return PrevValue != NewValue;
			}
			default:
				checkNoEntry();
				return false;
		}
	}

	template <>
	FString FStat::GetValue()
	{
		if (!IsTextual())
		{
			checkf(false, TEXT("Tried to get a string value from a non-string stat!"));
			return TEXT("");
		}

		return StatVariant.Get<FString>();
	}

	template <>
	double FStat::GetValue()
	{
		if (!IsNumeric())
		{
			checkf(false, TEXT("Tried to get a numeric value from a non-numeric stat!"));
			return -1.f;
		}

		return StatVariant.Get<double>();
	}

	template <>
	bool FStat::GetValue()
	{
		if (!IsBoolean())
		{
			checkf(false, TEXT("Tried to get a boolean value from a non-boolean stat!"));
			return false;
		}

		return StatVariant.Get<bool>();
	}

	template <>
	FString FStat::GetPrevValue()
	{
		if (PrevStatVariant.GetIndex() != FStatVariant::IndexOfType<FString>())
		{
			checkf(false, TEXT("Tried to get a string value from a non-string stat!"));
			return TEXT("");
		}

		return PrevStatVariant.Get<FString>();
	}

	template <>
	double FStat::GetPrevValue()
	{
		if (!IsNumeric())
		{
			checkf(false, TEXT("Tried to get a numeric value from a non-numeric stat!"));
			return -1.f;
		}

		return PrevStatVariant.Get<double>();
	}

	template <>
	bool FStat::GetPrevValue()
	{
		if (!IsBoolean())
		{
			checkf(false, TEXT("Tried to get a boolean value from a non-boolean stat!"));
			return false;
		}

		return PrevStatVariant.Get<bool>();
	}

	bool FStat::operator==(const FStat& Other) const
	{
		return Name == Other.Name;
	}

	bool FStat::IsHidden()
	{
		return DisplayFlags == EDisplayFlags::HIDDEN;
	}

	bool FStat::ShouldGraph()
	{
		return DisplayFlags & EDisplayFlags::GRAPH;
	}

	bool FStat::ShouldDisplayText()
	{
		return DisplayFlags & EDisplayFlags::TEXT;
	}

	FName FStat::GetName() const
	{
		return Name;
	}

	FName FStat::GetDisplayName() const
	{
		return Alias.Get(Name);
	}

	double FStat::CalcMA(double InPrevAvg, int InNumSamples, double InValue)
	{
		const double Result = InNumSamples * InPrevAvg + InValue;
		return Result / (InPrevAvg + 1.0);
	}

	double FStat::CalcEMA(double InPrevAvg, int InNumSamples, double InValue)
	{
		const double Mult = 2.0 / (InNumSamples + 1.0);
		const double Result = (InValue - InPrevAvg) * Mult + InPrevAvg;
		return Result;
	}

	TSharedPtr<FRTCStatsCollector> FRTCStatsCollector::Create(const FString& PlayerId)
	{
		TSharedPtr<FRTCStatsCollector> StatsCollector = TSharedPtr<FRTCStatsCollector>(new FRTCStatsCollector(PlayerId));

		if (UPixelStreaming2PluginSettings::FDelegates* Delegates = UPixelStreaming2PluginSettings::Delegates())
		{
			Delegates->OnWebRTCDisableStatsChanged.AddSP(StatsCollector.ToSharedRef(), &FRTCStatsCollector::OnWebRTCDisableStatsChanged);
		}

		return StatsCollector;
	}

	FRTCStatsCollector::FRTCStatsCollector()
		: FRTCStatsCollector(INVALID_PLAYER_ID)
	{
	}

	FRTCStatsCollector::FRTCStatsCollector(const FString& PlayerId)
		: AssociatedPlayerId(PlayerId)
		, LastCalculationCycles(FPlatformTime::Cycles64())
		, bIsEnabled(!UPixelStreaming2PluginSettings::CVarWebRTCDisableStats.GetValueOnAnyThread())
		, CandidatePairStatsSink(MakeUnique<FCandidatePairStatsSink>(FName(*RTCStatCategories::CandidatePair)))
	{
	}

	void FRTCStatsCollector::OnWebRTCDisableStatsChanged(IConsoleVariable* Var)
	{
		bIsEnabled = !Var->GetBool();
	}

	void FRTCStatsCollector::Process(const EpicRtcConnectionStats& InStats)
	{
		FStats* PSStats = FStats::Get();
		if (!bIsEnabled || !PSStats || IsEngineExitRequested())
		{
			return;
		}

		uint64 CyclesNow = FPlatformTime::Cycles64();
		double SecondsDelta = FGenericPlatformTime::ToSeconds64(CyclesNow - LastCalculationCycles);

		// Local video stats
		for (uint64 i = 0; i < InStats._localVideoTracks._size; i++)
		{
			const EpicRtcLocalVideoTrackStats& LocalVideoTrackStats = InStats._localVideoTracks._ptr[i];

			// Process video source stats
			if (!VideoSourceSinks.Contains(i))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu]"), *RTCStatCategories::VideoSource, i));
				VideoSourceSinks.Add(i, MakeUnique<FVideoSourceStatsSink>(SinkName));
			}

			VideoSourceSinks[i]->Process(LocalVideoTrackStats._source, AssociatedPlayerId, SecondsDelta);

			// Process video codec stats
			if (!VideoCodecSinks.Contains(i))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu]"), *RTCStatCategories::VideoCodec, i));
				VideoCodecSinks.Add(i, MakeUnique<FVideoCodecStatsSink>(SinkName));
			}

			VideoCodecSinks[i]->Process(LocalVideoTrackStats._codec, AssociatedPlayerId, SecondsDelta);

			// Process video track rtp stats
			if (!LocalVideoTrackSinks.Contains(i))
			{
				LocalVideoTrackSinks.Add(i, {});
			}

			TMap<uint32, TUniquePtr<FRTPLocalVideoTrackStatsSink>>& SsrcSinks = LocalVideoTrackSinks[i];
			for (int j = 0; j < LocalVideoTrackStats._rtp._size; j++)
			{
				const EpicRtcLocalTrackRtpStats& RtpStats = LocalVideoTrackStats._rtp._ptr[j];

				if (!SsrcSinks.Contains(RtpStats._local._ssrc))
				{
					FName SinkName = FName(*FString::Printf(TEXT("%s [%llu] (%s)"), *RTCStatCategories::LocalVideoTrack, i, *ToString(RtpStats._local._rid)));
					SsrcSinks.Add(RtpStats._local._ssrc, MakeUnique<FRTPLocalVideoTrackStatsSink>(SinkName));
				}

				SsrcSinks[RtpStats._local._ssrc]->Process(RtpStats, AssociatedPlayerId, SecondsDelta);
			}
		}

		// Local audio stats
		for (uint64 i = 0; i < InStats._localAudioTracks._size; i++)
		{
			const EpicRtcLocalAudioTrackStats& LocalAudioTrackStats = InStats._localAudioTracks._ptr[i];

			// Process audio source stats
			if (!AudioSourceSinks.Contains(i))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu]"), *RTCStatCategories::AudioSource, i));
				AudioSourceSinks.Add(i, MakeUnique<FAudioSourceStatsSink>(SinkName));
			}

			AudioSourceSinks[i]->Process(LocalAudioTrackStats._source, AssociatedPlayerId, SecondsDelta);

			// Process audio codec stats
			if (!AudioCodecSinks.Contains(i))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu]"), *RTCStatCategories::AudioCodec, i));
				AudioCodecSinks.Add(i, MakeUnique<FAudioCodecStatsSink>(SinkName));
			}

			AudioCodecSinks[i]->Process(LocalAudioTrackStats._codec, AssociatedPlayerId, SecondsDelta);

			// Process audio track rtp stats
			if (!LocalAudioTrackSinks.Contains(i))
			{
				LocalAudioTrackSinks.Add(i, {});
			}

			TMap<uint32, TUniquePtr<FRTPLocalAudioTrackStatsSink>>& SsrcSinks = LocalAudioTrackSinks[i];
			const EpicRtcLocalTrackRtpStats&						RtpStats = LocalAudioTrackStats._rtp;

			if (!SsrcSinks.Contains(RtpStats._local._ssrc))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu] (%s)"), *RTCStatCategories::LocalAudioTrack, i, *ToString(RtpStats._local._rid)));
				SsrcSinks.Add(RtpStats._local._ssrc, MakeUnique<FRTPLocalAudioTrackStatsSink>(SinkName));
			}

			SsrcSinks[RtpStats._local._ssrc]->Process(RtpStats, AssociatedPlayerId, SecondsDelta);
		}

		// remote video stats
		for (uint64 i = 0; i < InStats._remoteVideoTracks._size; i++)
		{
			const EpicRtcRemoteTrackStats& RemoteVideoTrackStats = InStats._remoteVideoTracks._ptr[i];

			// Process video track rtp stats
			if (!RemoteVideoTrackSinks.Contains(i))
			{
				RemoteVideoTrackSinks.Add(i, {});
			}

			TMap<uint32, TUniquePtr<FRTPRemoteTrackStatsSink>>& SsrcSinks = RemoteVideoTrackSinks[i];
			const EpicRtcRemoteTrackRtpStats&					RtpStats = RemoteVideoTrackStats._rtp;

			if (!SsrcSinks.Contains(RtpStats._local._ssrc))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu] (%u)"), *RTCStatCategories::RemoteVideoTrack, i, RtpStats._local._ssrc));
				SsrcSinks.Add(RtpStats._local._ssrc, MakeUnique<FRTPRemoteTrackStatsSink>(SinkName));
			}

			SsrcSinks[RtpStats._local._ssrc]->Process(RtpStats, AssociatedPlayerId, SecondsDelta);
		}

		// remote audio stats
		for (uint64 i = 0; i < InStats._remoteAudioTracks._size; i++)
		{
			const EpicRtcRemoteTrackStats& RemoteAudioTrackStats = InStats._remoteAudioTracks._ptr[i];

			// Process audio track rtp stats
			if (!RemoteAudioTrackSinks.Contains(i))
			{
				RemoteAudioTrackSinks.Add(i, {});
			}

			TMap<uint32, TUniquePtr<FRTPRemoteTrackStatsSink>>& SsrcSinks = RemoteAudioTrackSinks[i];
			const EpicRtcRemoteTrackRtpStats&					RtpStats = RemoteAudioTrackStats._rtp;

			if (!SsrcSinks.Contains(RtpStats._local._ssrc))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu] (%u)"), *RTCStatCategories::RemoteAudioTrack, i, RtpStats._local._ssrc));
				SsrcSinks.Add(RtpStats._local._ssrc, MakeUnique<FRTPRemoteTrackStatsSink>(SinkName));
			}

			SsrcSinks[RtpStats._local._ssrc]->Process(RtpStats, AssociatedPlayerId, SecondsDelta);
		}

		// data track stats
		for (uint64 i = 0; i < InStats._dataTracks._size; i++)
		{
			const EpicRtcDataTrackStats& DataTrackStats = InStats._dataTracks._ptr[i];

			// Process data track stats
			if (!DataTrackSinks.Contains(i))
			{
				FName SinkName = FName(*FString::Printf(TEXT("%s [%llu]"), *RTCStatCategories::DataChannel, i));
				DataTrackSinks.Add(i, MakeUnique<FDataTrackStatsSink>(SinkName));
			}

			DataTrackSinks[i]->Process(DataTrackStats, AssociatedPlayerId, SecondsDelta);
		}

		// transport stats
		if (InStats._transports._size > 0)
		{
			//(Nazar.Rudenko): More than one transport is possible only if we are not using bundle which we do
			const EpicRtcTransportStats& Transport = InStats._transports._ptr[0];
			FString						 SelectedPairId = ToString(Transport._selectedCandidatePairId);
			for (int i = 0; i < Transport._candidatePairs._size; i++)
			{
				FString PairId = ToString(Transport._candidatePairs._ptr[i]._id);
				if (SelectedPairId == PairId)
				{
					CandidatePairStatsSink->Process(Transport._candidatePairs._ptr[i], AssociatedPlayerId, SecondsDelta);
				}
			}
		}

		LastCalculationCycles = CyclesNow;
	}
	/**
	 * ---------- FRTPLocalVideoTrackSink ----------
	 */
	FRTCStatsCollector::FRTPLocalVideoTrackStatsSink::FRTPLocalVideoTrackStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// These stats will be extracted from the stat reports and emitted straight to screen
		Stats.Add(PixelStreaming2StatNames::FirCount, FStat({ .Name = PixelStreaming2StatNames::FirCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PliCount, FStat({ .Name = PixelStreaming2StatNames::PliCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::NackCount, FStat({ .Name = PixelStreaming2StatNames::NackCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RetransmittedBytesSent, FStat({ .Name = PixelStreaming2StatNames::RetransmittedBytesSent }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalEncodeBytesTarget, FStat({ .Name = PixelStreaming2StatNames::TotalEncodeBytesTarget }, 0.f));
		Stats.Add(PixelStreaming2StatNames::KeyFramesEncoded, FStat({ .Name = PixelStreaming2StatNames::KeyFramesEncoded }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FrameWidth, FStat({ .Name = PixelStreaming2StatNames::FrameWidth }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FrameHeight, FStat({ .Name = PixelStreaming2StatNames::FrameHeight }, 0.f));
		Stats.Add(PixelStreaming2StatNames::HugeFramesSent, FStat({ .Name = PixelStreaming2StatNames::HugeFramesSent }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PacketsLost, FStat({ .Name = PixelStreaming2StatNames::PacketsLost }, 0.f));
		Stats.Add(PixelStreaming2StatNames::Jitter, FStat({ .Name = PixelStreaming2StatNames::Jitter }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RoundTripTime, FStat({ .Name = PixelStreaming2StatNames::RoundTripTime }, 0.f));
		Stats.Add(PixelStreaming2StatNames::EncoderImplementation, FStat({ .Name = PixelStreaming2StatNames::EncoderImplementation }, FString(TEXT(""))));
		Stats.Add(PixelStreaming2StatNames::QualityLimitationReason, FStat({ .Name = PixelStreaming2StatNames::QualityLimitationReason }, FString(TEXT(""))));
		Stats.Add(PixelStreaming2StatNames::Rid, FStat({ .Name = PixelStreaming2StatNames::Rid }, FString(TEXT(""))));

		// These are values used to calculate extra values (stores time deltas etc)
		Stats.Add(PixelStreaming2StatNames::TargetBitrate, FStat({ .Name = PixelStreaming2StatNames::TargetBitrate, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesSent, FStat({ .Name = PixelStreaming2StatNames::FramesSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesReceived, FStat({ .Name = PixelStreaming2StatNames::FramesReceived, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesSent, FStat({ .Name = PixelStreaming2StatNames::BytesSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesReceived, FStat({ .Name = PixelStreaming2StatNames::BytesReceived, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::QPSum, FStat({ .Name = PixelStreaming2StatNames::QPSum, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalEncodeTime, FStat({ .Name = PixelStreaming2StatNames::TotalEncodeTime, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesEncoded, FStat({ .Name = PixelStreaming2StatNames::FramesEncoded, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesDecoded, FStat({ .Name = PixelStreaming2StatNames::FramesDecoded, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalPacketSendDelay, FStat({ .Name = PixelStreaming2StatNames::TotalPacketSendDelay, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PacketsSent, FStat({ .Name = PixelStreaming2StatNames::PacketsSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));

		// Calculated stats below:
		// FrameSent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* FramesSentStat = StatSource.Get(PixelStreaming2StatNames::FramesSent);
			if (FramesSentStat && FramesSentStat->GetValue<double>() > 0)
			{
				const double FramesSentPerSecond = (FramesSentStat->GetValue<double>() - FramesSentStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::FramesSentPerSecond, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, FramesSentPerSecond);
			}
			return {};
		});

		// FramesReceived Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* FramesReceivedStat = StatSource.Get(PixelStreaming2StatNames::FramesReceived);
			if (FramesReceivedStat && FramesReceivedStat->GetValue<double>() > 0)
			{
				const double FramesReceivedPerSecond = (FramesReceivedStat->GetValue<double>() - FramesReceivedStat->GetPrevValue<double>()) * Period;
				return FStat({
								 .Name = PixelStreaming2StatNames::FramesReceivedPerSecond,
							 },
					FramesReceivedPerSecond);
			}
			return {};
		});

		// Megabits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesSentStat = StatSource.Get(PixelStreaming2StatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetValue<double>() > 0)
			{
				const double BytesSentPerSecond = (BytesSentStat->GetValue<double>() - BytesSentStat->GetPrevValue<double>()) * Period;
				const double MegabitsPerSecond = BytesSentPerSecond / 1'000'000.0 * 8.0;
				return FStat({
								 .Name = PixelStreaming2StatNames::BitrateMegabits,
							 },
					MegabitsPerSecond, 2);
			}
			return {};
		});

		// Bits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesSentStat = StatSource.Get(PixelStreaming2StatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetValue<double>() > 0)
			{
				const double BytesSentPerSecond = (BytesSentStat->GetValue<double>() - BytesSentStat->GetPrevValue<double>()) * Period;
				const double BitsPerSecond = BytesSentPerSecond * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::Bitrate, .DisplayFlags = EDisplayFlags::HIDDEN }, BitsPerSecond);
			}
			return {};
		});

		// Target megabits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TargetBpsStats = StatSource.Get(PixelStreaming2StatNames::TargetBitrate);
			if (TargetBpsStats && TargetBpsStats->GetValue<double>() > 0)
			{
				const double TargetBps = (TargetBpsStats->GetValue<double>() + TargetBpsStats->GetPrevValue<double>()) * 0.5f;
				const double MegabitsPerSecond = TargetBps / 1'000'000.0;
				return FStat({ .Name = PixelStreaming2StatNames::TargetBitrateMegabits }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Megabits received Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesReceivedStat = StatSource.Get(PixelStreaming2StatNames::BytesReceived);
			if (BytesReceivedStat && BytesReceivedStat->GetValue<double>() > 0)
			{
				const double BytesReceivedPerSecond = (BytesReceivedStat->GetValue<double>() - BytesReceivedStat->GetPrevValue<double>()) * Period;
				const double MegabitsPerSecond = BytesReceivedPerSecond / 1000.0 * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::Bitrate }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Encoded fps
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* EncodedFramesStat = StatSource.Get(PixelStreaming2StatNames::FramesEncoded);
			if (EncodedFramesStat && EncodedFramesStat->GetValue<double>() > 0)
			{
				const double EncodedFramesPerSecond = (EncodedFramesStat->GetValue<double>() - EncodedFramesStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::EncodedFramesPerSecond }, EncodedFramesPerSecond);
			}
			return {};
		});

		// Decoded fps
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* DecodedFramesStat = StatSource.Get(PixelStreaming2StatNames::FramesDecoded);
			if (DecodedFramesStat && DecodedFramesStat->GetValue<double>() > 0)
			{
				const double DecodedFramesPerSecond = (DecodedFramesStat->GetValue<double>() - DecodedFramesStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::DecodedFramesPerSecond }, DecodedFramesPerSecond);
			}
			return {};
		});

		// Avg QP Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* QPSumStat = StatSource.Get(PixelStreaming2StatNames::QPSum);
			FStat* EncodedFramesPerSecond = StatSource.Get(PixelStreaming2StatNames::EncodedFramesPerSecond);
			if (QPSumStat && QPSumStat->GetValue<double>() > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->GetValue<double>() > 0.0)
			{
				const double QPSumDeltaPerSecond = (QPSumStat->GetValue<double>() - QPSumStat->GetPrevValue<double>()) * Period;
				const double MeanQPPerFrame = QPSumDeltaPerSecond / EncodedFramesPerSecond->GetValue<double>();
				return FStat({ .Name = PixelStreaming2StatNames::MeanQPPerSecond }, MeanQPPerFrame);
			}
			return {};
		});

		// Mean EncodeTime (ms) Per Frame
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TotalEncodeTimeStat = StatSource.Get(PixelStreaming2StatNames::TotalEncodeTime);
			FStat* EncodedFramesPerSecond = StatSource.Get(PixelStreaming2StatNames::EncodedFramesPerSecond);
			if (TotalEncodeTimeStat && TotalEncodeTimeStat->GetValue<double>() > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->GetValue<double>() > 0.0)
			{
				const double TotalEncodeTimePerSecond = (TotalEncodeTimeStat->GetValue<double>() - TotalEncodeTimeStat->GetPrevValue<double>()) * Period;
				const double MeanEncodeTimePerFrameMs = TotalEncodeTimePerSecond / EncodedFramesPerSecond->GetValue<double>() * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::MeanEncodeTime }, MeanEncodeTimePerFrameMs, 2);
			}
			return {};
		});

		// Mean SendDelay (ms) Per Frame
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TotalSendDelayStat = StatSource.Get(PixelStreaming2StatNames::TotalPacketSendDelay);
			FStat* TotalPacketsSent = StatSource.Get(PixelStreaming2StatNames::PacketsSent);
			if (TotalSendDelayStat && TotalSendDelayStat->GetValue<double>() > 0.0
				&& TotalPacketsSent && TotalPacketsSent->GetValue<double>() > 0.0)
			{
				const double MeanSendDelayPerFrameMs = (TotalSendDelayStat->GetValue<double>() / TotalPacketsSent->GetValue<double>()) * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::MeanSendDelay }, MeanSendDelayPerFrameMs, 2);
			}
			return {};
		});

		// JitterBufferDelay (ms)
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* JitterBufferDelayStat = StatSource.Get(PixelStreaming2StatNames::JitterBufferDelay);
			FStat* FramesReceivedPerSecond = StatSource.Get(PixelStreaming2StatNames::FramesReceivedPerSecond);
			if (JitterBufferDelayStat && JitterBufferDelayStat->GetValue<double>() > 0
				&& FramesReceivedPerSecond && FramesReceivedPerSecond->GetValue<double>() > 0.0)
			{
				const double TotalJitterBufferDelayPerSecond = (JitterBufferDelayStat->GetValue<double>() - JitterBufferDelayStat->GetPrevValue<double>()) * Period;
				const double MeanJitterBufferDelayMs = TotalJitterBufferDelayPerSecond / FramesReceivedPerSecond->GetValue<double>() * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::JitterBufferDelay }, MeanJitterBufferDelayMs, 2);
			}
			return {};
		});
	}

	void FRTCStatsCollector::FRTPLocalVideoTrackStatsSink::Process(const EpicRtcLocalTrackRtpStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::FirCount)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._firCount);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::PliCount)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._pliCount);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::NackCount)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._nackCount);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::RetransmittedBytesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._retransmittedBytesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::TotalEncodeBytesTarget)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._totalEncodedBytesTarget);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::KeyFramesEncoded)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._keyFramesEncoded);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FrameWidth)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._frameWidth);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FrameHeight)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._frameHeight);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::HugeFramesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._hugeFramesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::TotalPacketSendDelay)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._totalPacketSendDelay);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::TargetBitrate)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._targetBitrate);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FramesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._framesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FramesReceived)
			{
				// TODO(Nazar.Rudenko): Available for inbound tracks only
			}
			else if (Tuple.Key == PixelStreaming2StatNames::BytesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._bytesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::BytesReceived)
			{
				// TODO(Nazar.Rudenko): Available for inbound tracks only
			}
			else if (Tuple.Key == PixelStreaming2StatNames::QPSum)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._qpSum);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::TotalEncodeTime)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._totalEncodeTime);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FramesEncoded)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._framesEncoded);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FramesDecoded)
			{
				// TODO(Nazar.Rudenko): Available for inbound tracks only
			}
			else if (Tuple.Key == PixelStreaming2StatNames::EncoderImplementation)
			{
				NewValue = FStatVariant(TInPlaceType<FString>(), ToString(InStats._local._encoderImplementation));
			}
			else if (Tuple.Key == PixelStreaming2StatNames::PacketsSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._packetsSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::PacketsLost)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._packetsLost);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::Jitter)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._jitter);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::RoundTripTime)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._roundTripTime);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::QualityLimitationReason)
			{
				NewValue = FStatVariant(TInPlaceType<FString>(), ToString(InStats._local._qualityLimitationReason));
			}
			else if (Tuple.Key == PixelStreaming2StatNames::Rid)
			{
				NewValue = FStatVariant(TInPlaceType<FString>(), ToString(InStats._local._rid));
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPLocalAudioTrackStatsSink ----------
	 */
	FRTCStatsCollector::FRTPLocalAudioTrackStatsSink::FRTPLocalAudioTrackStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// These stats will be extracted from the stat reports and emitted straight to screen
		Stats.Add(PixelStreaming2StatNames::FirCount, FStat({ .Name = PixelStreaming2StatNames::FirCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PliCount, FStat({ .Name = PixelStreaming2StatNames::PliCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::NackCount, FStat({ .Name = PixelStreaming2StatNames::NackCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RetransmittedBytesSent, FStat({ .Name = PixelStreaming2StatNames::RetransmittedBytesSent }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalEncodeBytesTarget, FStat({ .Name = PixelStreaming2StatNames::TotalEncodeBytesTarget }, 0.f));
		Stats.Add(PixelStreaming2StatNames::KeyFramesEncoded, FStat({ .Name = PixelStreaming2StatNames::KeyFramesEncoded }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FrameWidth, FStat({ .Name = PixelStreaming2StatNames::FrameWidth }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FrameHeight, FStat({ .Name = PixelStreaming2StatNames::FrameHeight }, 0.f));
		Stats.Add(PixelStreaming2StatNames::HugeFramesSent, FStat({ .Name = PixelStreaming2StatNames::HugeFramesSent }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PacketsLost, FStat({ .Name = PixelStreaming2StatNames::PacketsLost }, 0.f));
		Stats.Add(PixelStreaming2StatNames::Jitter, FStat({ .Name = PixelStreaming2StatNames::Jitter }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RoundTripTime, FStat({ .Name = PixelStreaming2StatNames::RoundTripTime }, 0.f));

		// These are values used to calculate extra values (stores time deltas etc)
		Stats.Add(PixelStreaming2StatNames::TargetBitrate, FStat({ .Name = PixelStreaming2StatNames::TargetBitrate, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesSent, FStat({ .Name = PixelStreaming2StatNames::FramesSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesReceived, FStat({ .Name = PixelStreaming2StatNames::FramesReceived, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesSent, FStat({ .Name = PixelStreaming2StatNames::BytesSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesReceived, FStat({ .Name = PixelStreaming2StatNames::BytesReceived, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::QPSum, FStat({ .Name = PixelStreaming2StatNames::QPSum, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalEncodeTime, FStat({ .Name = PixelStreaming2StatNames::TotalEncodeTime, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesEncoded, FStat({ .Name = PixelStreaming2StatNames::FramesEncoded, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesDecoded, FStat({ .Name = PixelStreaming2StatNames::FramesDecoded, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalPacketSendDelay, FStat({ .Name = PixelStreaming2StatNames::TotalPacketSendDelay, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));

		// Calculated stats below:
		// FrameSent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* FramesSentStat = StatSource.Get(PixelStreaming2StatNames::FramesSent);
			if (FramesSentStat && FramesSentStat->GetValue<double>() > 0)
			{
				const double FramesSentPerSecond = (FramesSentStat->GetValue<double>() - FramesSentStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::FramesSentPerSecond, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, FramesSentPerSecond);
			}
			return {};
		});

		// FramesReceived Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* FramesReceivedStat = StatSource.Get(PixelStreaming2StatNames::FramesReceived);
			if (FramesReceivedStat && FramesReceivedStat->GetValue<double>() > 0)
			{
				const double FramesReceivedPerSecond = (FramesReceivedStat->GetValue<double>() - FramesReceivedStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::FramesReceivedPerSecond }, FramesReceivedPerSecond);
			}
			return {};
		});

		// Megabits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesSentStat = StatSource.Get(PixelStreaming2StatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetValue<double>() > 0)
			{
				const double BytesSentPerSecond = (BytesSentStat->GetValue<double>() - BytesSentStat->GetPrevValue<double>()) * Period;
				const double MegabitsPerSecond = BytesSentPerSecond / 1'000'000.0 * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::BitrateMegabits }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Bits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesSentStat = StatSource.Get(PixelStreaming2StatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetValue<double>() > 0)
			{
				const double BytesSentPerSecond = (BytesSentStat->GetValue<double>() - BytesSentStat->GetPrevValue<double>()) * Period;
				const double BitsPerSecond = BytesSentPerSecond * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::Bitrate, .DisplayFlags = EDisplayFlags::HIDDEN }, BitsPerSecond);
			}
			return {};
		});

		// Target megabits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TargetBpsStats = StatSource.Get(PixelStreaming2StatNames::TargetBitrate);
			if (TargetBpsStats && TargetBpsStats->GetValue<double>() > 0)
			{
				const double TargetBps = (TargetBpsStats->GetValue<double>() + TargetBpsStats->GetPrevValue<double>()) * 0.5f;
				const double MegabitsPerSecond = TargetBps / 1'000'000.0;
				return FStat({ .Name = PixelStreaming2StatNames::TargetBitrateMegabits }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Megabits received Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesReceivedStat = StatSource.Get(PixelStreaming2StatNames::BytesReceived);
			if (BytesReceivedStat && BytesReceivedStat->GetValue<double>() > 0)
			{
				const double BytesReceivedPerSecond = (BytesReceivedStat->GetValue<double>() - BytesReceivedStat->GetPrevValue<double>()) * Period;
				const double MegabitsPerSecond = BytesReceivedPerSecond / 1000.0 * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::Bitrate }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Encoded fps
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* EncodedFramesStat = StatSource.Get(PixelStreaming2StatNames::FramesEncoded);
			if (EncodedFramesStat && EncodedFramesStat->GetValue<double>() > 0)
			{
				const double EncodedFramesPerSecond = (EncodedFramesStat->GetValue<double>() - EncodedFramesStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::EncodedFramesPerSecond }, EncodedFramesPerSecond);
			}
			return {};
		});

		// Decoded fps
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* DecodedFramesStat = StatSource.Get(PixelStreaming2StatNames::FramesDecoded);
			if (DecodedFramesStat && DecodedFramesStat->GetValue<double>() > 0)
			{
				const double DecodedFramesPerSecond = (DecodedFramesStat->GetValue<double>() - DecodedFramesStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::DecodedFramesPerSecond }, DecodedFramesPerSecond);
			}
			return {};
		});

		// Avg QP Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* QPSumStat = StatSource.Get(PixelStreaming2StatNames::QPSum);
			FStat* EncodedFramesPerSecond = StatSource.Get(PixelStreaming2StatNames::EncodedFramesPerSecond);
			if (QPSumStat && QPSumStat->GetValue<double>() > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->GetValue<double>() > 0.0)
			{
				const double QPSumDeltaPerSecond = (QPSumStat->GetValue<double>() - QPSumStat->GetPrevValue<double>()) * Period;
				const double MeanQPPerFrame = QPSumDeltaPerSecond / EncodedFramesPerSecond->GetValue<double>();
				return FStat({ .Name = PixelStreaming2StatNames::MeanQPPerSecond }, MeanQPPerFrame);
			}
			return {};
		});

		// Mean EncodeTime (ms) Per Frame
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TotalEncodeTimeStat = StatSource.Get(PixelStreaming2StatNames::TotalEncodeTime);
			FStat* EncodedFramesPerSecond = StatSource.Get(PixelStreaming2StatNames::EncodedFramesPerSecond);
			if (TotalEncodeTimeStat && TotalEncodeTimeStat->GetValue<double>() > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->GetValue<double>() > 0.0)
			{
				const double TotalEncodeTimePerSecond = (TotalEncodeTimeStat->GetValue<double>() - TotalEncodeTimeStat->GetPrevValue<double>()) * Period;
				const double MeanEncodeTimePerFrameMs = TotalEncodeTimePerSecond / EncodedFramesPerSecond->GetValue<double>() * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::MeanEncodeTime }, MeanEncodeTimePerFrameMs, 2);
			}
			return {};
		});

		// Mean SendDelay (ms) Per Frame
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TotalSendDelayStat = StatSource.Get(PixelStreaming2StatNames::TotalPacketSendDelay);
			FStat* TotalPacketsSent = StatSource.Get(PixelStreaming2StatNames::PacketsSent);
			if (TotalSendDelayStat && TotalSendDelayStat->GetValue<double>() > 0.0
				&& TotalPacketsSent && TotalPacketsSent->GetValue<double>() > 0.0)
			{
				const double MeanSendDelayPerFrameMs = (TotalSendDelayStat->GetValue<double>() / TotalPacketsSent->GetValue<double>()) * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::MeanSendDelay }, MeanSendDelayPerFrameMs, 2);
			}
			return {};
		});

		// JitterBufferDelay (ms)
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* JitterBufferDelayStat = StatSource.Get(PixelStreaming2StatNames::JitterBufferDelay);
			FStat* FramesReceivedPerSecond = StatSource.Get(PixelStreaming2StatNames::FramesReceivedPerSecond);
			if (JitterBufferDelayStat && JitterBufferDelayStat->GetValue<double>() > 0
				&& FramesReceivedPerSecond && FramesReceivedPerSecond->GetValue<double>() > 0.0)
			{
				const double TotalJitterBufferDelayPerSecond = (JitterBufferDelayStat->GetValue<double>() - JitterBufferDelayStat->GetPrevValue<double>()) * Period;
				const double MeanJitterBufferDelayMs = TotalJitterBufferDelayPerSecond / FramesReceivedPerSecond->GetValue<double>() * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::JitterBufferDelay }, MeanJitterBufferDelayMs, 2);
			}
			return {};
		});
	}

	void FRTCStatsCollector::FRTPLocalAudioTrackStatsSink::Process(const EpicRtcLocalTrackRtpStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::TotalPacketSendDelay)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._totalPacketSendDelay);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::TargetBitrate)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._targetBitrate);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::BytesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._bytesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::PacketsLost)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._packetsLost);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::Jitter)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._jitter);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::RoundTripTime)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._roundTripTime);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPRemoteTrackStatsSink ----------
	 */
	FRTCStatsCollector::FRTPRemoteTrackStatsSink::FRTPRemoteTrackStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// These stats will be extracted from the stat reports and emitted straight to screen
		Stats.Add(PixelStreaming2StatNames::FirCount, FStat({ .Name = PixelStreaming2StatNames::FirCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PliCount, FStat({ .Name = PixelStreaming2StatNames::PliCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::NackCount, FStat({ .Name = PixelStreaming2StatNames::NackCount }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RetransmittedBytesReceived, FStat({ .Name = PixelStreaming2StatNames::RetransmittedBytesReceived }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RetransmittedPacketsReceived, FStat({ .Name = PixelStreaming2StatNames::RetransmittedPacketsReceived }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalEncodeBytesTarget, FStat({ .Name = PixelStreaming2StatNames::TotalEncodeBytesTarget }, 0.f));
		Stats.Add(PixelStreaming2StatNames::KeyFramesDecoded, FStat({ .Name = PixelStreaming2StatNames::KeyFramesDecoded }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FrameWidth, FStat({ .Name = PixelStreaming2StatNames::FrameWidth }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FrameHeight, FStat({ .Name = PixelStreaming2StatNames::FrameHeight }, 0.f));
		Stats.Add(PixelStreaming2StatNames::HugeFramesSent, FStat({ .Name = PixelStreaming2StatNames::HugeFramesSent }, 0.f));
		Stats.Add(PixelStreaming2StatNames::PacketsLost, FStat({ .Name = PixelStreaming2StatNames::PacketsLost }, 0.f));
		Stats.Add(PixelStreaming2StatNames::Jitter, FStat({ .Name = PixelStreaming2StatNames::Jitter }, 0.f));
		Stats.Add(PixelStreaming2StatNames::RoundTripTime, FStat({ .Name = PixelStreaming2StatNames::RoundTripTime }, 0.f));
		Stats.Add(PixelStreaming2StatNames::DecoderImplementation, FStat({ .Name = PixelStreaming2StatNames::DecoderImplementation }, FString(TEXT(""))));

		// These are values used to calculate extra values (stores time deltas etc)
		Stats.Add(PixelStreaming2StatNames::TargetBitrate, FStat({ .Name = PixelStreaming2StatNames::TargetBitrate, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesSent, FStat({ .Name = PixelStreaming2StatNames::FramesSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesReceived, FStat({ .Name = PixelStreaming2StatNames::FramesReceived, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesSent, FStat({ .Name = PixelStreaming2StatNames::BytesSent, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesReceived, FStat({ .Name = PixelStreaming2StatNames::BytesReceived, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::QPSum, FStat({ .Name = PixelStreaming2StatNames::QPSum, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalEncodeTime, FStat({ .Name = PixelStreaming2StatNames::TotalEncodeTime, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesEncoded, FStat({ .Name = PixelStreaming2StatNames::FramesEncoded, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::FramesDecoded, FStat({ .Name = PixelStreaming2StatNames::FramesDecoded, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));
		Stats.Add(PixelStreaming2StatNames::TotalPacketSendDelay, FStat({ .Name = PixelStreaming2StatNames::TotalPacketSendDelay, .DisplayFlags = EDisplayFlags::HIDDEN }, 0.f));

		// Calculated stats below:
		// FrameSent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* FramesSentStat = StatSource.Get(PixelStreaming2StatNames::FramesSent);
			if (FramesSentStat && FramesSentStat->GetValue<double>() > 0)
			{
				const double FramesSentPerSecond = (FramesSentStat->GetValue<double>() - FramesSentStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::FramesSentPerSecond, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, FramesSentPerSecond);
			}
			return {};
		});

		// FramesReceived Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* FramesReceivedStat = StatSource.Get(PixelStreaming2StatNames::FramesReceived);
			if (FramesReceivedStat && FramesReceivedStat->GetValue<double>() > 0)
			{
				const double FramesReceivedPerSecond = (FramesReceivedStat->GetValue<double>() - FramesReceivedStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::FramesReceivedPerSecond }, FramesReceivedPerSecond);
			}
			return {};
		});

		// Megabits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesSentStat = StatSource.Get(PixelStreaming2StatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetValue<double>() > 0)
			{
				const double BytesSentPerSecond = (BytesSentStat->GetValue<double>() - BytesSentStat->GetPrevValue<double>()) * Period;
				const double MegabitsPerSecond = BytesSentPerSecond / 1'000'000.0 * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::BitrateMegabits }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Bits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesSentStat = StatSource.Get(PixelStreaming2StatNames::BytesSent);
			if (BytesSentStat && BytesSentStat->GetValue<double>() > 0)
			{
				const double BytesSentPerSecond = (BytesSentStat->GetValue<double>() - BytesSentStat->GetPrevValue<double>()) * Period;
				const double BitsPerSecond = BytesSentPerSecond * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::Bitrate, .DisplayFlags = EDisplayFlags::HIDDEN }, BitsPerSecond);
			}
			return {};
		});

		// Target megabits sent Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TargetBpsStats = StatSource.Get(PixelStreaming2StatNames::TargetBitrate);
			if (TargetBpsStats && TargetBpsStats->GetValue<double>() > 0)
			{
				const double TargetBps = (TargetBpsStats->GetValue<double>() + TargetBpsStats->GetPrevValue<double>()) * 0.5f;
				const double MegabitsPerSecond = TargetBps / 1'000'000.0;
				return FStat({ .Name = PixelStreaming2StatNames::TargetBitrateMegabits }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Megabits received Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* BytesReceivedStat = StatSource.Get(PixelStreaming2StatNames::BytesReceived);
			if (BytesReceivedStat && BytesReceivedStat->GetValue<double>() > 0)
			{
				const double BytesReceivedPerSecond = (BytesReceivedStat->GetValue<double>() - BytesReceivedStat->GetPrevValue<double>()) * Period;
				const double MegabitsPerSecond = BytesReceivedPerSecond / 1000.0 * 8.0;
				return FStat({ .Name = PixelStreaming2StatNames::Bitrate }, MegabitsPerSecond, 2);
			}
			return {};
		});

		// Encoded fps
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* EncodedFramesStat = StatSource.Get(PixelStreaming2StatNames::FramesEncoded);
			if (EncodedFramesStat && EncodedFramesStat->GetValue<double>() > 0)
			{
				const double EncodedFramesPerSecond = (EncodedFramesStat->GetValue<double>() - EncodedFramesStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::EncodedFramesPerSecond }, EncodedFramesPerSecond);
			}
			return {};
		});

		// Decoded fps
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* DecodedFramesStat = StatSource.Get(PixelStreaming2StatNames::FramesDecoded);
			if (DecodedFramesStat && DecodedFramesStat->GetValue<double>() > 0)
			{
				const double DecodedFramesPerSecond = (DecodedFramesStat->GetValue<double>() - DecodedFramesStat->GetPrevValue<double>()) * Period;
				return FStat({ .Name = PixelStreaming2StatNames::DecodedFramesPerSecond }, DecodedFramesPerSecond);
			}
			return {};
		});

		// Avg QP Per Second
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* QPSumStat = StatSource.Get(PixelStreaming2StatNames::QPSum);
			FStat* EncodedFramesPerSecond = StatSource.Get(PixelStreaming2StatNames::EncodedFramesPerSecond);
			if (QPSumStat && QPSumStat->GetValue<double>() > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->GetValue<double>() > 0.0)
			{
				const double QPSumDeltaPerSecond = (QPSumStat->GetValue<double>() - QPSumStat->GetPrevValue<double>()) * Period;
				const double MeanQPPerFrame = QPSumDeltaPerSecond / EncodedFramesPerSecond->GetValue<double>();
				return FStat({ .Name = PixelStreaming2StatNames::MeanQPPerSecond }, MeanQPPerFrame);
			}
			return {};
		});

		// Mean EncodeTime (ms) Per Frame
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TotalEncodeTimeStat = StatSource.Get(PixelStreaming2StatNames::TotalEncodeTime);
			FStat* EncodedFramesPerSecond = StatSource.Get(PixelStreaming2StatNames::EncodedFramesPerSecond);
			if (TotalEncodeTimeStat && TotalEncodeTimeStat->GetValue<double>() > 0
				&& EncodedFramesPerSecond && EncodedFramesPerSecond->GetValue<double>() > 0.0)
			{
				const double TotalEncodeTimePerSecond = (TotalEncodeTimeStat->GetValue<double>() - TotalEncodeTimeStat->GetPrevValue<double>()) * Period;
				const double MeanEncodeTimePerFrameMs = TotalEncodeTimePerSecond / EncodedFramesPerSecond->GetValue<double>() * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::MeanEncodeTime }, MeanEncodeTimePerFrameMs, 2);
			}
			return {};
		});

		// Mean SendDelay (ms) Per Frame
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* TotalSendDelayStat = StatSource.Get(PixelStreaming2StatNames::TotalPacketSendDelay);
			FStat* TotalPacketsSent = StatSource.Get(PixelStreaming2StatNames::PacketsSent);
			if (TotalSendDelayStat && TotalSendDelayStat->GetValue<double>() > 0.0
				&& TotalPacketsSent && TotalPacketsSent->GetValue<double>() > 0.0)
			{
				const double MeanSendDelayPerFrameMs = (TotalSendDelayStat->GetValue<double>() / TotalPacketsSent->GetValue<double>()) * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::MeanSendDelay }, MeanSendDelayPerFrameMs, 2);
			}
			return {};
		});

		// JitterBufferDelay (ms)
		Calculators.Add([](FStatsSink& StatSource, double Period) -> TOptional<FStat> {
			FStat* JitterBufferDelayStat = StatSource.Get(PixelStreaming2StatNames::JitterBufferDelay);
			FStat* FramesReceivedPerSecond = StatSource.Get(PixelStreaming2StatNames::FramesReceivedPerSecond);
			if (JitterBufferDelayStat && JitterBufferDelayStat->GetValue<double>() > 0
				&& FramesReceivedPerSecond && FramesReceivedPerSecond->GetValue<double>() > 0.0)
			{
				const double TotalJitterBufferDelayPerSecond = (JitterBufferDelayStat->GetValue<double>() - JitterBufferDelayStat->GetPrevValue<double>()) * Period;
				const double MeanJitterBufferDelayMs = TotalJitterBufferDelayPerSecond / FramesReceivedPerSecond->GetValue<double>() * 1000.0;
				return FStat({ .Name = PixelStreaming2StatNames::JitterBufferDelay }, MeanJitterBufferDelayMs, 2);
			}
			return {};
		});
	}

	void FRTCStatsCollector::FRTPRemoteTrackStatsSink::Process(const EpicRtcRemoteTrackRtpStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::FirCount)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._firCount);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::PliCount)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._pliCount);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::NackCount)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._nackCount);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::RetransmittedBytesReceived)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._retransmittedBytesReceived);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::RetransmittedPacketsReceived)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._retransmittedPacketsReceived);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::KeyFramesDecoded)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._keyFramesDecoded);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FrameWidth)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._frameWidth);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FrameHeight)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._frameHeight);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FramesReceived)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._framesReceived);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::BytesReceived)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._bytesReceived);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::QPSum)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._qpSum);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::FramesDecoded)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._framesDecoded);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::PacketsLost)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._packetsLost);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::Jitter)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._local._jitter);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::DecoderImplementation)
			{
				NewValue = FStatVariant(TInPlaceType<FString>(), ToString(InStats._local._decoderImplementation));
			}
			else if (Tuple.Key == PixelStreaming2StatNames::RoundTripTime)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._remote._roundTripTime);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPVideoSourceSink ----------
	 */
	FRTCStatsCollector::FVideoSourceStatsSink::FVideoSourceStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// Track video source fps
		Stats.Add(PixelStreaming2StatNames::SourceFps, FStat({ .Name = PixelStreaming2StatNames::SourceFps }, 0.f));
	}

	void FRTCStatsCollector::FVideoSourceStatsSink::Process(const EpicRtcVideoSourceStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::SourceFps)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._framesPerSecond);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPVideoCodecSink ----------
	 */
	FRTCStatsCollector::FVideoCodecStatsSink::FVideoCodecStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// Track video source fps
		Stats.Add(PixelStreaming2StatNames::MimeType, FStat({ .Name = PixelStreaming2StatNames::MimeType }, FString(TEXT(""))));
	}

	void FRTCStatsCollector::FVideoCodecStatsSink::Process(const EpicRtcCodecStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::MimeType)
			{
				NewValue = FStatVariant(TInPlaceType<FString>(), ToString(InStats._mimeType));
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPAudioSourceSink ----------
	 */
	FRTCStatsCollector::FAudioSourceStatsSink::FAudioSourceStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		Stats.Add(PixelStreaming2StatNames::AudioLevel, FStat({ .Name = PixelStreaming2StatNames::AudioLevel }, 0.f, 2));
		Stats.Add(PixelStreaming2StatNames::TotalSamplesDuration, FStat({ .Name = PixelStreaming2StatNames::TotalSamplesDuration }, 0.f));
	}

	void FRTCStatsCollector::FAudioSourceStatsSink::Process(const EpicRtcAudioSourceStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::AudioLevel)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._audioLevel);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::TotalSamplesDuration)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._totalSamplesDuration);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPAudioCodecSink ----------
	 */
	FRTCStatsCollector::FAudioCodecStatsSink::FAudioCodecStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// Track video source fps
		Stats.Add(PixelStreaming2StatNames::MimeType, FStat({ .Name = PixelStreaming2StatNames::MimeType }, FString(TEXT(""))));
		Stats.Add(PixelStreaming2StatNames::Channels, FStat({ .Name = PixelStreaming2StatNames::Channels }, 0.f));
		Stats.Add(PixelStreaming2StatNames::ClockRate, FStat({ .Name = PixelStreaming2StatNames::ClockRate }, 0.f));
	}

	void FRTCStatsCollector::FAudioCodecStatsSink::Process(const EpicRtcCodecStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::MimeType)
			{
				NewValue = FStatVariant(TInPlaceType<FString>(), ToString(InStats._mimeType));
			}
			else if (Tuple.Key == PixelStreaming2StatNames::Channels)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._channels);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::ClockRate)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._clockRate);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ----------- FDataChannelSink -----------
	 */
	FRTCStatsCollector::FDataTrackStatsSink::FDataTrackStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		// These names are added as aliased names because `bytesSent` is ambiguous stat that is used across inbound-rtp, outbound-rtp, and data-channel
		// so to disambiguate which state we are referring to we record the `bytesSent` stat for the data-channel but store and report it as `data-channel-bytesSent`
		Stats.Add(PixelStreaming2StatNames::MessagesSent, FStat({ .Name = PixelStreaming2StatNames::MessagesSent, .Alias = PixelStreaming2StatNames::DataChannelMessagesSent, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, 0.f));
		Stats.Add(PixelStreaming2StatNames::MessagesReceived, FStat({ .Name = PixelStreaming2StatNames::MessagesReceived, .Alias = PixelStreaming2StatNames::DataChannelBytesReceived, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesSent, FStat({ .Name = PixelStreaming2StatNames::BytesSent, .Alias = PixelStreaming2StatNames::DataChannelBytesSent, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, 0.f));
		Stats.Add(PixelStreaming2StatNames::BytesReceived, FStat({ .Name = PixelStreaming2StatNames::BytesReceived, .Alias = PixelStreaming2StatNames::DataChannelMessagesReceived, .DisplayFlags = static_cast<EDisplayFlags>(EDisplayFlags::TEXT | EDisplayFlags::GRAPH) }, 0.f));
	}

	void FRTCStatsCollector::FDataTrackStatsSink::Process(const EpicRtcDataTrackStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::MessagesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._messagesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::MessagesReceived)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._messagesReceived);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::BytesSent)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._bytesSent);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::BytesReceived)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._bytesReceived);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * ---------- FRTPAudioSourceSink ----------
	 */
	FRTCStatsCollector::FCandidatePairStatsSink::FCandidatePairStatsSink(FName InCategory)
		: FStatsSink(InCategory)
	{
		Stats.Add(PixelStreaming2StatNames::AvailableOutgoingBitrate, FStat({ .Name = PixelStreaming2StatNames::AvailableOutgoingBitrate }, 0.f));
		Stats.Add(PixelStreaming2StatNames::AvailableIncomingBitrate, FStat({ .Name = PixelStreaming2StatNames::AvailableIncomingBitrate }, 0.f));
	}

	void FRTCStatsCollector::FCandidatePairStatsSink::Process(const EpicRtcIceCandidatePairStats& InStats, const FString& PeerId, double SecondsDelta)
	{
		FStats* PSStats = FStats::Get();
		if (!PSStats)
		{
			return;
		}

		for (TPair<FName, FStat>& Tuple : Stats)
		{
			FStatVariant NewValue;
			if (Tuple.Key == PixelStreaming2StatNames::AvailableOutgoingBitrate)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._availableOutgoingBitrate);
			}
			else if (Tuple.Key == PixelStreaming2StatNames::AvailableIncomingBitrate)
			{
				NewValue = FStatVariant(TInPlaceType<double>(), InStats._availableIncomingBitrate);
			}

			if (NewValue.GetIndex() == FStatVariant::IndexOfType<TYPE_OF_NULLPTR>())
			{
				continue;
			}

			if (Tuple.Value.SetValue(NewValue))
			{
				PSStats->StorePeerStat(PeerId, Category, Tuple.Value);
			}
		}

		PostProcess(PSStats, PeerId, SecondsDelta);
	}

	/**
	 * --------- FStatsSink ------------------------
	 */
	FRTCStatsCollector::FStatsSink::FStatsSink(FName InCategory)
		: Category(MoveTemp(InCategory))
	{
	}

	void FRTCStatsCollector::FStatsSink::PostProcess(FStats* PSStats, const FString& PeerId, double SecondsDelta)
	{
		for (auto& Calculator : Calculators)
		{
			TOptional<FStat> OptStatData = Calculator(*this, SecondsDelta);
			if (OptStatData.IsSet())
			{
				FStat& StatData = *OptStatData;
				Stats.Add(StatData.GetName(), StatData);
				PSStats->StorePeerStat(PeerId, Category, StatData);
			}
		}
	}
} // namespace UE::PixelStreaming2