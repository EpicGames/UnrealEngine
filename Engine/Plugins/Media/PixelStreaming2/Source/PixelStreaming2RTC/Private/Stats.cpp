// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stats.h"
#include "EpicRtcStreamer.h"
#include "Async/Async.h"
#include "CanvasTypes.h"
#include "Engine/GameViewportClient.h"
#include "RHIGlobals.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "Engine/Console.h"
#include "ConsoleSettings.h"
#include "UnrealClient.h"

// Complete the defintion for IPixelStreaming2Stats.h
IPixelStreaming2Stats& IPixelStreaming2Stats::Get()
{
	IPixelStreaming2Stats* Stats = UE::PixelStreaming2::FStats::Get();
	return *Stats;
}

// Create Canvas text with same font/size/appearance
static FCanvasTextItem CreateText(const FString& String, double X, double Y)
{
	FText			TextToDisplay = FText::FromString(String);
	FCanvasTextItem Text(FVector2D(X, Y), TextToDisplay, FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 10)), FLinearColor(0, 1, 0));
	Text.EnableShadow(FLinearColor::Black);
	return Text;
}

namespace UE::PixelStreaming2
{
	FStats* FStats::Instance = nullptr;

	FStats* FStats::Get()
	{
		if (Instance == nullptr)
		{
			Instance = new FStats();
		}
		return Instance;
	}

	FStats::FStats()
	{
		checkf(Instance == nullptr, TEXT("There should only ever been one PixelStreaming2 stats object."));

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStats::RegisterEngineHooks);
	}

	void FStats::StorePeerStat(const FString& PlayerId, FName StatCategory, FStat Stat)
	{
		FName StatName = Stat.GetDisplayName();

		bool Updated = false;
		{
			FScopeLock Lock(&PeerStatsCS);

			if (!PeerStats.Contains(PlayerId))
			{
				PeerStats.Add(PlayerId, FPeerStats(PlayerId));
				Updated = true;
			}
			else
			{
				Updated = PeerStats[PlayerId].StoreStat(StatCategory, Stat);
			}
		}

		if (Updated)
		{
			if (Stat.ShouldGraph())
			{
				GraphValue(StatName, Stat.GetValue<double>(), 60, 0, Stat.GetValue<double>() * 10.0f, 0);
			}

			if (Stat.IsNumeric())
			{
				// If a stat has an alias, use that as the storage key, otherwise use its underlying name
				FireStatChanged(PlayerId, StatName, Stat.GetValue<double>());
			}
		}
	}

	bool FStats::QueryPeerStat(const FString& PlayerId, FName InStatCategory, FName StatToQuery, double& OutValue)
	{
		FScopeLock Lock(&PeerStatsCS);

		if (FPeerStats* SinglePeerStats = PeerStats.Find(PlayerId))
		{
			TMap<FName, FStatGroup>& StatGroups = SinglePeerStats->GetStatGroups();
			TArray<FName>			 StatCategories;
			StatGroups.GetKeys(StatCategories);

			// Stat groups contain a name as well as additional info like track index and ssrc
			// When querying a stat we need to find all matching stats based on name
			TArray<FName> MatchedStatCategories;
			for (FName& StatCategory : StatCategories)
			{
				if (StatCategory.ToString().Contains(InStatCategory.ToString()))
				{
					MatchedStatCategories.Add(StatCategory);
				}
			}

			if (MatchedStatCategories.Num() == 0)
			{
				return false;
			}

			// TODO (william.belcher): This is lazy and only queries the first matched category but since
			// this code is only used on the p2p use case where there only is one matching category it's fine
			return SinglePeerStats->GetStat(MatchedStatCategories[0], StatToQuery, OutValue);
		}

		return false;
	}

	void FStats::RemovePeerStats(const FString& PlayerId)
	{
		FScopeLock Lock(&PeerStatsCS);

		PeerStats.Remove(PlayerId);

		if (IsSFU(PlayerId))
		{
			TArray<FString> ToRemove;

			for (auto& Entry : PeerStats)
			{
				FString PeerId = Entry.Key;
				if (PeerId.Contains(TEXT("Simulcast"), ESearchCase::IgnoreCase, ESearchDir::FromStart))
				{
					ToRemove.Add(PeerId);
				}
			}

			for (FString SimulcastLayerId : ToRemove)
			{
				PeerStats.Remove(SimulcastLayerId);
			}
		}
	}

	void FStats::StoreApplicationStat(FStat Stat)
	{
		// If a stat has an alias, use that as the storage key, otherwise use its underlying name
		FName StatName = Stat.GetDisplayName();

		if (Stat.ShouldGraph())
		{
			GraphValue(StatName, Stat.GetValue<double>(), 60, 0, Stat.GetValue<double>(), 0);
		}

		{
			FScopeLock Lock(&ApplicationStatsCS);

			if (ApplicationStats.Contains(StatName))
			{
				FRenderableStat* StoredStat = ApplicationStats.Find(StatName);

				if (StoredStat->Renderable.IsSet())
				{
					FText TextToDisplay = FText::FromString(FString::Printf(TEXT("%s: %s"), *StatName.ToString(), *Stat.ToString()));
					StoredStat->Renderable.GetValue().Text = TextToDisplay;
				}
			}
			else
			{
				FRenderableStat StoredStat(Stat);

				if (Stat.ShouldDisplayText())
				{
					FString			StringToDisplay = FString::Printf(TEXT("%s: %s"), *StatName.ToString(), *Stat.ToString());
					FCanvasTextItem Text = CreateText(StringToDisplay, 0, 0);
					StoredStat.Renderable = Text;
				}

				ApplicationStats.Add(StatName, StoredStat);
			}
		}

		if (Stat.IsNumeric())
		{
			FireStatChanged(FString(TEXT("Application")), StatName, Stat.GetValue<double>());
		}
	}

	void FStats::RemoveAllApplicationStats()
	{
		FScopeLock Lock(&ApplicationStatsCS);
		ApplicationStats.Empty();
	}

	void FStats::FireStatChanged(const FString& PlayerId, FName StatName, float StatValue)
	{
		// Broadcast must be done on the GameThread because the GameThread can remove the delegates.
		// If removing and broadcast happens simultaneously it causes a datarace failure.
		AsyncTask(ENamedThreads::GameThread, [PlayerId, StatName, StatValue]() {
			if (UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get())
			{
				Delegates->OnStatChangedNative.Broadcast(PlayerId, StatName, StatValue);
				Delegates->OnStatChanged.Broadcast(PlayerId, StatName, StatValue);
			}
		});
	}

	void FStats::UpdateConsoleAutoComplete(TArray<FAutoCompleteCommand>& AutoCompleteList)
	{
		// This *might* need to be on the game thread? I haven't seen issues not explicitly putting it on the game thread though.

		const UConsoleSettings* ConsoleSettings = GetDefault<UConsoleSettings>();

		AutoCompleteList.AddDefaulted();
		FAutoCompleteCommand& AutoCompleteCommand = AutoCompleteList.Last();
		AutoCompleteCommand.Command = TEXT("Stat PixelStreaming2");
		AutoCompleteCommand.Desc = TEXT("Displays stats about Pixel Streaming on screen.");
		AutoCompleteCommand.Color = ConsoleSettings->AutoCompleteCommandColor;

		AutoCompleteList.AddDefaulted();
		FAutoCompleteCommand& AutoCompleteGraphCommand = AutoCompleteList.Last();
		AutoCompleteGraphCommand.Command = TEXT("Stat PixelStreaming2Graphs");
		AutoCompleteGraphCommand.Desc = TEXT("Displays graphs about Pixel Streaming on screen.");
		AutoCompleteGraphCommand.Color = ConsoleSettings->AutoCompleteCommandColor;
	}

	int32 FStats::OnRenderStats(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		if (GAreScreenMessagesEnabled)
		{
			Y += 50;

			{
				FString			StringToDisplay = FString::Printf(TEXT("GPU: %s"), *GRHIAdapterName);
				FCanvasTextItem Text = CreateText(StringToDisplay, X, Y);
				Canvas->DrawItem(Text);
				Y += Text.DrawnSize.Y;
			}

			// Draw each peer's stats in a column, so we must recall where Y starts for each column
			int32 YStart = Y;

			// --------- Draw stats for this Pixel Streaming instance ----------

			{
				FScopeLock Lock(&ApplicationStatsCS);

				for (auto& ApplicationStatEntry : ApplicationStats)
				{
					FRenderableStat& StatToDraw = ApplicationStatEntry.Value;
					if (!StatToDraw.Renderable.IsSet())
					{
						continue;
					}
					FCanvasTextItem& Text = StatToDraw.Renderable.GetValue();
					Text.Position.X = X;
					Text.Position.Y = Y;
					Canvas->DrawItem(Text);
					Y += Text.DrawnSize.Y;
				}
			}

			// --------- Draw stats for each peer ----------

			// increment X now we are done drawing application stats
			X += 435;

			{
				FScopeLock Lock(&PeerStatsCS);

				// <FPixelStreaming2PlayerId, FPeerStats>
				for (auto& EachPeerEntry : PeerStats)
				{
					FPeerStats& SinglePeerStats = EachPeerEntry.Value;
					if (SinglePeerStats.GetStatGroups().Num() == 0)
					{
						continue;
					}

					// Reset Y for each peer as each peer gets it own column
					Y = YStart;

					SinglePeerStats.PlayerIdCanvasItem.Position.X = X;
					SinglePeerStats.PlayerIdCanvasItem.Position.Y = Y;
					Canvas->DrawItem(SinglePeerStats.PlayerIdCanvasItem);
					Y += SinglePeerStats.PlayerIdCanvasItem.DrawnSize.Y;

					// <FName, FStatGroup>
					for (auto& StatGroupEntry : SinglePeerStats.GetStatGroups())
					{
						FStatGroup& StatGroup = StatGroupEntry.Value;

						// Draw StatGroup category name
						{
							FCanvasTextItem& Text = StatGroup.CategoryCanvasItem;
							Text.Position.X = X;
							Text.Position.Y = Y;
							Canvas->DrawItem(Text);
							Y += Text.DrawnSize.Y;
						}

						// Draw the stat value
						for (auto& StatEntry : StatGroup.GetStoredStats())
						{
							FRenderableStat& Stat = StatEntry.Value;
							if (!Stat.Renderable.IsSet())
							{
								continue;
							}
							FCanvasTextItem& Text = Stat.Renderable.GetValue();
							Text.Position.X = X;
							Text.Position.Y = Y;
							Canvas->DrawItem(Text);
							Y += Text.DrawnSize.Y;
						}
					}

					// Each peer's stats gets its own column
					X += 250;
				}
			}
		}
		return Y;
	}

	bool FStats::OnToggleStats(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		return true;
	}

	bool FStats::OnToggleGraphs(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
	{
		return true;
	}

	int32 FStats::OnRenderGraphs(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		checkf(IsInGameThread(), TEXT("FStats::OnRenderGraphs must be called from the gamethread."));

		static const int XOffset = 50;
		static const int YOffset = 50;
		FVector2D		 GraphPos{ XOffset, YOffset };
		FVector2D		 GraphSize{ 200, 200 };
		float			 GraphSpacing = 5;

		for (auto& [GraphName, Graph] : Graphs)
		{
			Graph.Draw(Canvas, GraphPos, GraphSize);
			GraphPos.X += GraphSize.X + GraphSpacing;
			if ((GraphPos.X + GraphSize.X) > Canvas->GetRenderTarget()->GetSizeXY().X)
			{
				GraphPos.Y += GraphSize.Y + GraphSpacing;
				GraphPos.X = XOffset;
			}
		}

		for (auto& [TileName, Tile] : Tiles)
		{
			Tile.Position.X = GraphPos.X;
			Tile.Position.Y = GraphPos.Y;
			Tile.Size = GraphSize;
			Tile.Draw(Canvas);
			GraphPos.X += GraphSize.X + GraphSpacing;
			if ((GraphPos.X + GraphSize.X) > Canvas->GetRenderTarget()->GetSizeXY().X)
			{
				GraphPos.Y += GraphSize.Y + GraphSpacing;
				GraphPos.X = XOffset;
			}
		}

		return Y;
	}

	void FStats::PollPixelStreaming2Settings()
	{
		double DeltaSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTimeSettingsPolledCycles);
		if (DeltaSeconds > 1)
		{
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.Encoder.MinQuality")) }, UPixelStreaming2PluginSettings::CVarEncoderMinQuality.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.Encoder.MaxQuality")) }, UPixelStreaming2PluginSettings::CVarEncoderMaxQuality.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.Encoder.KeyframeInterval (frames)")) }, UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.Fps")) }, UPixelStreaming2PluginSettings::CVarWebRTCFps.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.StartBitrate")) }, UPixelStreaming2PluginSettings::CVarWebRTCStartBitrate.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.MinBitrate")) }, UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate.GetValueOnAnyThread(), 0));
			StoreApplicationStat(FStat({ .Name = FName(TEXT("PixelStreaming2.WebRTC.MaxBitrate")) }, UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate.GetValueOnAnyThread(), 0));

			LastTimeSettingsPolledCycles = FPlatformTime::Cycles64();
		}
	}

	void FStats::Tick(float DeltaTime)
	{
		PollPixelStreaming2Settings();
	}

	void FStats::RemoveAllPeerStats()
	{
		FScopeLock LockPeers(&PeerStatsCS);
		PeerStats.Empty();
	}

	void FStats::RegisterEngineHooks()
	{
		GAreScreenMessagesEnabled = true;

		const FName				   StatName("STAT_PixelStreaming2");
		const FName				   StatCategory("STATCAT_PixelStreaming2");
		const FText				   StatDescription(FText::FromString("Stats for the Pixel Streaming plugin and its peers."));
		UEngine::FEngineStatRender RenderStatFunc = UEngine::FEngineStatRender::CreateRaw(this, &FStats::OnRenderStats);
		UEngine::FEngineStatToggle ToggleStatFunc = UEngine::FEngineStatToggle::CreateRaw(this, &FStats::OnToggleStats);
		GEngine->AddEngineStat(StatName, StatCategory, StatDescription, RenderStatFunc, ToggleStatFunc, false);

		const FName				   GraphName("STAT_PixelStreaming2Graphs");
		const FText				   GraphDescription(FText::FromString("Draws stats graphs for the Pixel Streaming plugin."));
		UEngine::FEngineStatRender RenderGraphFunc = UEngine::FEngineStatRender::CreateRaw(this, &FStats::OnRenderGraphs);
		UEngine::FEngineStatToggle ToggleGraphFunc = UEngine::FEngineStatToggle::CreateRaw(this, &FStats::OnToggleGraphs);
		GEngine->AddEngineStat(GraphName, StatCategory, GraphDescription, RenderGraphFunc, ToggleGraphFunc, false);

		UConsole::RegisterConsoleAutoCompleteEntries.AddRaw(this, &FStats::UpdateConsoleAutoComplete);

		// Check the command line for launch args to automatically enable stats for users
		TFunction<bool(const TCHAR*)> CheckLaunchArgFunc = [](const TCHAR* Match) -> bool {
			FString ValueMatch(Match);
			ValueMatch.Append(TEXT("="));
			FString Value;
			if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
			{
				if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
				{
					return true;
				}
				else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
				{
					return false;
				}
			}
			else if (FParse::Param(FCommandLine::Get(), Match))
			{
				return true;
			}

			return false;
		};

		bool bHudStats = CheckLaunchArgFunc(TEXT("PixelStreamingHudStats"));
		bool bOnScreenStats = CheckLaunchArgFunc(TEXT("PixelStreamingOnScreenStats"));

		if (bHudStats || bOnScreenStats)
		{
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{
				UWorld*				 World = WorldContext.World();
				UGameViewportClient* ViewportClient = World->GetGameViewport();
				GEngine->SetEngineStat(World, ViewportClient, TEXT("PixelStreaming2"), true);
			}
		}
	}

	//
	// ---------------- FStatGroup ---------------------------
	// A collection of stats grouped together by a category name
	//
	bool FStatGroup::StoreStat(FStat& StatToStore)
	{
		// If a stat has an alias, use that as the storage key, otherwise use its underlying name
		FName StatName = StatToStore.GetDisplayName();

		if (!StoredStats.Contains(StatName))
		{
			FRenderableStat NewStat(StatToStore);

			// If we are displaying the stat, add a renderable for it
			if (StatToStore.ShouldDisplayText())
			{
				FString			StringToDisplay = FString::Printf(TEXT("%s: %s"), *StatName.ToString(), *StatToStore.ToString());
				FCanvasTextItem Text = CreateText(StringToDisplay, 0, 0);
				NewStat.Renderable = Text;
			}

			// Actually store the stat
			StoredStats.Add(StatName, NewStat);

			// first time this stat has been stored, so we also need to sort our stats so they render in consistent order
			StoredStats.KeySort([](const FName& A, const FName& B) {
				return A.FastLess(B);
			});

			return true;
		}
		else
		{
			// We already have this stat, so just update it
			FRenderableStat* StoredStat = StoredStats.Find(StatName);
			if (!StoredStat)
			{
				return false;
			}

			if (StoredStat->ShouldDisplayText() && StoredStat->Renderable.IsSet())
			{
				FText TextToDisplay = FText::FromString(FString::Printf(TEXT("%s: %s"), *StatName.ToString(), *StatToStore.ToString()));
				StoredStat->Renderable.GetValue().Text = TextToDisplay;
			}
			return true;
		}
	}

	//
	// ---------------- FPeerStats ---------------------------
	// Stats specific to a particular peer, as opposed to the entire app.
	//

	bool FPeerStats::StoreStat(FName StatCategory, FStat& StatToStore)
	{
		if (!StatGroups.Contains(StatCategory))
		{
			StatGroups.Add(StatCategory, FStatGroup(StatCategory));
			StatGroups.KeySort([](FName A, FName B) {
				return A.ToString().Compare(B.ToString(), ESearchCase::IgnoreCase) < 0;
			});
		}
		return StatGroups[StatCategory].StoreStat(StatToStore);
	}

	bool UE::PixelStreaming2::FPeerStats::GetStat(FName StatCategory, FName StatToQuery, double& OutValue)
	{
		FStatGroup* Group = StatGroups.Find(StatCategory);
		if (!Group)
		{
			return false;
		}

		FRenderableStat* StoredStat = Group->GetStoredStats().Find(StatToQuery);
		if (!StoredStat)
		{
			return false;
		}

		OutValue = StoredStat->GetValue<double>();
		return true;
	}

	void FStats::GraphValue(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue)
	{
		if (IsInGameThread())
		{
			GraphValue_GameThread(InName, Value, InSamples, InMinRange, InMaxRange, InRefValue);
		}
		else
		{
			AsyncTask(ENamedThreads::Type::GameThread, [this, InName, Value, InSamples, InMinRange, InMaxRange, InRefValue]() {
				GraphValue_GameThread(InName, Value, InSamples, InMinRange, InMaxRange, InRefValue);
			});
		}
	}

	void FStats::GraphValue_GameThread(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue)
	{
		checkf(IsInGameThread(), TEXT("FStats::GraphValue_GameThread must be called from the gamethread."));

		if (!Graphs.Contains(InName))
		{
			auto& Graph = Graphs.Add(InName, FDebugGraph(InName, InSamples, InMinRange, InMaxRange, InRefValue));
			Graph.AddValue(Value);
		}
		else
		{
			Graphs[InName].AddValue(Value);
		}
	}

	double FStats::AddTimeStat(double Millis, const FString& Label)
	{
		const double DeltaMs = Millis;
		const FStat	 TimeData({ .Name = FName(*Label) }, DeltaMs, 2, true);
		StoreApplicationStat(TimeData);
		return DeltaMs;
	}

	double FStats::AddTimeDeltaStat(uint64 Millis1, uint64 Millis2, const FString& Label)
	{
		const uint64 MaxMillis = FGenericPlatformMath::Max(Millis1, Millis2);
		const uint64 MinMillis = FGenericPlatformMath::Min(Millis1, Millis2);
		const double DeltaMs = (MaxMillis - MinMillis) * ((Millis1 > Millis2) ? 1.0 : -1.0);
		const FStat	 TimeData({ .Name = FName(*Label) }, DeltaMs, 2, true);
		StoreApplicationStat(TimeData);
		return DeltaMs;
	}

	void FStats::AddFrameTimingStats(const FPixelCaptureFrameMetadata& FrameMetadata, FIntPoint Resolution)
	{
		const int Samples = 100;

		TSharedPtr<FVideoProducerUserData> UserData = StaticCastSharedPtr<FVideoProducerUserData>(FrameMetadata.UserData);
		if (UserData)
		{
			const double TimeProduce = AddTimeStat(FPlatformTime::ToMilliseconds64(UserData->ProductionEndCycles - UserData->ProductionBeginCycles), FString::Printf(TEXT("%s Frame Production Time"), *(UserData->ProducerName)));
			GraphValue(*FString::Printf(TEXT("%s Produce Time"), *(UserData->ProducerName)), StaticCast<float>(TimeProduce), Samples, 0.0f, 30.0f);
		}

		const double TimeCapture = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureEndCyles - FrameMetadata.CaptureStartCyles), FString::Printf(TEXT("%s (%dx%d) Frame Capture Time"), *FrameMetadata.ProcessName, Resolution.X, Resolution.Y));
		const double TimeCPU = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessCPUEndCycles - FrameMetadata.CaptureProcessCPUStartCycles), FString::Printf(TEXT("%s (%dx%d) Frame Capture CPU Time"), *FrameMetadata.ProcessName, Resolution.X, Resolution.Y));
		const double TimeGPUDelay = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessGPUEnqueueEndCycles - FrameMetadata.CaptureProcessGPUEnqueueStartCycles), FString::Printf(TEXT("%s (%dx%d) Frame Capture GPU Delay Time"), *FrameMetadata.ProcessName, Resolution.X, Resolution.Y));
		const double TimeGPU = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessGPUEndCycles - FrameMetadata.CaptureProcessGPUStartCycles), FString::Printf(TEXT("%s (%dx%d) Frame Capture GPU Time"), *FrameMetadata.ProcessName, Resolution.X, Resolution.Y));
		const double TimePostGPU = AddTimeStat(FPlatformTime::ToMilliseconds64(FrameMetadata.CaptureProcessPostGPUEndCycles - FrameMetadata.CaptureProcessPostGPUStartCycles), FString::Printf(TEXT("%s (%dx%d) Frame Capture Post GPU Time"), *FrameMetadata.ProcessName, Resolution.X, Resolution.Y));

		const FStat UseData({ .Name = FName(*FString::Printf(TEXT("%s (%dx%d) Frame Uses"), *FrameMetadata.ProcessName, Resolution.X, Resolution.Y)) }, static_cast<double>(FrameMetadata.UseCount));
		StoreApplicationStat(UseData);

		GraphValue(*FString::Printf(TEXT("(%dx%d) Capture Time"), Resolution.X, Resolution.Y), StaticCast<float>(TimeCapture), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("(%dx%d) CPU Time"), Resolution.X, Resolution.Y), StaticCast<float>(TimeCPU), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("(%dx%d) GPU Delay Time"), Resolution.X, Resolution.Y), StaticCast<float>(TimeGPUDelay), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("(%dx%d) GPU Time"), Resolution.X, Resolution.Y), StaticCast<float>(TimeGPU), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("(%dx%d) Post GPU Time"), Resolution.X, Resolution.Y), StaticCast<float>(TimePostGPU), Samples, 0.0f, 30.0f);
		GraphValue(*FString::Printf(TEXT("(%dx%d) Frame Uses"), Resolution.X, Resolution.Y), StaticCast<float>(FrameMetadata.UseCount), Samples, 0.0f, 10.0f);
	}

	void FStats::AddCanvasTile(FName Name, const FCanvasTileItem& Tile)
	{
		if (IsInGameThread())
		{
			AddCanvasTile_GameThread(Name, Tile);
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, [this, Name, Tile]() {
				AddCanvasTile_GameThread(Name, Tile);
			});
		}
	}

	void FStats::AddCanvasTile_GameThread(FName Name, const FCanvasTileItem& Tile)
	{
		checkf(IsInGameThread(), TEXT("FStats::AddCanvasTile_GameThread must be called from the gamethread."));
		Tiles.FindOrAdd(Name, Tile);
	}
} // namespace UE::PixelStreaming2
