// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"
#include "CanvasItem.h"
#include "UnrealEngine.h"
#include "ConsoleSettings.h"
#include "DebugGraph.h"
#include "PixelCaptureFrameMetadata.h"
#include "IPixelStreaming2Stats.h"
#include "RTCStatsCollector.h"

namespace UE::PixelStreaming2
{
	// Stat that can be optionally rendered
	struct FRenderableStat : public FStat
	{
		FRenderableStat(FStat& Stat)
			: FStat(Stat)
		{
		}

		TOptional<FCanvasTextItem> Renderable;
	};

	// A grouping of stats by some category name
	class FStatGroup
	{
	public:
		FStatGroup(FName CategoryName)
			: GroupName(CategoryName)
			, CategoryCanvasItem(FVector2D(0, 0), FText::FromString(FString::Printf(TEXT("---%s---"), *CategoryName.ToString())), FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 12)), FLinearColor(0, 0.9f, 0.1f))
		{
			CategoryCanvasItem.EnableShadow(FLinearColor::Black);
		}

		virtual ~FStatGroup() = default;
		bool						  StoreStat(FStat& StatToStore);
		TMap<FName, FRenderableStat>& GetStoredStats() { return StoredStats; }

	private:
		FName						 GroupName;
		TMap<FName, FRenderableStat> StoredStats;

	public:
		FCanvasTextItem CategoryCanvasItem;
	};

	// Pixel Streaming stats that are associated with a specific peer.
	class FPeerStats
	{

	public:
		FPeerStats(const FString& InAssociatedPlayer)
			: AssociatedPlayer(InAssociatedPlayer)
			, PlayerIdCanvasItem(FVector2D(0, 0), FText::FromString(FString::Printf(TEXT("[Peer Stats(%s)]"), *AssociatedPlayer)), FSlateFontInfo(FSlateFontInfo(UEngine::GetSmallFont(), 15)), FLinearColor(0, 1, 0))
		{
			PlayerIdCanvasItem.EnableShadow(FLinearColor::Black);
		};

		bool					 StoreStat(FName StatCategory, FStat& StatToStore);
		bool					 GetStat(FName StatCategory, FName StatToQuery, double& OutValue);
		TMap<FName, FStatGroup>& GetStatGroups() { return StatGroups; }

	private:
		int						DisplayId = 0;
		FString					AssociatedPlayer;
		TMap<FName, FStatGroup> StatGroups;

	public:
		FCanvasTextItem PlayerIdCanvasItem;
	};

	// Stats about Pixel Streaming that can displayed either in the in-application HUD, in the log, or simply reported to some subscriber.
	// Stats can be enabled to draw on screen with:
	// `stat pixelstreaming2`
	// `stat pixelstreaming2graphs`
	class FStats : public FTickableGameObject, public IPixelStreaming2Stats
	{
	public:
		virtual bool IsTickableInEditor() const override { return true; }

		static constexpr double SmoothingPeriod = 3.0 * 60.0;
		static constexpr double SmoothingFactor = 10.0 / 100.0;
		static FStats*			Get();

		FStats(const FStats&) = delete;
		bool QueryPeerStat(const FString& PlayerId, FName StatCategory, FName StatToQuery, double& OutValue);
		void RemovePeerStats(const FString& PlayerId);
		void RemoveAllPeerStats();
		void StorePeerStat(const FString& PlayerId, FName StatCategory, FStat Stat);
		void StoreApplicationStat(FStat PeerStat);
		void RemoveAllApplicationStats();
		void Tick(float DeltaTime);

		bool  OnToggleStats(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream);
		int32 OnRenderStats(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

		bool  OnToggleGraphs(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream);
		int32 OnRenderGraphs(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);

		FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(PixelStreaming2Stats, STATGROUP_Tickables); }

		double AddTimeStat(double Millis, const FString& Label);
		double AddTimeDeltaStat(uint64 Millis1, uint64 Millis2, const FString& Label);
		void   AddFrameTimingStats(const FPixelCaptureFrameMetadata& FrameMetadata, FIntPoint Resolution);
		void   AddCanvasTile(FName Name, const FCanvasTileItem& Tile);

		// Begin IPixelStreaming2Stats interface
		friend class IPixelStreaming2Stats;
		void GraphValue(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue = 0.0f) override;
		// End IPixelStreaming2Stats interface

	private:
		FStats();

		void RegisterEngineHooks();
		void UpdateConsoleAutoComplete(TArray<FAutoCompleteCommand>& AutoCompleteList);

		void PollPixelStreaming2Settings();
		void FireStatChanged(const FString& PlayerId, FName StatName, float StatValue);
		void GraphValue_GameThread(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue);
		void AddCanvasTile_GameThread(FName Name, const FCanvasTileItem& Tile);

	private:
		static FStats* Instance;

		FCriticalSection		  PeerStatsCS;
		TMap<FString, FPeerStats> PeerStats;

		FCriticalSection			 ApplicationStatsCS;
		TMap<FName, FRenderableStat> ApplicationStats;

		int64 LastTimeSettingsPolledCycles = 0;

		TMap<FName, FDebugGraph>	 Graphs;
		TMap<FName, FCanvasTileItem> Tiles;

		FCriticalSection StatNotificationCS;
	};
} // namespace UE::PixelStreaming2
