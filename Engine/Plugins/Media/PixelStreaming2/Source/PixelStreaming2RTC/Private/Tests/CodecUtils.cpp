// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodecUtils.h"

#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	void DoFrameReceiveTest()
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		FMockVideoFrameConfig FrameConfig = { 128 /*Width*/, 128 /*Height*/, 255 /*Y*/, 137 /*U*/, 216 /*V*/ };

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString								 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer> Streamer = CreateStreamer(StreamerName, StreamerPort);
		TSharedPtr<FVideoProducerBase>		 VideoProducer = FVideoProducerBase::Create();
		Streamer->SetVideoProducer(VideoProducer);

		TSharedPtr<FMockPlayer>	   Player = CreatePlayer();
		TSharedPtr<FMockVideoSink> VideoSink = Player->GetVideoSink();

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check streaming started"), [bStreamingStarted]() { return *bStreamingStarted.Get(); }, 5.0))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, PlayerPort, StreamerName]() { Player->Connect(PlayerPort, StreamerName); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check player connected"), [Player]() { return Player->IsConnected(); }, 5.0))

		// Send 30 frames
		for (int i = 0; i < 30; i++)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FSendSolidColorFrame(VideoProducer, FrameConfig))
			ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(0.033)) // send at 30fps interval
		}

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForFrameReceived(5.0, VideoSink, FrameConfig))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { Player }))
	}

	void DoFrameResizeMultipleTimesTest()
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString								 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer> Streamer = CreateStreamer(StreamerName, StreamerPort);
		TSharedPtr<FVideoProducerBase>		 VideoProducer = FVideoProducerBase::Create();
		Streamer->SetVideoProducer(VideoProducer);

		TSharedPtr<FMockPlayer>	   Player = CreatePlayer();
		TSharedPtr<FMockVideoSink> VideoSink = Player->GetVideoSink();

		// Note: Important to couple framerate as we are manually passing frames and don't want any cached frames
		Streamer->SetCoupleFramerate(true);

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check streaming started"), [bStreamingStarted]() { return *bStreamingStarted.Get(); }, 5.0))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, PlayerPort, StreamerName]() { Player->Connect(PlayerPort, StreamerName); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check player connected"), [Player]() { return Player->IsConnected(); }, 5.0))

		for (int Res = 2; Res < 512; Res *= 2)
		{
			FMockVideoFrameConfig FrameConfig = { Res /*Width*/, Res /*Height*/, 255 /*Y*/, 0 /*U*/, 255 /*V*/ };

			// Send 30 frames
			for (int i = 0; i < 30; i++)
			{
				ADD_LATENT_AUTOMATION_COMMAND(FSendSolidColorFrame(VideoProducer, FrameConfig))
				ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(0.033)) // send at 30fps interval
			}

			ADD_LATENT_AUTOMATION_COMMAND(FWaitForFrameReceived(5.0, VideoSink, FrameConfig))
		}

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { Player }));
	}
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS
