// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace  UE::PixelStreaming2
{
	void DoFrameReceiveTest();
	void DoFrameResizeMultipleTimesTest();
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS
