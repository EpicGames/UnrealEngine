// Copyright Epic Games, Inc. All Rights Reserved.

#include "DelegateTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DelegateTest)

#if WITH_TESTS

#include "CodecUtils.h"
#include "HAL/PlatformProcess.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2PluginSettings.h"
#include "TestUtils.h"

void UPixelStreaming2DynamicDelegateTest::OnConnectedToSignallingServer(FString StreamerId)
{
	DynamicDelegateCalled(TEXT("OnConnectedToSignallingServer"), StreamerId);
}

void UPixelStreaming2DynamicDelegateTest::OnDisconnectedFromSignallingServer(FString StreamerId)
{
	DynamicDelegateCalled(TEXT("OnDisconnectedFromSignallingServer"), StreamerId);
}

void UPixelStreaming2DynamicDelegateTest::OnNewConnection(FString StreamerId, FString PlayerId)
{
	DynamicDelegateCalled(TEXT("OnNewConnection"), StreamerId, PlayerId);
}

void UPixelStreaming2DynamicDelegateTest::OnClosedConnection(FString StreamerId, FString PlayerId)
{
	DynamicDelegateCalled(TEXT("OnClosedConnection"), StreamerId, PlayerId);
}

void UPixelStreaming2DynamicDelegateTest::OnAllConnectionsClosed(FString StreamerId)
{
	DynamicDelegateCalled(TEXT("OnAllConnectionsClosed"), StreamerId);
}

void UPixelStreaming2DynamicDelegateTest::OnDataTrackOpen(FString StreamerId, FString PlayerId)
{
	DynamicDelegateCalled(TEXT("OnDataTrackOpen"), StreamerId, PlayerId);
}

void UPixelStreaming2DynamicDelegateTest::OnDataTrackClosed(FString StreamerId, FString PlayerId)
{
	DynamicDelegateCalled(TEXT("OnDataTrackClosed"), StreamerId, PlayerId);
}

void UPixelStreaming2DynamicDelegateTest::OnStatChanged(FString PlayerId, FName StatName, float StatValue)
{
	DynamicDelegateCalled(TEXT("OnStatChanged"), PlayerId, StatName, StatValue);
}

void UPixelStreaming2DynamicDelegateTest::OnFallbackToSoftwareEncoding()
{
	DynamicDelegateCalled(TEXT("OnFallbackToSoftwareEncoding"));
}

// Macros used because passing the callback into a function results in a runtime check hit 
// because UE checks the variable name of UFUNCTIONs
#define BIND_DELEGATE(Delegate, Callback, Name, ...) 	\
 	Delegate.AddDynamic(this, Callback); 				\
 	BindDelegate<__VA_ARGS__>(Name)

bool UPixelStreaming2DynamicDelegateTest::Init(UE::PixelStreaming2::DelegateTestConfig Config, FString StreamerName)
{
	UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
	if (!Delegates)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Delegates are null.");
		return false;
	}
	else
	{
		const bool bIsRemote = false;
		
		BIND_DELEGATE(Delegates->OnConnectedToSignallingServer, &UPixelStreaming2DynamicDelegateTest::OnConnectedToSignallingServer, TEXT("OnConnectedToSignallingServer"), FString)->Times(UE::PixelStreaming2::Exactly(1)).With(StreamerName);
		BIND_DELEGATE(Delegates->OnDisconnectedFromSignallingServer, &UPixelStreaming2DynamicDelegateTest::OnDisconnectedFromSignallingServer, TEXT("OnDisconnectedFromSignallingServer"), FString)->Times(UE::PixelStreaming2::Exactly(1)).With(StreamerName);
		BIND_DELEGATE(Delegates->OnNewConnection, &UPixelStreaming2DynamicDelegateTest::OnNewConnection, TEXT("OnNewConnection"), FString, TOptional<FString>)->Times(UE::PixelStreaming2::Exactly(Config.NumPlayers)).With(StreamerName, UE::PixelStreaming2::Any<FString>());
		BIND_DELEGATE(Delegates->OnClosedConnection, &UPixelStreaming2DynamicDelegateTest::OnClosedConnection, TEXT("OnClosedConnection"), FString, TOptional<FString>)->Times(UE::PixelStreaming2::Exactly(Config.NumPlayers)).With(StreamerName, UE::PixelStreaming2::Any<FString>());
		BIND_DELEGATE(Delegates->OnAllConnectionsClosed, &UPixelStreaming2DynamicDelegateTest::OnAllConnectionsClosed, TEXT("OnAllConnectionsClosed"), FString)->Times(UE::PixelStreaming2::Exactly(1)).With(StreamerName);
		BIND_DELEGATE(Delegates->OnDataTrackOpen, &UPixelStreaming2DynamicDelegateTest::OnDataTrackOpen, TEXT("OnDataTrackOpen"), FString, TOptional<FString>)->Times(UE::PixelStreaming2::Exactly(Config.NumPlayers)).With(StreamerName, UE::PixelStreaming2::Any<FString>());
		BIND_DELEGATE(Delegates->OnDataTrackClosed, &UPixelStreaming2DynamicDelegateTest::OnDataTrackClosed, TEXT("OnDataTrackClosed"), FString, TOptional<FString>)->Times(UE::PixelStreaming2::Exactly(Config.NumPlayers)).With(StreamerName, UE::PixelStreaming2::Any<FString>());
		BIND_DELEGATE(Delegates->OnStatChanged, &UPixelStreaming2DynamicDelegateTest::OnStatChanged, TEXT("OnStatChanged"), TOptional<FString>, TOptional<FName>, TOptional<float>)->Times(UE::PixelStreaming2::AtLeast(1)).With(UE::PixelStreaming2::Any<FString>(), UE::PixelStreaming2::Any<FName>(), UE::PixelStreaming2::Any<float>());
		BIND_DELEGATE(Delegates->OnFallbackToSoftwareEncoding, &UPixelStreaming2DynamicDelegateTest::OnFallbackToSoftwareEncoding, TEXT("OnFallbackToSoftwareEncoding"))->Times(UE::PixelStreaming2::Exactly(Config.SoftwareEncodingCount));
	}
	return true;
}

void UPixelStreaming2DynamicDelegateTest::Destroy()
{
	UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
	if (!Delegates)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Error, "Delegates are null.");
	}
	else
	{
		for (const auto& DelegateTest : DelegatesMap)
		{
			if (const auto& Value = DelegateTest.Value; !Value->bWasCalledExpectedTimes(true))
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} was called {1} times.", Value->Name, Value->CallCount);
			}
		}
		
		Delegates->OnConnectedToSignallingServer.RemoveAll(this);
		Delegates->OnDisconnectedFromSignallingServer.RemoveAll(this);
		Delegates->OnNewConnection.RemoveAll(this);
		Delegates->OnClosedConnection.RemoveAll(this);
		Delegates->OnAllConnectionsClosed.RemoveAll(this);
		Delegates->OnDataTrackOpen.RemoveAll(this);
		Delegates->OnDataTrackClosed.RemoveAll(this);
		Delegates->OnStatChanged.RemoveAll(this);
		Delegates->OnFallbackToSoftwareEncoding.RemoveAll(this);
	}
	DelegatesMap.Empty();
}

namespace UE::PixelStreaming2
{
	FCardinality AnyNumber()
	{
		return FCardinality();
	}

	FCardinality AtLeast(int Min)
	{
		return FCardinality(Min, std::numeric_limits<int>::max());
	}

	FCardinality AtMost(int Max)
	{
		return FCardinality(std::numeric_limits<int>::min(), Max);
	}

	FCardinality Between(int Min, int Max)
	{
		return FCardinality(Min, Max);
	}

	FCardinality Exactly(int ExactValue)
	{
		return FCardinality(ExactValue, ExactValue);
	}
	
	bool FDelegateTestBase::CheckCalled(bool bPrintErrors) const
	{
		for (const auto& DelegateTest : DelegatesMap)
		{
			if (!DelegateTest.Value->bWasCalledExpectedTimes(bPrintErrors))
			{
				return false;
			}
		}

		return true;
	}
	// Used to hold onto the lifetime of the Dynamic delegate
	class FDynamicDelegateLifetime
	{
	public:
		FDynamicDelegateLifetime() = default;
		~FDynamicDelegateLifetime() 
		{
			DelegateTest->Destroy();
		}

		bool Init(DelegateTestConfig Config, FString StreamerName)
		{
			DelegateTest = TStrongObjectPtr<UPixelStreaming2DynamicDelegateTest>(NewObject<UPixelStreaming2DynamicDelegateTest>());
			if (!DelegateTest->Init(Config, StreamerName))
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "Unable to create FDelegatesTest");
				return false;
			}
			return true;
		}

		TStrongObjectPtr<UPixelStreaming2DynamicDelegateTest> DelegateTest;
	};

	template <typename DelegateType, typename... Args>
	TSharedPtr<FSingleDelegateTest<StripOptionalType<Args>...>> CreateSingleDelegateTest(TMap<FString, TSharedPtr<UE::PixelStreaming2::FSingleDelegateTestBase>>& Map, DelegateType& InDelegate, FString InName)
	{
		TSharedPtr<FSingleDelegateTest<StripOptionalType<Args>...>> DelegateTest = MakeShared<FSingleDelegateTest<StripOptionalType<Args>...>>(InName);
		DelegateTest->BindDelegate(InDelegate);
		Map.Add(InName, DelegateTest);
		return DelegateTest;
	}

	class FDelegateNativeTest: public FDelegateTestBase
	{
	public:
		~FDelegateNativeTest()
		{
			for (auto& DelegateTest : DelegatesMap)
			{
				if (const auto& Value = DelegateTest.Value; !Value->bWasCalledExpectedTimes(true))
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} was called {1} times.", Value->Name, Value->CallCount);
				}
			}
		}

		bool Init(DelegateTestConfig Config, FString StreamerName)
		{
			UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
			if (!Delegates)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "Delegates are null.");
				return false;
			}

			const bool bIsRemote = false;

			CreateSingleDelegateTest<decltype(Delegates->OnConnectedToSignallingServerNative), FString>(DelegatesMap, Delegates->OnConnectedToSignallingServerNative, TEXT("OnConnectedToSignallingServerNative"))->Times(Exactly(1)).With(StreamerName);
			CreateSingleDelegateTest<decltype(Delegates->OnDisconnectedFromSignallingServerNative), FString>(DelegatesMap, Delegates->OnDisconnectedFromSignallingServerNative, TEXT("OnDisconnectedFromSignallingServerNative"))->Times(Exactly(1)).With(StreamerName);
			CreateSingleDelegateTest<decltype(Delegates->OnNewConnectionNative), FString, TOptional<FString>>(DelegatesMap, Delegates->OnNewConnectionNative, TEXT("OnNewConnectionNative"))->Times(Exactly(Config.NumPlayers)).With(StreamerName, Any<FString>());
			CreateSingleDelegateTest<decltype(Delegates->OnClosedConnectionNative), FString, TOptional<FString>>(DelegatesMap, Delegates->OnClosedConnectionNative, TEXT("OnClosedConnectionNative"))->Times(Exactly(Config.NumPlayers)).With(StreamerName, Any<FString>());
			CreateSingleDelegateTest<decltype(Delegates->OnAllConnectionsClosedNative), FString>(DelegatesMap, Delegates->OnAllConnectionsClosedNative, TEXT("OnAllConnectionsClosedNative"))->Times(Exactly(1)).With(StreamerName);
			CreateSingleDelegateTest<decltype(Delegates->OnDataTrackOpenNative), FString, TOptional<FString>>(DelegatesMap, Delegates->OnDataTrackOpenNative, TEXT("OnDataTrackOpenNative"))->Times(Exactly(Config.NumPlayers)).With(StreamerName, Any<FString>());
			CreateSingleDelegateTest<decltype(Delegates->OnDataTrackClosedNative), FString, TOptional<FString>>(DelegatesMap, Delegates->OnDataTrackClosedNative, TEXT("OnDataTrackClosedNative"))->Times(Exactly(Config.NumPlayers)).With(StreamerName, Any<FString>());

			const int NumCalls = Config.bIsBidirectional ? Config.NumPlayers * 2: Config.NumPlayers;

			auto& OnVideoTrackOpenNativeDelegate = CreateSingleDelegateTest<decltype(Delegates->OnVideoTrackOpenNative), FString, TOptional<FString>, bool>(DelegatesMap, Delegates->OnVideoTrackOpenNative, TEXT("OnVideoTrackOpenNative"))->Times(Exactly(NumCalls)).With(StreamerName, Any<FString>(), bIsRemote);
			auto& OnVideoTrackClosedNativeDelegate = CreateSingleDelegateTest<decltype(Delegates->OnVideoTrackClosedNative), FString, TOptional<FString>, bool>(DelegatesMap, Delegates->OnVideoTrackClosedNative, TEXT("OnVideoTrackClosedNative"))->Times(Exactly(NumCalls)).With(StreamerName, Any<FString>(), bIsRemote);
			auto& OnAudioTrackOpenNativeDelegate = CreateSingleDelegateTest<decltype(Delegates->OnAudioTrackOpenNative), FString, TOptional<FString>, bool>(DelegatesMap, Delegates->OnAudioTrackOpenNative, TEXT("OnAudioTrackOpenNative"))->Times(Exactly(NumCalls)).With( StreamerName, Any<FString>(), bIsRemote);
			auto& OnAudioTrackClosedNativeDelegate = CreateSingleDelegateTest<decltype(Delegates->OnAudioTrackClosedNative), FString, TOptional<FString>, bool>(DelegatesMap, Delegates->OnAudioTrackClosedNative, TEXT("OnAudioTrackClosedNative"))->Times(Exactly(NumCalls)).With(StreamerName, Any<FString>(), bIsRemote);
			
			if(Config.bIsBidirectional)
			{
				OnVideoTrackOpenNativeDelegate.With(StreamerName, Any<FString>(), !bIsRemote);
				OnVideoTrackClosedNativeDelegate.With(StreamerName, Any<FString>(), !bIsRemote);
				OnAudioTrackOpenNativeDelegate.With(StreamerName, Any<FString>(), !bIsRemote);
				OnAudioTrackClosedNativeDelegate.With(StreamerName, Any<FString>(), !bIsRemote);
			}

			CreateSingleDelegateTest<decltype(Delegates->OnStatChangedNative), TOptional<FString>, TOptional<FName>, TOptional<float>>(DelegatesMap, Delegates->OnStatChangedNative, TEXT("OnStatChangedNative"))->Times(AtLeast(1)).With(Any<FString>(), Any<FName>(), Any<float>());
			CreateSingleDelegateTest<decltype(Delegates->OnFallbackToSoftwareEncodingNative)>(DelegatesMap, Delegates->OnFallbackToSoftwareEncodingNative, TEXT("OnFallbackToSoftwareEncodingNative"))->Times(Exactly(Config.SoftwareEncodingCount));

			return true;
		}
	};

	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCleanupDelegatesNative, TSharedPtr<FDelegateNativeTest>, DelegatesTest, float, TimeoutSeconds);

	bool FCleanupDelegatesNative::Update()
	{
		const double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting for delegates.");
			return true;
		}
		
		if (DelegatesTest && DelegatesTest->CheckCalled(false))
		{
			DelegatesTest.Reset();
			UE_LOGFMT(LogPixelStreaming2RTC, Log, "Cleaning up DelegatesTest.");
			return true;
		}
		else if (!DelegatesTest)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "DelegatesTest is null.");
			return true;
		}
		return false;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCleanupDelegates, TSharedPtr<FDynamicDelegateLifetime>, DelegateTestScope, float, TimeoutSeconds);

	bool FCleanupDelegates::Update()
	{
		const double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting for delegates.");
			return true;
		}

		if (DelegateTestScope->DelegateTest && DelegateTestScope->DelegateTest->CheckCalled(false))
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Log, "Cleaning up DelegatesTest.");
			return true;
		}

		return false;
	}

	template<typename T>
	void RunDelegateTest(DelegateTestConfig Config)
	{
		const int32 StreamerPort = TestUtils::NextStreamerPort();
		const int32 PlayerPort = TestUtils::NextPlayerPort();
		const FString StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		
		UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
		if (!Delegates)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Delegates are null.");
			return;
		}
		
		const TSharedPtr<T> DelegatesTest = MakeShared<T>();
		if (!DelegatesTest->Init(Config, StreamerName))
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Unable to create FDelegatesTest");
			return;
		}

		const TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		const TSharedPtr<IPixelStreaming2Streamer> Streamer = CreateStreamer(StreamerName, StreamerPort);
		const TSharedPtr<FVideoProducerBase>	   VideoProducer = FVideoProducerBase::Create();
		Streamer->SetVideoProducer(VideoProducer);

		TArray<TSharedPtr<FMockPlayer>> Players;
		Players.SetNum(Config.NumPlayers);
		for (TSharedPtr<FMockPlayer>& Player : Players)
		{
			FMockPlayerConfig PlayerConfig = {};
			if(Config.bIsBidirectional)
			{
				PlayerConfig.AudioDirection = EMediaDirection::Bidirectional;
				PlayerConfig.VideoDirection = EMediaDirection::Bidirectional;
			}
			Player = CreatePlayer(PlayerConfig);
		}
		TArray<TSharedPtr<FMockVideoSink>> VideoSinks;
		VideoSinks.SetNum(Config.NumPlayers);
		for (int i = 0; i < VideoSinks.Num(); ++i)
		{
			VideoSinks[i] = Players[i]->GetVideoSink();
		}

		const TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		const TSharedPtr<bool> bStreamingDisconnected = MakeShared<bool>(false);
		Delegates->OnDisconnectedFromSignallingServerNative.AddLambda([bStreamingDisconnected](FString) {
			*(bStreamingDisconnected.Get()) = true;
		});

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming started"), 5.0, Streamer, bStreamingStarted, true))

		for (TSharedPtr<FMockPlayer>& Player : Players)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, PlayerPort, StreamerName]() { Player->Connect(PlayerPort, StreamerName); }))
			ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check player connected"), [Player]() { return Player->IsConnected(); }, 5.0))
		}

		for (TSharedPtr<FMockPlayer>& Player : Players)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelOrTimeout(5.0, Player));
		}

		for (TSharedPtr<FMockPlayer>& Player : Players)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player]() { Player->Disconnect(TEXT("Disconnecting after tests")); }))
		}

		// Wait 1 second to ensure any websocket message have correctly flowed
		ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(1.0))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StopStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check disconnected"), 5.0, Streamer, bStreamingDisconnected, true))

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, Players))
		
		if constexpr (std::is_same_v<T, FDelegateNativeTest>)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FCleanupDelegatesNative(DelegatesTest, 5.0))
		}
		else if constexpr (std::is_same_v<T, FDynamicDelegateLifetime>)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FCleanupDelegates(DelegatesTest, 5.0))
		}
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateSoftwareFallbackTest, "System.Plugins.PixelStreaming2.FPS2DelegateSoftwareFallbackTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateSoftwareFallbackTest::RunTest(const FString&)
	{
		bool Result = true;
		if(IsRHIDeviceNVIDIA())
		{
			AddExpectedError(TEXT("No more HW encoders available. Falling back to software encoding"), EAutomationExpectedMessageFlags::MatchType::Exact, 1, false);
			const int PrevPixelStreamingEncoderMaxSessions = UPixelStreaming2PluginSettings::CVarEncoderMaxSessions.GetValueOnAnyThread();
			const EVideoCodec PrevCodec = UE::PixelStreaming2::GetEnumFromCVar<EVideoCodec>(UPixelStreaming2PluginSettings::CVarEncoderCodec);
			UPixelStreaming2PluginSettings::CVarEncoderMaxSessions.AsVariable()->Set(0);
			UPixelStreaming2PluginSettings::CVarEncoderCodec.AsVariable()->Set(*UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::H264));
			RunDelegateTest<FDelegateNativeTest>({ 1, 1, false });

			// Reset to previous
			ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([PrevPixelStreamingEncoderMaxSessions, PrevCodec]() { 
				UPixelStreaming2PluginSettings::CVarEncoderMaxSessions.AsVariable()->Set(PrevPixelStreamingEncoderMaxSessions);
				UPixelStreaming2PluginSettings::CVarEncoderCodec.AsVariable()->Set(*UE::PixelStreaming2::GetCVarStringFromEnum(PrevCodec));
			}))
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Log, "FPS2DelegateSoftwareFallbackTest requires Nvidia GPU to test Software fallback");
		}
		
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateNativeSingleTest, "System.Plugins.PixelStreaming2.FPS2DelegateNativeSingleTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateNativeSingleTest::RunTest(const FString&)
	{
		RunDelegateTest<FDelegateNativeTest>({0, 1, false});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateDynamicSingleTest, "System.Plugins.PixelStreaming2.FPS2DelegateDynamicSingleTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateDynamicSingleTest::RunTest(const FString&)
	{
		RunDelegateTest<FDynamicDelegateLifetime>({0, 1, false});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateNativeMultipleTest, "System.Plugins.PixelStreaming2.FPS2DelegateNativeMultipleTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateNativeMultipleTest::RunTest(const FString&)
	{
		RunDelegateTest<FDelegateNativeTest>({0, 3, false});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateDynamicMultipleTest, "System.Plugins.PixelStreaming2.FPS2DelegateDynamicMultipleTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateDynamicMultipleTest::RunTest(const FString&)
	{
		RunDelegateTest<FDynamicDelegateLifetime>({0, 3, false});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateNativeSingleBidirectionalTest, "System.Plugins.PixelStreaming2.FPS2DelegateNativeSingleBidirectionalTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateNativeSingleBidirectionalTest::RunTest(const FString&)
	{
		RunDelegateTest<FDelegateNativeTest>({0, 1, true});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateDynamicSingleBidirectionalTest, "System.Plugins.PixelStreaming2.FPS2DelegateDynamicSingleBidirectionalTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateDynamicSingleBidirectionalTest::RunTest(const FString&)
	{
		RunDelegateTest<FDynamicDelegateLifetime>({0, 1, true});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateNativeMultipleBidirectionalTest, "System.Plugins.PixelStreaming2.FPS2DelegateNativeMultipleBidirectionalTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateNativeMultipleBidirectionalTest::RunTest(const FString&)
	{
		RunDelegateTest<FDelegateNativeTest>({0, 3, true});
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2DelegateDynamicMultipleBidirectionalTest, "System.Plugins.PixelStreaming2.FPS2DelegateDynamicMultipleBidirectionalTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2DelegateDynamicMultipleBidirectionalTest::RunTest(const FString&)
	{
		RunDelegateTest<FDynamicDelegateLifetime>({0, 3, true});
		return true;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_TESTS
