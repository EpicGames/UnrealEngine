// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging.h"
#include "PixelStreaming2Delegates.h"
#include "UObject/Object.h"

#include "DelegateTest.generated.h"
#if WITH_TESTS

namespace UE::PixelStreaming2
{
	template <typename T>
	TOptional<T> Any()
	{
		TOptional<T> Temp;
		return Temp;
	}

	template<typename T>
	struct IsOptional : std::false_type {};

	template<typename T>
	struct IsOptional<TOptional<T>> : std::true_type {};

	template<typename T>
	struct StripOptional{using type = T;};

	template<typename T>
	struct StripOptional<TOptional<T>>{using type = T;};

	template<typename T>
	using StripOptionalType = typename StripOptional<T>::type;

	template<typename T>
	auto ToOptional(T&& a)
	{
		if constexpr(IsOptional<std::decay_t<T>>::value)
		{
			return Forward<T>(a);
		}
		else
		{
			return TOptional<std::decay_t<T>>(Forward<T>(a));
		}
	}
	
	class FCardinality
	{
	public:
		FCardinality()
			: Min(std::numeric_limits<int>::min())
			, Max(std::numeric_limits<int>::max())
		{
		}

		FCardinality(int Min, int Max)
			: Min(Min)
			, Max(Max)
		{
		}

		int Min;
		int Max;
	};

	FCardinality AnyNumber();
	FCardinality AtLeast(int Min);
	FCardinality AtMost(int Max);
	FCardinality Between(int Min, int Max);
	FCardinality Exactly(int ExactValue);

	struct DelegateTestConfig
	{
		int SoftwareEncodingCount = 0;
		int NumPlayers = 0;
		bool bIsBidirectional = false;
	};

	// Base class to allow for Delegate Tests to be stored in an array with different parameters
	class FSingleDelegateTestBase
	{
	public:
		FSingleDelegateTestBase(FString InName): 
			Name(MoveTemp(InName))
			{}

		virtual ~FSingleDelegateTestBase() = default;

		bool bWasCalledExpectedTimes(bool bPrintErrors) const
		{
			const bool bGreaterThanMin = CallCount >= ExpectedCallCount.Min;
			const bool bLessThanMax = CallCount <= ExpectedCallCount.Max;

			if (bPrintErrors && (!bGreaterThanMin || !bLessThanMax))
			{
				if (!bGreaterThanMin)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} Expected Value {1} not greater than {2}", Name, ExpectedCallCount.Min, CallCount);
				}

				if (!bLessThanMax)
				{
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} Expected Value {1} not less than {2}", Name, ExpectedCallCount.Max, CallCount);
				}
			}

			return CallCount >= ExpectedCallCount.Min
				&& CallCount <= ExpectedCallCount.Max
				&& bCallbackMatchesExpectedValues;
		}

		FString Name;
		int			 CallCount = 0;
		FCardinality ExpectedCallCount;
		bool bCallbackMatchesExpectedValues = true;
	};

	template <typename... Args>
	class FSingleDelegateArgsTest: public FSingleDelegateTestBase
	{
	public:
		FSingleDelegateArgsTest(FString InName)
			: FSingleDelegateTestBase(MoveTemp(InName)) 
			{}

		void OnCalled(Args... InActualValues)
		{
			++CallCount;

			auto ActualValues = MakeTuple(InActualValues...);

			const int NumExpectedValues = ExpectedValuesArray.Num();

			// If there's no expected values, the check is an automatic success
			bool bCheckSuccess = NumExpectedValues == 0; 
			for (int i = (NumExpectedValues - 1); i >= 0; i--)
			{
				TArray<FString> CheckStrings;

				auto& ExpectedValueSet = ExpectedValuesArray[i];
				bool  bThisCheckSuccess = true;

				VisitTupleElements([&bThisCheckSuccess, &CheckStrings, Name = Name](auto& ExpectedValue, auto& ActualValue) {

					if (!ExpectedValue.IsSet())
					{
						bThisCheckSuccess &= true;
					}
					else
					{
						bThisCheckSuccess &= ExpectedValue == ActualValue;
					}
				}, ExpectedValueSet, ActualValues);

				// This check has succeeded so no need to check earlier registered expected values
				if (bThisCheckSuccess)
				{
					bCheckSuccess = true;
					break;
				}
			}

			if (!bCheckSuccess)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} expected Value do not match actual values", Name);
			}

			bCallbackMatchesExpectedValues &= bCheckSuccess;
		}

		FSingleDelegateArgsTest<Args...>& Times(const FCardinality& InExpectedCallCount)
		{
			ExpectedCallCount = InExpectedCallCount;
			return *this;
		}

		// Use Separate template args because the types may include TOptional while args does not.
		template<typename... Params>
		FSingleDelegateArgsTest<Args...>& With(Params... Values)
		{
			static_assert(sizeof...(Params) == sizeof...(Args));
			
			// Make sure all parameters get wrapped in a TOptional if they are not already. 
			// This is to get around issues with embedding an existing TOptional inside another TOptional.
			// When double embedded, the tests will fail because the outer TOptional thinks it is set when inner is empty.
			ExpectedValuesArray.Add(MakeTuple(ToOptional(Forward<Params>(Values))...));
			return *this;
		}

		TArray<TTuple<TOptional<Args>...>> ExpectedValuesArray;
	};

	template <typename... Args>
	class FSingleDynamicDelegateTest: public FSingleDelegateArgsTest<Args...>
	{
	public:
		FSingleDynamicDelegateTest(FString InName)
			: FSingleDelegateArgsTest<Args...>(InName) 
			{}
	};

	template <typename... Args>
	class FSingleDelegateTest : public FSingleDelegateArgsTest<Args...>, public TSharedFromThis<FSingleDelegateTest<Args...>>
	{
	public:
		FSingleDelegateTest(FString InName)
			: FSingleDelegateArgsTest<Args...>(MoveTemp(InName)) 
			{}
	
		virtual ~FSingleDelegateTest()
		{
			if (UnbindDelegateFunc)
			{
				UnbindDelegateFunc();
			}
		}

		template<typename DelegateType>
		void BindDelegate(DelegateType& InDelegate)
		{
			DelegateHandle = InDelegate.AddLambda([WeakThis = this->AsWeak()](auto... InActualValues)
			{
				if (const TSharedPtr<FSingleDelegateTest> SharedThis = WeakThis.Pin())
				{
					auto ActualValues = MakeTuple(InActualValues...);
					SharedThis->OnCalled(InActualValues...);
				}
			});

			UnbindDelegateFunc = [&InDelegate, Handle = DelegateHandle]()
			{
				InDelegate.Remove(Handle);
			};
		}

		TFunction<void()> UnbindDelegateFunc;
		FDelegateHandle DelegateHandle;
	};

	class FDelegateTestBase
	{
	public:
		bool CheckCalled(bool bPrintErrors) const;

		TMap<FString, TSharedPtr<UE::PixelStreaming2::FSingleDelegateTestBase>> DelegatesMap;
	};
}

// Blueprint dynamic delegates require UE's reflection system.
UCLASS()
class UPixelStreaming2DynamicDelegateTest : public UObject, public UE::PixelStreaming2::FDelegateTestBase
{
	GENERATED_BODY()
public:
	UFUNCTION()
	void OnConnectedToSignallingServer(FString StreamerId);

	UFUNCTION()
	void OnDisconnectedFromSignallingServer(FString StreamerId);

	UFUNCTION()
	void OnNewConnection(FString StreamerId, FString PlayerId);

	UFUNCTION()
	void OnClosedConnection(FString StreamerId, FString PlayerId);

	UFUNCTION()
	void OnAllConnectionsClosed(FString StreamerId);

	UFUNCTION()
	void OnDataTrackOpen(FString StreamerId, FString PlayerId);

	UFUNCTION()
	void OnDataTrackClosed(FString StreamerId, FString PlayerId);

	UFUNCTION()
	void OnStatChanged(FString PlayerId, FName StatName, float StatValue);

	UFUNCTION()
	void OnFallbackToSoftwareEncoding();

	bool Init(UE::PixelStreaming2::DelegateTestConfig Config, FString StreamerName);
	void Destroy();

	template <typename... Args>
	TSharedPtr<UE::PixelStreaming2::FSingleDelegateArgsTest<UE::PixelStreaming2::StripOptionalType<Args>...>> BindDelegate(FString InName)
	{
		UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
		if (!Delegates)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Delegates are null.");
			return nullptr;
		}
		else
		{
			TSharedPtr<UE::PixelStreaming2::FSingleDelegateArgsTest<UE::PixelStreaming2::StripOptionalType<Args>...>> Ptr = MakeShared<UE::PixelStreaming2::FSingleDynamicDelegateTest<UE::PixelStreaming2::StripOptionalType<Args>...>>(InName);
			DelegatesMap.Add(InName, Ptr);
			return Ptr;
		}
	}

private:
	
	template <typename... Args>
	void DynamicDelegateCalled(FString InName, Args... InActualValues)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "{0} was called", InName);
		if (!IsInGameThread())
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} was not called on the game thread", InName);
		}
		
		TSharedPtr<UE::PixelStreaming2::FSingleDelegateTestBase>* DelegateTest = DelegatesMap.Find(InName);
		if (DelegateTest)
		{
			TSharedPtr<UE::PixelStreaming2::FSingleDynamicDelegateTest<Args...>> DynamicTest = StaticCastSharedPtr<UE::PixelStreaming2::FSingleDynamicDelegateTest<Args...>>(*DelegateTest);
			DynamicTest->OnCalled(InActualValues...);
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "unknown Delegate Test {0}", InName);
		}
	}
};

#endif // WITH_TESTS
