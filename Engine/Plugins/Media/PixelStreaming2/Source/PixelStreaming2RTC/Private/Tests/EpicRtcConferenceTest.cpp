// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcUtils.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "UtilsString.h"

#include "epic_rtc/core/platform.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	// Tests the creation and removal of a conference from the platform
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcConferenceLifetimeTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcConferenceLifetimeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcConferenceLifetimeTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		TRefCountPtr<FMockWebSocketFactory> WebsocketFactory = MakeRefCount<FMockWebSocketFactory>();

		{
			TRefCountPtr<EpicRtcPlatformInterface> Platform;
			Result = GetOrCreatePlatform({}, Platform.GetInitReference());
			if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
			{
				return false;
			}

			FUtf8String ConferenceId = "test_conference";

			TRefCountPtr<EpicRtcConferenceInterface> Conference;
			Result = Platform->CreateConference(ToEpicRtcStringView(ConferenceId),
				{
					._websocketFactory = WebsocketFactory.GetReference(),
					._signallingType = EpicRtcSignallingType::PixelStreaming,
				},
				Conference.GetInitReference());
			// Count should be two. One for `Conference` and another from EpicRtc storing internally
			if (!ValidateResultRefCount(Conference, "Conference", Result, { EpicRtcErrorCode::Ok }, 2))
			{
				return false;
			}

			if (!ValidateRefCount(WebsocketFactory, "WebsocketFactory", 2))
			{
				return false;
			}

			// Release the platform's handle to the conference
			Platform->ReleaseConference(ToEpicRtcStringView(ConferenceId));

			// Conference should still be valid and with a ref count of 1 as EpicRtc has released
			if (!ValidateRefCount(Conference, "Conference", 1))
			{
				return false;
			}

			// Check epicrtc is no longer storing conference
			TRefCountPtr<EpicRtcConferenceInterface> NullConference;
			Result = Platform->GetConference(ToEpicRtcStringView(ConferenceId), NullConference.GetInitReference());
			if (Result != EpicRtcErrorCode::ConferenceDoesNotExists)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate conference. Unexpected result. Expected (%ls), Actual (%ls)", *ToString(EpicRtcErrorCode::ConferenceDoesNotExists), *ToString(Result));
				return false;
			}

			if (NullConference.GetReference() != nullptr)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate conference. Expected NullConference to reference a nullptr");
				return false;
			}
		}

		// Conference has been destroyed so WebsocketFactory count will have decreased
		if (!ValidateRefCount(WebsocketFactory, "WebsocketFactory", 1))
		{
			return false;
		}

		return true;
	}

	// Tests the conference creation logic to ensure that two conferences with the same name can't be created
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcConferenceCreateTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcConferenceCreateTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcConferenceCreateTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		TRefCountPtr<EpicRtcPlatformInterface> Platform;
		Result = GetOrCreatePlatform({}, Platform.GetInitReference());
		if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
		{
			return false;
		}

		FUtf8String ConferenceId = "test_conference";

		TRefCountPtr<EpicRtcConferenceInterface> Conference;
		Result = Platform->CreateConference(ToEpicRtcStringView(ConferenceId),
			{
				._signallingType = EpicRtcSignallingType::PixelStreaming,
			},
			Conference.GetInitReference());
		// Count should be two. One for `Conference` and another from EpicRtc storing internally
		if (!ValidateResultRefCount(Conference, "Conference", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		// Try to create conference with the same name
		TRefCountPtr<EpicRtcConferenceInterface> BadConference;
		Result = Platform->CreateConference(ToEpicRtcStringView(ConferenceId),
			{
				._signallingType = EpicRtcSignallingType::PixelStreaming,
			},
			BadConference.GetInitReference());
		if (Result != EpicRtcErrorCode::ConferenceAlreadyExists)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate conference. Unexpected result. Expected (%ls), Actual (%ls)", *ToString(EpicRtcErrorCode::ConferenceAlreadyExists), *ToString(Result));
			return false;
		}

		// Release the platform's handle to the conference
		Platform->ReleaseConference(ToEpicRtcStringView(ConferenceId));

		// Conference should still be valid and with a ref count of 1 as SameConference and EpicRtc have released
		if (!ValidateRefCount(Conference, "Conference", 1))
		{
			return false;
		}

		return true;
	}

	// Tests the conference retrieval logic to ensure that a conference can be retrieved after is has been created
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcConferenceGetTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcConferenceGetTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcConferenceGetTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		TRefCountPtr<EpicRtcPlatformInterface> Platform;
		Result = GetOrCreatePlatform({}, Platform.GetInitReference());
		if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
		{
			return false;
		}

		FUtf8String ConferenceId = "test_conference";

		TRefCountPtr<EpicRtcConferenceInterface> Conference;
		Result = Platform->CreateConference(ToEpicRtcStringView(ConferenceId),
			{
				._signallingType = EpicRtcSignallingType::PixelStreaming,
			},
			Conference.GetInitReference());
		// Count should be two. One for `Conference` and another from EpicRtc storing internally
		if (!ValidateResultRefCount(Conference, "Conference", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		{
			// Get another handle to the initial conference
			TRefCountPtr<EpicRtcConferenceInterface> SameConference;
			Result = Platform->GetConference(ToEpicRtcStringView(ConferenceId), SameConference.GetInitReference());
			// Count should be three. One for `Conference`, one for `SameConference` and another from EpicRtc storing internally
			if (!ValidateResultRefCount(SameConference, "Conference", Result, { EpicRtcErrorCode::Ok }, 3))
			{
				return false;
			}

			if (Conference.GetReference() != SameConference.GetReference())
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Expected Conference and SameConference to reference the same pointer");
				return false;
			}

			// Get handle to conference again
			Result = Platform->GetConference(ToEpicRtcStringView(ConferenceId), SameConference.GetInitReference());
			// Count should still be three as we used GetInitReference on SameConference
			if (!ValidateResultRefCount(SameConference, "Conference", Result, { EpicRtcErrorCode::Ok }, 3))
			{
				return false;
			}
		}

		// Release the platform's handle to the conference
		Platform->ReleaseConference(ToEpicRtcStringView(ConferenceId));

		// Conference should still be valid and with a ref count of 1 as SameConference and EpicRtc have released
		if (!ValidateRefCount(Conference, "Conference", 1))
		{
			return false;
		}

		return true;
	}
} // namespace UE::PixelStreaming2

#endif
