// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcUtils.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"

#include "epic_rtc/core/platform.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcPlatformCreateTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcPlatformCreateTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcPlatformCreateTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		TRefCountPtr<EpicRtcPlatformInterface> Platform;
		Result = GetOrCreatePlatform({}, Platform.GetInitReference());
		if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
		{
			return false;
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcPlatformLifetimeTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcPlatformLifetimeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcPlatformLifetimeTest::RunTest(const FString& Parameters)
	{
		EpicRtcPlatformInterface* PlatformPtr;
		uint32 CreationCount;
		{
			TRefCountPtr<EpicRtcPlatformInterface> Platform;

			EpicRtcErrorCode Result = GetOrCreatePlatform({}, Platform.GetInitReference());
			if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
			{
				return false;
			}
			CreationCount = Platform->Count();

			PlatformPtr = Platform.GetReference();
			PlatformPtr->AddRef();	// Platform will go out of scope so will have sane count
		}

		// Because the platform is is stored internal to EpicRtc and is shared between PS, EOSSDK, Platform will still exists so will be at least 1
		// Count should be lower than CreationCount
		if (uint32 Count = PlatformPtr->Release(); Count != CreationCount-1 && Count != 0)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate platform. Platform has invalid reference count. Expected (%d), Actual (%d)", CreationCount-1, Count);
			return false;
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcPlatformGetTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcPlatformGetTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcPlatformGetTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		TRefCountPtr<EpicRtcPlatformInterface> Platform;
		Result = GetOrCreatePlatform({}, Platform.GetInitReference());
		if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
		{
			return false;
		}

		TRefCountPtr<EpicRtcPlatformInterface> OtherPlatform;

		Result = GetOrCreatePlatform({}, OtherPlatform.GetInitReference());
		if (!ValidatePlatform(OtherPlatform, Result, { EpicRtcErrorCode::FoundExistingPlatform }, 2))
		{
			return false;
		}

		if (Platform.GetReference() != OtherPlatform.GetReference())
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Expected Platform and OtherPlatform to reference the same pointer");
			return false;
		}

		return true;
	}
} // namespace UE::PixelStreaming2

#endif
