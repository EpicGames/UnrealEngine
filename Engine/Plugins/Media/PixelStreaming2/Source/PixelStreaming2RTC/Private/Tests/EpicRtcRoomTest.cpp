// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcUtils.h"
#include "EpicRtcWebsocket.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "UtilsString.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	// Tests the creation and removal of a session
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcRoomLifetimeTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcRoomLifetimeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcRoomLifetimeTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		FUtf8String ConferenceId = "test_conference";
		FUtf8String RoomId = "test_room";
		FUtf8String SessionId = "test_session";
		FUtf8String SessionUrl = "test_url";

		// Create and validate platform
		TRefCountPtr<EpicRtcPlatformInterface> Platform;
		Result = GetOrCreatePlatform({}, Platform.GetInitReference());
		if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
		{
			return false;
		}

		TRefCountPtr<FMockWebSocketFactory>		WebsocketFactory = MakeRefCount<FMockWebSocketFactory>();
		TSharedPtr<FMockWebSocket>				MockWebsocketConnection;
		TRefCountPtr<EpicRtcWebsocketInterface> Websocket = WebsocketFactory->Get(MockWebsocketConnection);

		// Create and validate conference
		TSharedPtr<FMockManager> Manager = MakeShared<FMockManager>();
		Result = Platform->CreateConference(ToEpicRtcStringView(ConferenceId),
			{
				._websocketFactory = WebsocketFactory.GetReference(),
				._signallingType = EpicRtcSignallingType::PixelStreaming,
			},
			Manager->GetEpicRtcConference().GetInitReference());
		// Count should be two. One for `Conference` and another from EpicRtc storing internally
		if (!ValidateResultRefCount(Manager->GetEpicRtcConference(), "Conference", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		// Initialize session requirements
		MockWebsocketConnection->OnConnected().AddLambda([MockWebsocketConnection]() {
			// simulate the signalling server sending an identify message on connection
			MockWebsocketConnection->OnMessage().Broadcast(R"({"type" : "identify"})");
		});

		Manager->GetSessionObserver() = MakeRefCount<FEpicRtcSessionObserver>(TObserver<IPixelStreaming2SessionObserver>(Manager));
		Manager->GetRoomObserver() = MakeRefCount<FEpicRtcRoomObserver>(TObserver<IPixelStreaming2RoomObserver>(Manager));
		Manager->GetAudioTrackObserverFactory() = MakeRefCount<FEpicRtcAudioTrackObserverFactory>(TObserver<IPixelStreaming2AudioTrackObserver>(Manager));
		Manager->GetDataTrackObserverFactory() = MakeRefCount<FEpicRtcDataTrackObserverFactory>(TObserver<IPixelStreaming2DataTrackObserver>(Manager));
		Manager->GetVideoTrackObserverFactory() = MakeRefCount<FEpicRtcVideoTrackObserverFactory>(TObserver<IPixelStreaming2VideoTrackObserver>(Manager));

		EpicRtcSessionState SessionState = EpicRtcSessionState::Disconnected;
		EpicRtcRoomState	RoomState = EpicRtcRoomState::Failed;

		Manager->OnSessionStateUpdateNative.AddLambda([this, RoomId, Manager, &SessionState](const EpicRtcSessionState State) {
			switch (State)
			{
				case EpicRtcSessionState::Connected:
				{
					// Create and validate room
					EpicRtcConnectionConfig ConnectionConfig = {
						._iceServers = { ._ptr = nullptr, ._size = 0 },
						._iceConnectionPolicy = EpicRtcIcePolicy::All,
						._disableTcpCandidates = false
					};

					EpicRtcRoomConfig RoomConfig = {
						._id = ToEpicRtcStringView(RoomId),
						._connectionConfig = ConnectionConfig,
						._ticket = { ._ptr = nullptr, ._length = 0 },
						._joinParameters = EpicRtcSignallingJoinParameters{
							._options = nullptr,
							._reservedAudioStreamsMaxPoolSize = 0,
							._audioOnly = false
						},
						._observer = Manager->GetRoomObserver(),
						._audioTrackObserverFactory = Manager->GetAudioTrackObserverFactory(),
						._dataTrackObserverFactory = Manager->GetDataTrackObserverFactory(),
						._videoTrackObserverFactory = Manager->GetVideoTrackObserverFactory()
					};

					EpicRtcErrorCode Result = Manager->GetEpicRtcSession()->CreateRoom(RoomConfig, Manager->GetEpicRtcRoom().GetInitReference());
					ValidateResultRefCount(Manager->GetEpicRtcRoom(), "Room", Result, { EpicRtcErrorCode::Ok }, 2);
					break;
				}
				case EpicRtcSessionState::Disconnected:
				case EpicRtcSessionState::New:
				case EpicRtcSessionState::Pending:
				case EpicRtcSessionState::Failed:
				case EpicRtcSessionState::Exiting:
				default:
					break;
			}
			SessionState = State;
		});

		Manager->OnSessionErrorUpdateNative.AddLambda([](EpicRtcErrorCode Error) {
			UE_LOGF(LogPixelStreaming2RTC, Error, "Manager->OnSessionErrorUpdate() {%d}", EnumToUnderlyingType(Error));
		});

		Manager->OnRoomStateUpdateNative.AddLambda([this, &RoomState](const EpicRtcRoomState State) {
			UE_LOGF(LogPixelStreaming2RTC, Log, "Manager->OnRoomStateUpdate() Old State (%ls), New State (%ls)", *ToString(RoomState), *ToString(State));
			RoomState = State;
		});

		// Create and validate session
		EpicRtcSessionConfig SessionConfig = {
			._id = ToEpicRtcStringView(SessionId),
			._url = ToEpicRtcStringView(SessionUrl),
			._observer = Manager->GetSessionObserver()
		};

		Result = Manager->GetEpicRtcConference()->CreateSession(SessionConfig, Manager->GetEpicRtcSession().GetInitReference());
		// Count should be two. One for `Session` and another from EpicRtc storing internally
		if (!ValidateResultRefCount(Manager->GetEpicRtcSession(), "Session", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		Result = Manager->GetEpicRtcSession()->Connect();
		if (!ValidateResultRefCount(Manager->GetEpicRtcSession(), "Session", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		ADD_LATENT_AUTOMATION_COMMAND(FTickAndWaitOrTimeout(Manager, 5.0, [&SessionState]() { return SessionState == EpicRtcSessionState::Connected; }))

		ADD_LATENT_AUTOMATION_COMMAND(FDisconnectRoom(Manager))
		ADD_LATENT_AUTOMATION_COMMAND(FTickAndWaitOrTimeout(Manager, 5.0, [&RoomState]() { return RoomState == EpicRtcRoomState::Left; }))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupRoom(Manager, RoomId))

		ADD_LATENT_AUTOMATION_COMMAND(FDisconnectSession(Manager))
		ADD_LATENT_AUTOMATION_COMMAND(FTickAndWaitOrTimeout(Manager, 5.0, [&SessionState]() { return SessionState == EpicRtcSessionState::Disconnected; }))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupSession(Manager, SessionId))

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupConference(Platform, ConferenceId))

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupManager(Manager))

		return true;
	}
} // namespace UE::PixelStreaming2

#endif
