// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcUtils.h"
#include "EpicRtcWebsocket.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "UtilsString.h"

#include "epic_rtc/core/platform.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	// Tests the creation and removal of a session
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcSessionLifetimeTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcSessionLifetimeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcSessionLifetimeTest::RunTest(const FString& Parameters)
	{
		EpicRtcErrorCode Result;

		FUtf8String ConferenceId = "test_conference";
		FUtf8String SessionId = "test_session";
		FUtf8String SessionUrl = "test_url";

		// Create and validate platform
		TRefCountPtr<EpicRtcPlatformInterface> Platform;
		Result = GetOrCreatePlatform({}, Platform.GetInitReference());
		if (!ValidatePlatform(Platform, Result, { EpicRtcErrorCode::Ok, EpicRtcErrorCode::FoundExistingPlatform }, 1))
		{
			return false;
		}

		TRefCountPtr<FMockWebSocketFactory> WebsocketFactory = MakeRefCount<FMockWebSocketFactory>();

		// Create and validate conference
		TSharedPtr<FMockManager> Manager = MakeShared<FMockManager>();
		Result = Platform->CreateConference(ToEpicRtcStringView(ConferenceId),
			{
				._websocketFactory = WebsocketFactory.GetReference(),
				._signallingType = EpicRtcSignallingType::PixelStreaming,
			},
			Manager->GetEpicRtcConference().GetInitReference());
		// Count should be two. One for `Conference` and another from EpicRtc storing internally
		if (!ValidateResultRefCount(Manager->GetEpicRtcConference(), "Conference", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		// Initialize session requirements
		Manager->GetSessionObserver() = MakeRefCount<FEpicRtcSessionObserver>(TObserver<IPixelStreaming2SessionObserver>(Manager));

		// Create and validate session
		EpicRtcSessionConfig SessionConfig = {
			._id = ToEpicRtcStringView(SessionId),
			._url = ToEpicRtcStringView(SessionUrl),
			._observer = Manager->GetSessionObserver()
		};

		Result = Manager->GetEpicRtcConference()->CreateSession(SessionConfig, Manager->GetEpicRtcSession().GetInitReference());
		// Count should be two. One for `Session` and another from EpicRtc storing internally
		if (!ValidateResultRefCount(Manager->GetEpicRtcSession(), "Session", Result, { EpicRtcErrorCode::Ok }, 2))
		{
			return false;
		}

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupSession(Manager, SessionId))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupConference(Platform, ConferenceId))
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupManager(Manager))

		return true;
	}
} // namespace UE::PixelStreaming2

#endif
