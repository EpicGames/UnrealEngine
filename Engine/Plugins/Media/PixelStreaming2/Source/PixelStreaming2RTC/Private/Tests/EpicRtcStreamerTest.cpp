// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcUtils.h"
#include "EpicRtcWebsocket.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "TestUtils.h"
#include "UtilsString.h"
#include "PixelStreaming2Delegates.h"
#include "PixelStreaming2PluginSettings.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcStreamerDelegateTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcStreamerDelegateTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcStreamerDelegateTest::RunTest(const FString& Parameters)
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString								 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer> Streamer = CreateStreamer(StreamerName, StreamerPort);

		UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
		if (!Delegates)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to obtain delegates pointer. FPS2EpicRtcStreamerDelegateTest will not continue!");
			return true;
		}

		TSharedPtr<bool> bConnectedToSignallingServerNative = MakeShared<bool>(false);
		Delegates->OnConnectedToSignallingServerNative.AddLambda([StreamerName, bConnectedToSignallingServerNative](FString ConnectedStreamer) {
			if (ConnectedStreamer == StreamerName)
			{
				*(bConnectedToSignallingServerNative.Get()) = true;
			}
		});

		TSharedPtr<bool> bDisconnectedFromSignallingServerNative = MakeShared<bool>(false);
		Delegates->OnDisconnectedFromSignallingServerNative.AddLambda([StreamerName, bDisconnectedFromSignallingServerNative](FString ConnectedStreamer) {
			if (ConnectedStreamer == StreamerName)
			{
				*(bDisconnectedFromSignallingServerNative.Get()) = true;
			}
		});

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		TSharedPtr<bool> bStreamingStopped = MakeShared<bool>(false);
		Streamer->OnStreamingStopped().AddLambda([bStreamingStopped](IPixelStreaming2Streamer*) {
			*(bStreamingStopped.Get()) = true;
		});

		// Start streaming
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming started"), 5.0, Streamer, bStreamingStarted, true))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check connected to signalling server"), 5.0, Streamer, bConnectedToSignallingServerNative, true))
		// Wait 1 second to ensure any websocket message have correctly flowed
		ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(1.0))
		// Stop streaming
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StopStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming stopped"), 5.0, Streamer, bStreamingStopped, true))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check disconnected from signalling server"), 5.0, Streamer, bDisconnectedFromSignallingServerNative, true))
		// Cleanup
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { }));

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2EpicRtcStreamerReconnectTest, "System.Plugins.PixelStreaming2.FPS2EpicRtcStreamerReconnectTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2EpicRtcStreamerReconnectTest::RunTest(const FString& Parameters)
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString								 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer> Streamer = CreateStreamer(StreamerName, StreamerPort);

		UPixelStreaming2Delegates* Delegates = UPixelStreaming2Delegates::Get();
		if (!Delegates)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to obtain delegates pointer. FPS2EpicRtcStreamerReconnectTest will not continue!");
			return true;
		}

		TSharedPtr<bool> bConnectedToSignallingServerNative = MakeShared<bool>(false);
		Delegates->OnConnectedToSignallingServerNative.AddLambda([StreamerName, bConnectedToSignallingServerNative](FString ConnectedStreamer) {
			if (ConnectedStreamer == StreamerName)
			{
				*(bConnectedToSignallingServerNative.Get()) = true;
			}
		});

		TSharedPtr<bool> bDisconnectedFromSignallingServerNative = MakeShared<bool>(false);
		Delegates->OnDisconnectedFromSignallingServerNative.AddLambda([StreamerName, bDisconnectedFromSignallingServerNative](FString ConnectedStreamer) {
			if (ConnectedStreamer == StreamerName)
			{
				*(bDisconnectedFromSignallingServerNative.Get()) = true;
			}
		});

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		TSharedPtr<bool> bStreamingStopped = MakeShared<bool>(false);
		Streamer->OnStreamingStopped().AddLambda([bStreamingStopped](IPixelStreaming2Streamer*) {
			*(bStreamingStopped.Get()) = true;
		});

		// Start streaming
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming started"), 5.0, Streamer, bStreamingStarted, true))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check connected to signalling server"), 5.0, Streamer, bConnectedToSignallingServerNative, true))
		// Wait 1 second to ensure any websocket message have correctly flowed
		ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(1.0))
		// Reset the signalling server, this will trigger the reconnection flow
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([SignallingServer]() { SignallingServer->Stop(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming stopped from SS going away"), 5.0, Streamer, bStreamingStopped, true))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check disconnected from signalling server from SS going away"), 5.0, Streamer, bDisconnectedFromSignallingServerNative, true))
		// Reset the state variables
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([bStreamingStarted, bConnectedToSignallingServerNative, bStreamingStopped, bDisconnectedFromSignallingServerNative]() {
			*(bStreamingStarted.Get()) = false;
			*(bConnectedToSignallingServerNative.Get()) = false;
			*(bStreamingStopped.Get()) = false;
			*(bDisconnectedFromSignallingServerNative.Get()) = false;
		}))
		// Restart the signalling server, the streamer should reconnect
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([SignallingServer, StreamerPort, PlayerPort]() {
			UE::PixelStreaming2Servers::FLaunchArgs LaunchArgs;
			LaunchArgs.ProcessArgs = FString::Printf(TEXT("--StreamerPort=%d --HttpPort=%d"), StreamerPort, PlayerPort);
			bool bLaunchedSignallingServer = SignallingServer->Launch(LaunchArgs);
			if (!bLaunchedSignallingServer)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to relaunch signalling server.");
			}
		}))
		// Wait to ensure any websocket message have correctly flowed. Time is 2 times the reconnect interval for safety
		ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(2 * UPixelStreaming2PluginSettings::CVarSignalingReconnectInterval.GetValueOnAnyThread()))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming restarted"), 5.0, Streamer, bStreamingStarted, true))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check reconnected to signalling server"), 5.0, Streamer, bConnectedToSignallingServerNative, true))
		// Wait 1 second to ensure any websocket message have correctly flowed
		ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(1.0))
		// Stop streaming
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StopStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming stopped"), 5.0, Streamer, bStreamingStopped, true))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check disconnected from signalling server"), 5.0, Streamer, bDisconnectedFromSignallingServerNative, true))

		// Cleanup
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { }))

		return true;
	}
} // namespace UE::PixelStreaming2

#endif
