// Copyright Epic Games, Inc. All Rights Reserved.

#include "EpicRtcUtils.h"

namespace UE::PixelStreaming2
{
	bool FTickAndWaitOrTimeout::Update()
	{
		if (Manager->GetEpicRtcConference())
		{
			while (Manager->GetEpicRtcConference()->NeedsTick())
			{
				Manager->GetEpicRtcConference()->Tick();
			}
		}

		if (CheckFunc())
		{
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out");
			return true;
		}
		return false; // Latent Test return false will run again next frame
	}

	bool FDisconnectRoom::Update()
	{
		TRefCountPtr<EpicRtcRoomInterface>& Room = Manager->GetEpicRtcRoom();

		if (!Room)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Unable to disconnect room, Room does not exist");
			return true;
		}

		Room->Leave();

		return true;
	}

	bool FCleanupRoom::Update()
	{
		TRefCountPtr<EpicRtcRoomInterface>& Room = Manager->GetEpicRtcRoom();
		if (!Room)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Unable to update room, Room does not exist");
			return true;
		}

		Manager->GetEpicRtcSession()->RemoveRoom(ToEpicRtcStringView(RoomId));

		// EpicRtc has released its hold on the room. All that should be holding a ref is the manager
		if (Manager->GetEpicRtcRoom()->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Room has invalid reference count. Expected 1, Actual (%d)", Manager->GetEpicRtcRoom()->Count());
			return true;
		}

		// Cannot call Release on TRefCountPtr without it storing a pointer to a released object.
		// So grab the pointer to the object, set to nullptr which will call release
		// By directly calling release, we get the final count which we can check.
		EpicRtcRoomInterface* RoomPtr = Room.GetReference();
		RoomPtr->AddRef();
		Room = nullptr; // will callrelease so count is same as before calling RoomPtr->AddRef();
		if (uint32 Count = RoomPtr->Release(); Count != 0)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Room has invalid reference count. Expected 0, Actual (%d)", Count);
		}

		// The session has been destroyed, the only thing holding a ref to the track observer factories should be the manager
		if (Manager->GetAudioTrackObserverFactory()->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "AudioTrackObserverFactory has invalid reference count. Expected 1, Actual (%d)", Manager->GetAudioTrackObserverFactory()->Count());
			return true;
		}

		if (Manager->GetVideoTrackObserverFactory()->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "VideoTrackObserverFactory has invalid reference count. Expected 1, Actual (%d)", Manager->GetVideoTrackObserverFactory()->Count());
			return true;
		}

		if (Manager->GetDataTrackObserverFactory()->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "DataTrackObserverFactory has invalid reference count. Expected 1, Actual (%d)", Manager->GetDataTrackObserverFactory()->Count());
			return true;
		}

		return true;
	}

	bool FDisconnectSession::Update()
	{
		Manager->GetEpicRtcSession()->Disconnect(EpicRtcStringView{});

		return true;
	}

	bool FCleanupSession::Update()
	{
		Manager->GetEpicRtcConference()->RemoveSession(ToEpicRtcStringView(SessionId));

		// EpicRtc has released its hold on the session. All that should be holding a ref is the manager
		if (Manager->GetEpicRtcSession()->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Session has invalid reference count. Expected 1, Actual (%d)", Manager->GetEpicRtcSession()->Count());
			return true;
		}

		// We know refcount was 1, so setting to nullptr should call the final release and destroy the session
		Manager->GetEpicRtcSession() = nullptr;

		// The session has been destroyed, the only thing holding a ref to the session observer should be the manager
		if (Manager->GetSessionObserver()->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "SessionObserver has invalid reference count. Expected 1, Actual (%d)", Manager->GetSessionObserver()->Count());
			return true;
		}

		return true;
	}

	bool FCleanupConference::Update()
	{
		Platform->ReleaseConference(ToEpicRtcStringView(ConferenceId));

		return true;
	}

	bool FValidateRefCount::Update()
	{
		if (RefCountInterface->Count() != 1)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Object has invalid reference count. Expected (%d), Actual (%d)", 1, RefCountInterface->Count());
			return true;
		}

		return true;
	}

	bool FCleanupManager::Update()
	{
		return true;
	}
} // namespace UE::PixelStreaming2
