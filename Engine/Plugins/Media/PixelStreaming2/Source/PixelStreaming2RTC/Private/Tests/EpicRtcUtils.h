// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcWebsocketFactory.h"
#include "IWebSocket.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "UtilsString.h"

#include "EpicRtcAudioTrackObserver.h"
#include "EpicRtcAudioTrackObserverFactory.h"
#include "EpicRtcDataTrackObserver.h"
#include "EpicRtcDataTrackObserverFactory.h"
#include "EpicRtcVideoTrackObserver.h"
#include "EpicRtcVideoTrackObserverFactory.h"
#include "EpicRtcRoomObserver.h"
#include "EpicRtcSessionObserver.h"

#include "epic_rtc/common/common.h"
#include "epic_rtc/core/conference.h"
#include "epic_rtc/core/platform.h"

namespace UE::PixelStreaming2
{
	// A mock manager class for tests to receive callbacks from EpicRtc. Typically, the controlling class will inherit from the relevant observer interfaces
	// and implement the methods itself (see streamer.cpp). However, we can't force the tests to inherit the class, so instead we have the
	// mock manager and the test bodies bind to the event delegates they're interested in
	class FMockManager :
		public TSharedFromThis<FMockManager>,
		public IPixelStreaming2SessionObserver,
		public IPixelStreaming2RoomObserver,
		public IPixelStreaming2AudioTrackObserver,
		public IPixelStreaming2DataTrackObserver,
		public IPixelStreaming2VideoTrackObserver
	{
	public:
		// Begin IPixelStreaming2SessionObserver
		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnSessionStateUpdate, const EpicRtcSessionState);
		FOnSessionStateUpdate OnSessionStateUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnSessionErrorUpdate, const EpicRtcErrorCode);
		FOnSessionErrorUpdate OnSessionErrorUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnSessionRoomsAvailableUpdate, EpicRtcStringArrayInterface*);
		FOnSessionRoomsAvailableUpdate OnSessionRoomsAvailableUpdateNative;
		// End IPixelStreaming2SessionObserver

		// Begin IPixelStreaming2RoomObserver
		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnRoomStateUpdate, const EpicRtcRoomState);
		FOnRoomStateUpdate OnRoomStateUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnRoomJoinedUpdate, EpicRtcParticipantInterface*);
		FOnRoomJoinedUpdate OnRoomJoinedUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnRoomLeftUpdate, const EpicRtcStringView);
		FOnRoomLeftUpdate OnRoomLeftUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnAudioTrackUpdate, EpicRtcParticipantInterface*, EpicRtcAudioTrackInterface*);
		FOnAudioTrackUpdate OnAudioTrackUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnVideoTrackUpdate, EpicRtcParticipantInterface*, EpicRtcVideoTrackInterface*);
		FOnVideoTrackUpdate OnVideoTrackUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnDataTrackUpdate, EpicRtcParticipantInterface*, EpicRtcDataTrackInterface*);
		FOnDataTrackUpdate OnDataTrackUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnLocalSdpUpdate, EpicRtcParticipantInterface*, EpicRtcSdpInterface*);
		FOnLocalSdpUpdate OnLocalSdpUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnRemoteSdpUpdate, EpicRtcParticipantInterface*, EpicRtcSdpInterface*);
		FOnRemoteSdpUpdate OnRemoteSdpUpdateNative;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnRoomErrorUpdate, const EpicRtcErrorCode);
		FOnRoomErrorUpdate OnRoomErrorUpdateNative;
		// End IPixelStreaming2RoomObserver

		// Begin IPixelStreaming2AudioTrackObserver
		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnAudioTrackMuted, EpicRtcAudioTrackInterface*, EpicRtcBool);
		FOnAudioTrackMuted OnAudioTrackMutedNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnAudioTrackFrame, EpicRtcAudioTrackInterface*, const EpicRtcAudioFrame&);
		FOnAudioTrackFrame OnAudioTrackFrameNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnAudioTrackState, EpicRtcAudioTrackInterface*, const EpicRtcTrackState);
		FOnAudioTrackState OnAudioTrackStateNative;
		// End IPixelStreaming2AudioTrackObserver

		// Begin IPixelStreaming2VideoTrackObserver
		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnVideoTrackMuted, EpicRtcVideoTrackInterface*, EpicRtcBool);
		FOnVideoTrackMuted OnVideoTrackMutedNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnVideoTrackFrame, EpicRtcVideoTrackInterface*, const EpicRtcVideoFrame&);
		FOnVideoTrackFrame OnVideoTrackFrameNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnVideoTrackState, EpicRtcVideoTrackInterface*, const EpicRtcTrackState);
		FOnVideoTrackState OnVideoTrackStateNative;
		// End IPixelStreaming2VideoTrackObserver

		// Begin IPixelStreaming2DataTrackObserver
		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnDataTrackState, EpicRtcDataTrackInterface*, const EpicRtcTrackState);
		FOnDataTrackState OnDataTrackStateNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnDataTrackMessage, EpicRtcDataTrackInterface*, const EpicRtcDataFrame&);
		FOnDataTrackMessage OnDataTrackMessageNative;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnDataTrackError, EpicRtcDataTrackInterface*, const EpicRtcErrorCode);
		FOnDataTrackError OnDataTrackErrorNative;
		// End IPixelStreaming2DataTrackObserver

	public:
		// Begin IPixelStreaming2SessionObserver
		virtual void OnSessionStateUpdate(const EpicRtcSessionState StateUpdate) override
		{
			OnSessionStateUpdateNative.Broadcast(StateUpdate);
		}
		virtual void OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate) override
		{
			OnSessionErrorUpdateNative.Broadcast(ErrorUpdate);
		}
		virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) override
		{
			OnSessionRoomsAvailableUpdateNative.Broadcast(RoomsList);
		}
		// End IPixelStreaming2SessionObserver

		// Begin IPixelStreaming2RoomObserver
		virtual void OnRoomStateUpdate(const EpicRtcRoomState State) override
		{
			OnRoomStateUpdateNative.Broadcast(State);
		}
		virtual void OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) override
		{
			OnRoomJoinedUpdateNative.Broadcast(Participant);
		}
		virtual void OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) override
		{
			OnRoomLeftUpdateNative.Broadcast(ParticipantId);
		}
		virtual void OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) override
		{
			OnAudioTrackUpdateNative.Broadcast(Participant, AudioTrack);
		}
		virtual void OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) override
		{
			OnVideoTrackUpdateNative.Broadcast(Participant, VideoTrack);
		}
		virtual void OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) override
		{
			OnDataTrackUpdateNative.Broadcast(Participant, DataTrack);
		}
		[[nodiscard]] virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override
		{
			OnLocalSdpUpdateNative.Broadcast(Participant, Sdp);
			return nullptr;
		}
		[[nodiscard]] virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override
		{
			OnRemoteSdpUpdateNative.Broadcast(Participant, Sdp);
			return nullptr;
		}
		virtual void OnRoomErrorUpdate(const EpicRtcErrorCode Error) override
		{
			OnRoomErrorUpdateNative.Broadcast(Error);
		}
		// End IPixelStreaming2RoomObserver

		// Begin IPixelStreaming2AudioTrackObserver
		virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) override
		{
			OnAudioTrackMutedNative.Broadcast(AudioTrack, bIsMuted);
		}
		virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) override
		{
			OnAudioTrackFrameNative.Broadcast(AudioTrack, Frame);
		}
		virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) override
		{
			OnAudioTrackStateNative.Broadcast(AudioTrack, State);
		}
		// End IPixelStreaming2AudioTrackObserver

		// Begin IPixelStreaming2VideoTrackObserver
		virtual void OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) override
		{
			OnVideoTrackMutedNative.Broadcast(VideoTrack, bIsMuted);
		}
		virtual void OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) override
		{
			OnVideoTrackFrameNative.Broadcast(VideoTrack, Frame);
		}
		virtual void OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) override
		{
			OnVideoTrackStateNative.Broadcast(VideoTrack, State);
		}
		virtual void OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) override
		{
		}

		virtual EpicRtcBool Enabled() const override
		{
			return true;
		}
		// End IPixelStreaming2VideoTrackObserver

		// Begin IPixelStreaming2DataTrackObserver
		virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) override
		{
			OnDataTrackStateNative.Broadcast(DataTrack, State);
		}
		virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) override
		{
			OnDataTrackMessageNative.Broadcast(DataTrack, DataFrame);
		}
		virtual void OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error) override
		{
			OnDataTrackErrorNative.Broadcast(DataTrack, Error);
		}
		// End IPixelStreaming2DataTrackObserver

	private:
		// Begin EpicRtc Classes
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
		TRefCountPtr<EpicRtcSessionInterface>	 EpicRtcSession;
		TRefCountPtr<EpicRtcRoomInterface>		 EpicRtcRoom;
		// End EpicRtc Classes

		// Begin EpicRtc Observers
		TRefCountPtr<FEpicRtcSessionObserver>			SessionObserver;
		TRefCountPtr<FEpicRtcRoomObserver>				RoomObserver;
		TRefCountPtr<FEpicRtcAudioTrackObserverFactory> AudioTrackObserverFactory;
		TRefCountPtr<FEpicRtcVideoTrackObserverFactory> VideoTrackObserverFactory;
		TRefCountPtr<FEpicRtcDataTrackObserverFactory>	DataTrackObserverFactory;
		// End EpicRtc Observers

	public:
		TRefCountPtr<EpicRtcConferenceInterface>& GetEpicRtcConference()
		{
			return EpicRtcConference;
		}

		TRefCountPtr<EpicRtcSessionInterface>& GetEpicRtcSession()
		{
			return EpicRtcSession;
		}

		TRefCountPtr<EpicRtcRoomInterface>& GetEpicRtcRoom()
		{
			return EpicRtcRoom;
		}

		TRefCountPtr<FEpicRtcSessionObserver>& GetSessionObserver()
		{
			return SessionObserver;
		}

		TRefCountPtr<FEpicRtcRoomObserver>& GetRoomObserver()
		{
			return RoomObserver;
		}

		TRefCountPtr<FEpicRtcAudioTrackObserverFactory>& GetAudioTrackObserverFactory()
		{
			return AudioTrackObserverFactory;
		}

		TRefCountPtr<FEpicRtcVideoTrackObserverFactory>& GetVideoTrackObserverFactory()
		{
			return VideoTrackObserverFactory;
		}

		TRefCountPtr<FEpicRtcDataTrackObserverFactory>& GetDataTrackObserverFactory()
		{
			return DataTrackObserverFactory;
		}
	};

	// For faking a web socket connection
	class FMockWebSocket : public ::IWebSocket
	{
	public:
		FMockWebSocket() = default;
		virtual ~FMockWebSocket() = default;
		virtual void Connect() override
		{
			bConnected = true;
			OnConnectedEvent.Broadcast();
		}
		virtual void							Close(int32 Code = 1000, const FString& Reason = FString()) override { bConnected = false; }
		virtual bool							IsConnected() override { return bConnected; }
		virtual void							Send(const FString& Data) override { OnMessageSentEvent.Broadcast(Data); }
		virtual void							Send(const void* Data, SIZE_T Size, bool bIsBinary = false) override {}
		virtual void							SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit) override {}
		virtual FWebSocketConnectedEvent&		OnConnected() override { return OnConnectedEvent; }
		virtual FWebSocketConnectionErrorEvent& OnConnectionError() override { return OnErrorEvent; }
		virtual FWebSocketClosedEvent&			OnClosed() override { return OnClosedEvent; }
		virtual FWebSocketMessageEvent&			OnMessage() override { return OnMessageEvent; }
		virtual FWebSocketBinaryMessageEvent&	OnBinaryMessage() override { return OnBinaryMessageEvent; }
		virtual FWebSocketRawMessageEvent&		OnRawMessage() override { return OnRawMessageEvent; }
		virtual FWebSocketMessageSentEvent&		OnMessageSent() override { return OnMessageSentEvent; }

	private:
		FWebSocketConnectedEvent	   OnConnectedEvent;
		FWebSocketConnectionErrorEvent OnErrorEvent;
		FWebSocketClosedEvent		   OnClosedEvent;
		FWebSocketMessageEvent		   OnMessageEvent;
		FWebSocketBinaryMessageEvent   OnBinaryMessageEvent;
		FWebSocketRawMessageEvent	   OnRawMessageEvent;
		FWebSocketMessageSentEvent	   OnMessageSentEvent;

		bool bConnected = false;
	};

	class FMockWebSocketFactory : public EpicRtcWebsocketFactoryInterface
	{
	public:
		~FMockWebSocketFactory()
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "FMockWebSocketFactory");
		}

		virtual EpicRtcErrorCode CreateWebsocket(EpicRtcWebsocketInterface** outWebsocket) override
		{
			if (!Websocket)
			{
				TSharedPtr<FMockWebSocket> MockWebsocketConnection = MakeShared<FMockWebSocket>();
				Websocket = MakeRefCount<FEpicRtcWebsocket>(true, MockWebsocketConnection);
			}

			Websocket->AddRef(); // increment for adding the reference to the out

			*outWebsocket = Websocket.GetReference();
			return EpicRtcErrorCode::Ok;
		}

		virtual TRefCountPtr<EpicRtcWebsocketInterface> Get(TSharedPtr<FMockWebSocket>& MockWebsocketConnection)
		{
			if (!Websocket)
			{
				MockWebsocketConnection = MakeShared<FMockWebSocket>();
				Websocket = MakeRefCount<FEpicRtcWebsocket>(true, MockWebsocketConnection);
			}
			return Websocket;
		}

		virtual TRefCountPtr<EpicRtcWebsocketInterface> Get()
		{
			TSharedPtr<FMockWebSocket> MockWebsocketConnection;
			return Get(MockWebsocketConnection);
		}

	public:
		// Begin EpicRtcRefCountInterface
		EPICRTC_REFCOUNT_INTERFACE_IN_PLACE
		// End EpicRtcRefCountInterface

	private:
		TRefCountPtr<EpicRtcWebsocketInterface> Websocket;
	};

	inline FString ToString(TArray<EpicRtcErrorCode> Errors)
	{
		FString Ret = TEXT("");
		for (size_t i = 0; i < Errors.Num(); i++)
		{
			Ret += ToString(Errors[i]);
			if (i < Errors.Num() - 1)
			{
				Ret += TEXT(", ");
			}
		}

		return Ret;
	}

	template <typename RefCountClass>
	bool ValidateRefCount(TRefCountPtr<RefCountClass>& Class, FString Name, uint32_t ExpectedCount)
	{
		if (Class.GetReference() == nullptr)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate %ls. GetReference() = nullptr", *Name);
			return false;
		}

		if (Class->Count() != ExpectedCount)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate %ls. Has invalid reference count. Expected (%d), Actual (%d)", *Name, ExpectedCount, Class->Count());
			return false;
		}

		return true;
	}

	template <typename RefCountClass>
	bool ValidateResultRefCount(TRefCountPtr<RefCountClass>& Class, FString Name, EpicRtcErrorCode Result, TArray<EpicRtcErrorCode> ExpectedResult, uint32_t ExpectedCount)
	{
		if (!ExpectedResult.Contains(Result))
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate %ls. Unexpected result. Expected one of ([%ls]), Actual (%ls)", *Name, *ToString(ExpectedResult), *ToString(Result));
			return false;
		}

		return ValidateRefCount<RefCountClass>(Class, Name, ExpectedCount);
	}

	// NOTE: Because the platform is shared between PS, EOSSDK and these tests, we can't do a != comparison because we don't know what else could have created a platform
	inline bool ValidatePlatform(TRefCountPtr<EpicRtcPlatformInterface>& Platform, EpicRtcErrorCode Result, TArray<EpicRtcErrorCode> ExpectedResult, uint8_t ExpectedCount)
	{
		// NOTE: Because platforms can return either Ok or FoundExistingPlatform (both success cases), we need to check if the result is one of them
		if (!ExpectedResult.Contains(Result))
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate platform. Unexpected result. Expected one of ([%ls]), Actual (%ls)", *ToString(ExpectedResult), *ToString(Result));
			return false;
		}

		if (Platform.GetReference() == nullptr)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate platform. Platform.GetReference() = nullptr");
			return false;
		}

		// NOTE: Because the platform is shared between PS, EOSSDK and these tests, we can't do a != comparison because we don't know what else could have created a platform
		if (Platform->Count() < ExpectedCount)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to validate platform. Platform has invalid reference count. Expected (%d), Actual (%d)", ExpectedCount, Platform->Count());
			return false;
		}

		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FTickAndWaitOrTimeout, TSharedPtr<FMockManager>, Manager, double, TimeoutSeconds, TFunction<bool()>, CheckFunc);
	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDisconnectRoom, TSharedPtr<FMockManager>, Manager);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCleanupRoom, TSharedPtr<FMockManager>, Manager, FUtf8String, RoomId);
	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDisconnectSession, TSharedPtr<FMockManager>, Manager);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCleanupSession, TSharedPtr<FMockManager>, Manager, FUtf8String, SessionId);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCleanupConference, TRefCountPtr<EpicRtcPlatformInterface>, Platform, FUtf8String, ConferenceId);
	// NOTE: This is required to be the last command for any test that uses observers. It's required to keep the manager object alive
	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupManager, TSharedPtr<FMockManager>, Manager);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FValidateRefCount, TRefCountPtr<EpicRtcRefCountInterface>, RefCountInterface, uint8_t, ExpectedCount);
} // namespace UE::PixelStreaming2
