// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

	#include "Logging.h"
	#include "Misc/AutomationTest.h"
	#include "PixelStreaming2PluginSettings.h"
	#include "TestUtils.h"
	#include "UtilsAsync.h"

#define NO_TRACK 0
#define LOCAL_TRACK 1
#define REMOTE_TRACK 2

namespace UE::PixelStreaming2
{
	void DoMediaDirectionTest(EMediaType Media, EMediaDirection Direction)
	{
		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString								 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer> Streamer = CreateStreamer(StreamerName, StreamerPort);
		TSharedPtr<FVideoProducerBase>		 VideoProducer = FVideoProducerBase::Create();
		Streamer->SetVideoProducer(VideoProducer);

		EMediaDirection PlayerMediaDirection = EMediaDirection::Bidirectional;
		if (Direction == EMediaDirection::SendOnly)
		{
			// Streamer is only sending so player should only receive
			PlayerMediaDirection = EMediaDirection::RecvOnly;
		}
		else if (Direction == EMediaDirection::RecvOnly)
		{
			// Streamer is only receiving so player should only send
			PlayerMediaDirection = EMediaDirection::SendOnly;
		}
		else if (Direction == EMediaDirection::Disabled)
		{
			// Streamer has disabled media so player disables it too
			PlayerMediaDirection = EMediaDirection::Disabled;
		}

		FMockPlayerConfig PlayerConfig = { .AudioDirection = EMediaDirection::Disabled, .VideoDirection = EMediaDirection::Disabled };
		if (Media == EMediaType::Audio)
		{
			PlayerConfig.AudioDirection = PlayerMediaDirection;
		}
		else if (Media == EMediaType::Video)
		{
			PlayerConfig.VideoDirection = PlayerMediaDirection;
		}

		TSharedPtr<FMockPlayer> Player = CreatePlayer(PlayerConfig);

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForBoolOrTimeout(TEXT("Check streaming started"), 5.0, Streamer, bStreamingStarted, true))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, PlayerPort, StreamerName]() { Player->Connect(PlayerPort, StreamerName); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check player connected"), [Player]() { return Player->IsConnected(); }, 5.0))
		// Wait 1 second to ensure connection is fully established
		ADD_LATENT_AUTOMATION_COMMAND(FWaitSeconds(1.f))

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, Media, Direction]() { 
			uint8 Tracks = 0;
			if (Media == EMediaType::Audio)
			{
				Tracks |= Player->GetHasLocalAudioTrack() ? LOCAL_TRACK : NO_TRACK;
				Tracks |= Player->GetHasRemoteAudioTrack() ? REMOTE_TRACK : NO_TRACK;
			}
			else if (Media == EMediaType::Video)
			{
				Tracks |= Player->GetHasLocalVideoTrack() ? LOCAL_TRACK : NO_TRACK;
				Tracks |= Player->GetHasRemoteVideoTrack() ? REMOTE_TRACK : NO_TRACK;
			}
			else
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest called with invalid media type: {0}", static_cast<uint8>(Media));
				return;
			}

			switch (Direction)
			{
				case EMediaDirection::SendOnly:
				{
					if (Tracks & LOCAL_TRACK)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(SendOnly) expected player to have no local track!");
					}
					if (!(Tracks & REMOTE_TRACK))
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(SendOnly) expected player to have a remote track!");
					}
					break;
				}
				case EMediaDirection::RecvOnly:
				{
					if (!(Tracks & LOCAL_TRACK))
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(RecvOnly) expected player to have a local track!");
					}
					if (Tracks & REMOTE_TRACK)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(RecvOnly) expected player to have no remote track!");
					}
					break;
				}
				case EMediaDirection::Bidirectional:
				{
					if (!(Tracks & LOCAL_TRACK))
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(Bidirectional) expected player to have a local track!");
					}
					if (!(Tracks & REMOTE_TRACK))
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(Bidirectional) expected player to have a remote track!");
					}
					break;
				}
				case EMediaDirection::Disabled:
				{
					if (Tracks & LOCAL_TRACK)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(Disabled) expected player to have no local track!");
					}
					if (Tracks & REMOTE_TRACK)
					{
						UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest(Disabled) expected player to have no remote track!");
					}
					break;
				}
				default:
					UE_LOGFMT(LogPixelStreaming2RTC, Error, "DoMediaDirectionTest called with invalid media direction: {0}", static_cast<uint8>(Direction));
					checkNoEntry();
					break;
			}
		}))

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { Player }))
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2AudioSendOnlyTest, "System.Plugins.PixelStreaming2.FPS2AudioSendOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2AudioSendOnlyTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Audio, EMediaDirection::SendOnly);
		DoMediaDirectionTest(EMediaType::Audio, EMediaDirection::SendOnly);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2AudioRecvOnlyTest, "System.Plugins.PixelStreaming2.FPS2AudioRecvOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2AudioRecvOnlyTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Audio, EMediaDirection::RecvOnly);
		DoMediaDirectionTest(EMediaType::Audio, EMediaDirection::RecvOnly);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2AudioDisabledTest, "System.Plugins.PixelStreaming2.FPS2AudioDisabledTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2AudioDisabledTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Audio, EMediaDirection::Disabled);
		DoMediaDirectionTest(EMediaType::Audio, EMediaDirection::Disabled);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2AudioBidirectionalTest, "System.Plugins.PixelStreaming2.FPS2AudioBidirectionalTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2AudioBidirectionalTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Audio, EMediaDirection::Bidirectional);
		DoMediaDirectionTest(EMediaType::Audio, EMediaDirection::Bidirectional);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VideoSendOnlyTest, "System.Plugins.PixelStreaming2.FPS2VideoSendOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VideoSendOnlyTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Video, EMediaDirection::SendOnly);
		DoMediaDirectionTest(EMediaType::Video, EMediaDirection::SendOnly);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VideoRecvOnlyTest, "System.Plugins.PixelStreaming2.FPS2VideoRecvOnlyTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VideoRecvOnlyTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Video, EMediaDirection::RecvOnly);
		DoMediaDirectionTest(EMediaType::Video, EMediaDirection::RecvOnly);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VideoDisabledTest, "System.Plugins.PixelStreaming2.FPS2VideoDisabledTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VideoDisabledTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Video, EMediaDirection::Disabled);
		DoMediaDirectionTest(EMediaType::Video, EMediaDirection::Disabled);

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VideoBidirectionalTest, "System.Plugins.PixelStreaming2.FPS2VideoBidirectionalTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VideoBidirectionalTest::RunTest(const FString& Parameters)
	{
		SetMediaDirection(EMediaType::Video, EMediaDirection::Bidirectional);
		DoMediaDirectionTest(EMediaType::Video, EMediaDirection::Bidirectional);

		return true;
	}
} // namespace UE::PixelStreaming2

#undef NO_TRACK
#undef LOCAL_TRACK
#undef REMOTE_TRACK

#endif // WITH_DEV_AUTOMATION_TESTS