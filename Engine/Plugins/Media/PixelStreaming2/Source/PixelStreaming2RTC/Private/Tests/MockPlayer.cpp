// Copyright Epic Games, Inc. All Rights Reserved.

#include "MockPlayer.h"

#include "Async/Async.h"
#include "EpicRtcVideoEncoderInitializer.h"
#include "EpicRtcVideoDecoderInitializer.h"
#include "EpicRtcWebsocketFactory.h"
#include "Logging.h"
#include "PixelStreaming2DefaultDataProtocol.h"
#include "UtilsString.h"

#include "epic_rtc/core/platform.h"

#if WITH_TESTS

namespace UE::PixelStreaming2
{
	uint32_t FMockPlayer::PlayerId = 0;

	void FMockVideoSink::OnFrame(const EpicRtcVideoFrame& Frame)
	{
		VideoBuffer = Frame._buffer;
		bReceivedFrame = true;
	}

	void FMockVideoSink::ResetReceivedFrame()
	{
		bReceivedFrame = false;
		VideoBuffer.SafeRelease();
	}

	TSharedPtr<FMockPlayer> FMockPlayer::Create(FMockPlayerConfig Config)
	{
		TSharedPtr<FMockPlayer> Player = MakeShareable(new FMockPlayer(Config));

		TWeakPtr<FMockPlayer> WeakPlayer = Player;

		Player->SessionObserver = MakeRefCount<FEpicRtcSessionObserver>(TObserver<IPixelStreaming2SessionObserver>(WeakPlayer));
		Player->RoomObserver = MakeRefCount<FEpicRtcRoomObserver>(TObserver<IPixelStreaming2RoomObserver>(WeakPlayer));

		Player->AudioTrackObserverFactory = MakeRefCount<FEpicRtcAudioTrackObserverFactory>(TObserver<IPixelStreaming2AudioTrackObserver>(WeakPlayer));
		Player->VideoTrackObserverFactory = MakeRefCount<FEpicRtcVideoTrackObserverFactory>(TObserver<IPixelStreaming2VideoTrackObserver>(WeakPlayer));
		Player->DataTrackObserverFactory = MakeRefCount<FEpicRtcDataTrackObserverFactory>(TObserver<IPixelStreaming2DataTrackObserver>(WeakPlayer));

		Player->EpicRtcVideoEncoderInitializers = { new FEpicRtcVideoEncoderInitializer() };
		Player->EpicRtcVideoDecoderInitializers = { new FEpicRtcVideoDecoderInitializer() };

		FUtf8String ConferenceId = FUtf8String::Printf("test_conference_%d", PlayerId);

		if (const EpicRtcErrorCode Result = GetOrCreatePlatform({}, Player->Platform.GetInitReference()); Result != EpicRtcErrorCode::Ok && Result != EpicRtcErrorCode::FoundExistingPlatform)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to get or create EpicRtc platform, GetOrCreatePlatform returned: {0}", ToString(Result));
			return nullptr;
		}

		TRefCountPtr<FEpicRtcWebsocketFactory> WebsocketFactory = MakeRefCount<FEpicRtcWebsocketFactory>(false);
		
		EpicRtcConfig ConferenceConfig = { 
			._websocketFactory = WebsocketFactory.GetReference(),
			._signallingType = EpicRtcSignallingType::PixelStreaming,
			._signingPlugin = nullptr,
			._migrationPlugin = nullptr,
			._audioDevicePlugin = nullptr,
			._audioConfig = {
				._tickAdm = true,
				._audioEncoderInitializers = {},
				._audioDecoderInitializers = {},
				._enableBuiltInAudioCodecs = true,
			},
			._videoConfig = { ._videoEncoderInitializers = { ._ptr = const_cast<const EpicRtcVideoEncoderInitializerInterface**>(Player->EpicRtcVideoEncoderInitializers.GetData()), ._size = (uint64_t)Player->EpicRtcVideoEncoderInitializers.Num() }, ._videoDecoderInitializers = { ._ptr = const_cast<const EpicRtcVideoDecoderInitializerInterface**>(Player->EpicRtcVideoDecoderInitializers.GetData()), ._size = (uint64_t)Player->EpicRtcVideoDecoderInitializers.Num() }, ._enableBuiltInVideoCodecs = false },
			._fieldTrials = { ._fieldTrials = EpicRtcStringView{ ._ptr = nullptr, ._length = 0 }, ._isGlobal = 0 } 
		};

		if (const EpicRtcErrorCode Result = Player->Platform->CreateConference(ToEpicRtcStringView(ConferenceId), ConferenceConfig, Player->EpicRtcConference.GetInitReference()); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to create EpicRtc conference, CreateConference returned: {0}", ToString(Result));
			return nullptr;
		}

		Player->TickConferenceTask = FPixelStreamingTickableTask::Create<FEpicRtcTickConferenceTask>(Player->EpicRtcConference, TEXT("FMockPlayer TickConferenceTask"));

		return Player;
	}

	FMockPlayer::FMockPlayer(FMockPlayerConfig Config)
		: VideoSink(MakeShared<FMockVideoSink>())
		, ToStreamerProtocol(UE::PixelStreaming2Input::GetDefaultToStreamerProtocol())
		, PlayerName(FUtf8String::Printf("MockPlayer%d", PlayerId++))
		, AudioDirection(Config.AudioDirection)
		, VideoDirection(Config.VideoDirection)
	{
	}

	FMockPlayer::~FMockPlayer()
	{
		Disconnect(TEXT("Mock player being destroyed"));

		if (EpicRtcConference)
		{
			Platform->ReleaseConference(EpicRtcConference->GetId());
		}
	}

	void FMockPlayer::Connect(int StreamerPort, const FString& StreamerId)
	{
		TargetStreamer = *StreamerId;

		FUtf8String Url(FString::Printf(TEXT("ws://127.0.0.1:%d/"), StreamerPort));
		FUtf8String ConnectionUrl = Url + +(Url.Contains(TEXT("?")) ? TEXT("&") : TEXT("?")) + TEXT("isStreamer=false");

		EpicRtcSessionConfig SessionConfig = {
			._id = ToEpicRtcStringView(PlayerName),
			._url = ToEpicRtcStringView(ConnectionUrl),
			._observer = SessionObserver.GetReference()
		};

		TRefCountPtr<EpicRtcConferenceInterface> SafeConference = EpicRtcConference;
		if (!SafeConference)
		{
			return;
		}

		if (const EpicRtcErrorCode Result = EpicRtcConference->CreateSession(SessionConfig, EpicRtcSession.GetInitReference()); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to create EpicRtc session, CreateSession returned: {0}", ToString(Result));
			return;
		}

		if (const EpicRtcErrorCode Result = EpicRtcSession->Connect(); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to connect EpicRtcSession, Connect returned: {0}", ToString(Result));
		}
	}

	void FMockPlayer::OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack)
	{
		FString ParticipantId{ (int32)Participant->GetId()._length, Participant->GetId()._ptr };
		FString VideoTrackId{ (int32)VideoTrack->GetId()._length, VideoTrack->GetId()._ptr };
		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FMockPlayer::OnVideoTrackUpdate(Participant [{0}], VideoTrack [{1}], Remote [{2}])", ParticipantId, VideoTrackId, static_cast<bool>(VideoTrack->IsRemote()));

		if (VideoTrack->IsRemote())
		{
			bHasRemoteVideoTrack = true;
		}
		else
		{
			bHasLocalVideoTrack = true;
		}
	}

	void FMockPlayer::OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame)
	{
		UE_LOGF(LogPixelStreaming2RTC, VeryVerbose, "FMockPlayer::OnVideoTrackFrame received a video frame.");

		VideoSink->OnFrame(Frame);
	}

	void FMockPlayer::OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted)
	{
	}

	void FMockPlayer::OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State)
	{
	}

	void FMockPlayer::OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame)
	{
	}

	EpicRtcBool FMockPlayer::Enabled() const
	{
		return true;
	}

	void FMockPlayer::OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList)
	{
		TArray<FString> Rooms;
		for (uint64_t i = 0; i < RoomsList->Size(); i++)
		{
			EpicRtcStringInterface* RoomName = RoomsList->Get()[i];
			Rooms.Add(FString{ RoomName->Get(), static_cast<int32>(RoomName->Length()) });
		}

		if (Rooms.IsEmpty())
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Room list empty!");
			return;
		}

		if (!Rooms.Contains(FString(*TargetStreamer)))
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Warning, "Room list [{0}] doesn't contain target room [{1}]!", FString::Join(Rooms, TEXT(",")), TargetStreamer);
			return;
		}

		if (SessionState != EpicRtcSessionState::Connected)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to create subscribe to streamer. EpicRtc session isn't connected");
			return;
		}

		EpicRtcConnectionConfig ConnectionConfig = {
			._iceServers = EpicRtcIceServerSpan{ ._ptr = nullptr, ._size = 0 },
			._iceConnectionPolicy = EpicRtcIcePolicy::All,
			._disableTcpCandidates = false
		};

		EpicRtcRoomConfig RoomConfig = {
			._id = ToEpicRtcStringView(TargetStreamer),
			._connectionConfig = ConnectionConfig,
			._ticket = EpicRtcStringView{ ._ptr = nullptr, ._length = 0 },
			._joinParameters = EpicRtcSignallingJoinParameters{
				._options = nullptr,
				._reservedAudioStreamsMaxPoolSize = 0,
				._audioOnly = false
			},
			._observer = RoomObserver,
			._audioTrackObserverFactory = AudioTrackObserverFactory,
			._dataTrackObserverFactory = DataTrackObserverFactory,
			._videoTrackObserverFactory = VideoTrackObserverFactory
		};

		TRefCountPtr<EpicRtcSessionInterface> SafeSession = EpicRtcSession;
		if (!SafeSession)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to obtain EpicRtcSessionInterface");
			return;
		}

		if (const EpicRtcErrorCode Result = SafeSession->CreateRoom(RoomConfig, EpicRtcRoom.GetInitReference()); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to create EpicRtc room. CreateRoom returned: {0}", ToString(Result));
			return;
		}

		TRefCountPtr<EpicRtcRoomInterface> SafeRoom = EpicRtcRoom;
		if (!SafeRoom)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to obtain EpicRtcRoomInterface");
			return;
		}

		TRefCountPtr<EpicRtcConnectionInterface> ConnectionInterface;
		if (const EpicRtcErrorCode Result = SafeRoom->GetConnection(ConnectionInterface.GetInitReference()); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Failed to get connection interface. GetConnection returned {0}", *ToString(Result));
			return;
		}

		const bool bSyncVideoAndAudio = !UPixelStreaming2PluginSettings::CVarWebRTCDisableAudioSync.GetValueOnAnyThread();

		if (VideoDirection == EMediaDirection::SendOnly || VideoDirection == EMediaDirection::Bidirectional)
		{
			TArray<EpicRtcVideoEncodingConfig> VideoEncodingConfigs;

			VideoEncodingConfigs.Add({
				._rid = EpicRtcStringView{ ._ptr = nullptr, ._length = 0 },
				._scaleResolutionDownBy = 1.0,
				._scalabilityMode = EpicRtcVideoScalabilityMode::L1T1,
				._minBitrate = 1'000'000,
				._maxBitrate = 10'000'000,
				._maxFrameRate = 60 //
			});

			EpicRtcVideoEncodingConfigSpan VideoEncodingConfigSpan = {
				._ptr = VideoEncodingConfigs.GetData(),
				._size = (uint64_t)VideoEncodingConfigs.Num()
			};

			FUtf8String		   VideoStreamID = bSyncVideoAndAudio ? "pixelstreaming_av_stream_id" : "pixelstreaming_video_stream_id";
			EpicRtcVideoSource VideoSource = {
				._streamId = ToEpicRtcStringView(VideoStreamID),
				._encodings = VideoEncodingConfigSpan,
				._direction = EpicRtcMediaSourceDirection::SendRecv
			};

			ConnectionInterface->AddVideoSource(VideoSource);
		}

		if (AudioDirection == EMediaDirection::SendOnly || AudioDirection == EMediaDirection::Bidirectional)
		{

			FUtf8String		   AudioStreamID = bSyncVideoAndAudio ? "pixelstreaming_av_stream_id" : "pixelstreaming_audio_stream_id";
			EpicRtcAudioSource AudioSource = {
				._streamId = ToEpicRtcStringView(AudioStreamID),
				._bitrate = 510000,
				._channels = 2,
				._direction = EpicRtcMediaSourceDirection::SendRecv
			};

			ConnectionInterface->AddAudioSource(AudioSource);
		}

		SafeRoom->Join();
	}

	void FMockPlayer::OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FMockPlayer::OnSessionErrorUpdate({0})", ToString(ErrorUpdate));
	}

	void FMockPlayer::OnRoomStateUpdate(const EpicRtcRoomState State)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FMockPlayer::OnRoomStateUpdate({0})", ToString(State));
	}

	void FMockPlayer::OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FMockPlayer::OnRoomJoinedUpdate(Participant [{0}] joined the room.)", ToString(Participant->GetId()));
	}

	void FMockPlayer::OnRoomLeftUpdate(const EpicRtcStringView ParticipantId)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FMockPlayer::OnRoomLeftUpdate(Participant [{0}] left the room.)", ToString(ParticipantId));
	}

	void FMockPlayer::OnRoomErrorUpdate(const EpicRtcErrorCode Error)
	{
		UE_LOGFMT(LogPixelStreaming2RTC, Log, "FMockPlayer::OnRoomErrorUpdate({0})", ToString(Error));
	}

	void FMockPlayer::OnSessionStateUpdate(const EpicRtcSessionState StateUpdate)
	{
		switch (StateUpdate)
		{
			case EpicRtcSessionState::New:
			case EpicRtcSessionState::Pending:
			case EpicRtcSessionState::Connected:
			case EpicRtcSessionState::Disconnected:
			case EpicRtcSessionState::Failed:
			case EpicRtcSessionState::Exiting:
				SessionState = StateUpdate;
				break;
			default:
				break;
		}
	}

	void FMockPlayer::OnDataTrackMessage(EpicRtcDataTrackInterface* InDataTrack, const EpicRtcDataFrame& InDataFrame)
	{
		// Broadcast must be done on the GameThread because the GameThread can remove the delegates.
		// If removing and broadcast happens simultaneously it causes a datarace failure.
		TWeakPtr<FMockPlayer> WeakPlayer = AsShared();
		const TArray<uint8> Data(InDataFrame._data, InDataFrame._size);
		AsyncTask(ENamedThreads::GameThread, [WeakPlayer, Data]() {
			if (TSharedPtr<FMockPlayer> PinnedPlayer = WeakPlayer.Pin())
			{
				PinnedPlayer->OnMessageReceived.Broadcast(Data);
			}
		});
	}

	void FMockPlayer::OnDataTrackState(EpicRtcDataTrackInterface*, const EpicRtcTrackState) {}

	void FMockPlayer::OnDataTrackUpdate(EpicRtcParticipantInterface*, EpicRtcDataTrackInterface* InDataTrack)
	{
		DataTrack = FEpicRtcDataTrack::Create(InDataTrack, ToStreamerProtocol);
	}

	[[nodiscard]] EpicRtcSdpInterface* FMockPlayer::OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		return nullptr;
	}

	[[nodiscard]] EpicRtcSdpInterface* FMockPlayer::OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp)
	{
		return nullptr;
	}

	void FMockPlayer::OnDataTrackError(EpicRtcDataTrackInterface* InDataTrack, const EpicRtcErrorCode Error)
	{
	}

	void FMockPlayer::OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack)
	{
		FString ParticipantId{ (int32)Participant->GetId()._length, Participant->GetId()._ptr };
		FString AudioTrackId{ (int32)AudioTrack->GetId()._length, AudioTrack->GetId()._ptr };
		UE_LOGFMT(LogPixelStreaming2RTC, VeryVerbose, "FMockPlayer::OnAudioTrackUpdate(Participant [{0}], AudioTrack [{1}], Remote [{2}])", ParticipantId, AudioTrackId, static_cast<bool>(AudioTrack->IsRemote()));

		if (AudioTrack->IsRemote())
		{
			bHasRemoteAudioTrack = true;
		}
		else
		{
			bHasLocalAudioTrack = true;
		}
	}

	void FMockPlayer::OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted)
	{
	}

	void FMockPlayer::OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame)
	{
	}

	void FMockPlayer::OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState)
	{
	}

	void FMockPlayer::Disconnect(const FString& Reason)
	{
		TRefCountPtr<EpicRtcSessionInterface> SafeSession = EpicRtcSession;
		if (!SafeSession)
		{
			return;
		}

		if (TRefCountPtr<EpicRtcRoomInterface> SafeRoom = EpicRtcRoom; SafeRoom)
		{
			SafeRoom->Leave();
			SafeSession->RemoveRoom(ToEpicRtcStringView(TargetStreamer));
			EpicRtcRoom = nullptr;
		}

		FUtf8String Utf8Reason = *Reason;
		if (const EpicRtcErrorCode Result = SafeSession->Disconnect(ToEpicRtcStringView(Utf8Reason)); Result != EpicRtcErrorCode::Ok)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "FMockPlayer failed to disconnect EpicRtcSession. Disconnect returned {0}", ToString(Result));
			return;
		}

		if (TRefCountPtr<EpicRtcConferenceInterface> SafeConference = EpicRtcConference; SafeConference)
		{
			SafeConference->RemoveSession(ToEpicRtcStringView(PlayerName));
			EpicRtcSession = nullptr;
		}
	}

	bool FMockPlayer::IsConnected() const
	{
		return SessionState == EpicRtcSessionState::Connected;
	}

	bool FMockPlayer::IsDisconnected() const
	{
		return SessionState == EpicRtcSessionState::Disconnected;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_TESTS
