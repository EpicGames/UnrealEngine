// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcDataTrack.h"
#include "HAL/ThreadSafeBool.h"
#include "EpicRtcConferenceUtils.h"
#include "IPixelStreaming2DataProtocol.h"
#include "Containers/Utf8String.h"

#include "EpicRtcAudioTrackObserver.h"
#include "EpicRtcAudioTrackObserverFactory.h"
#include "EpicRtcDataTrackObserver.h"
#include "EpicRtcDataTrackObserverFactory.h"
#include "EpicRtcVideoTrackObserver.h"
#include "EpicRtcVideoTrackObserverFactory.h"
#include "EpicRtcRoomObserver.h"
#include "EpicRtcSessionObserver.h"

#if WITH_TESTS

class EpicRtcPlatformInterface;

namespace UE::PixelStreaming2
{
	enum class EMediaDirection : uint8
	{
		Disabled,
		SendOnly,
		RecvOnly,
		Bidirectional
	};

	class FEpicRtcDataTrack;

	struct FMockVideoFrameConfig
	{
		int	  Height;
		int	  Width;
		uint8 Y;
		uint8 U;
		uint8 V;
	};

	class FMockVideoSink
	{
	public:
		void									  OnFrame(const EpicRtcVideoFrame& Frame);
		bool									  HasReceivedFrame() const { return bReceivedFrame; }
		void									  ResetReceivedFrame();
		TRefCountPtr<EpicRtcVideoBufferInterface> GetReceivedBuffer() { return VideoBuffer; };

	private:
		TRefCountPtr<EpicRtcVideoBufferInterface> VideoBuffer;
		FThreadSafeBool							  bReceivedFrame = false;
	};

	struct FMockPlayerConfig
	{
		EMediaDirection AudioDirection;
		EMediaDirection VideoDirection;
	};

	class FMockPlayer :
		public TSharedFromThis<FMockPlayer>,
		public IPixelStreaming2SessionObserver,
		public IPixelStreaming2RoomObserver,
		public IPixelStreaming2AudioTrackObserver,
		public IPixelStreaming2DataTrackObserver,
		public IPixelStreaming2VideoTrackObserver
	{
	public:
		static TSharedPtr<FMockPlayer> Create(FMockPlayerConfig Config = { .AudioDirection = EMediaDirection::RecvOnly, .VideoDirection = EMediaDirection::RecvOnly });
		virtual ~FMockPlayer();

		void Connect(int StreamerPort, const FString& StreamerId);
		void Disconnect(const FString& Reason);
		bool IsConnected() const;
		bool IsDisconnected() const;

		template <typename... Args>
		bool SendMessage(FString MessageType, Args... VarArgs)
		{
			if (!DataTrack)
			{
				return false;
			}

			return DataTrack->SendMessage(MessageType, VarArgs...);
		}

		bool DataChannelAvailable() { return DataTrack.IsValid(); };

		TSharedPtr<FMockVideoSink>				 GetVideoSink() { return VideoSink; };
		TSharedPtr<IPixelStreaming2DataProtocol> GetToStreamerProtocol() { return ToStreamerProtocol; }

		bool GetHasLocalAudioTrack() const { return bHasLocalAudioTrack; };
		bool GetHasRemoteAudioTrack() const { return bHasRemoteAudioTrack; };
		bool GetHasLocalVideoTrack() const { return bHasLocalVideoTrack; };
		bool GetHasRemoteVideoTrack() const { return bHasRemoteVideoTrack; };

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnMessageReceived, const TArray<uint8>&);
		FOnMessageReceived OnMessageReceived;

	public:
		// Begin IPixelStreaming2SessionObserver
		virtual void OnSessionStateUpdate(const EpicRtcSessionState StateUpdate) override;
		virtual void OnSessionErrorUpdate(const EpicRtcErrorCode ErrorUpdate) override;
		virtual void OnSessionRoomsAvailableUpdate(EpicRtcStringArrayInterface* RoomsList) override;
		// End IPixelStreaming2SessionObserver

		// Begin IPixelStreaming2RoomObserver
		virtual void							   OnRoomStateUpdate(const EpicRtcRoomState State) override;
		virtual void							   OnRoomJoinedUpdate(EpicRtcParticipantInterface* Participant) override;
		virtual void							   OnRoomLeftUpdate(const EpicRtcStringView ParticipantId) override;
		virtual void							   OnAudioTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcAudioTrackInterface* AudioTrack) override;
		virtual void							   OnVideoTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcVideoTrackInterface* VideoTrack) override;
		virtual void							   OnDataTrackUpdate(EpicRtcParticipantInterface* Participant, EpicRtcDataTrackInterface* DataTrack) override;
		[[nodiscard]] virtual EpicRtcSdpInterface* OnLocalSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		[[nodiscard]] virtual EpicRtcSdpInterface* OnRemoteSdpUpdate(EpicRtcParticipantInterface* Participant, EpicRtcSdpInterface* Sdp) override;
		virtual void							   OnRoomErrorUpdate(const EpicRtcErrorCode Error) override;
		// End IPixelStreaming2RoomObserver

		// Begin IPixelStreaming2AudioTrackObserver
		virtual void OnAudioTrackMuted(EpicRtcAudioTrackInterface* AudioTrack, EpicRtcBool bIsMuted) override;
		virtual void OnAudioTrackFrame(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcAudioFrame& Frame) override;
		virtual void OnAudioTrackState(EpicRtcAudioTrackInterface* AudioTrack, const EpicRtcTrackState State) override;
		// End IPixelStreaming2AudioTrackObserver

		// Begin IPixelStreaming2VideoTrackObserver
		virtual void		OnVideoTrackMuted(EpicRtcVideoTrackInterface* VideoTrack, EpicRtcBool bIsMuted) override;
		virtual void		OnVideoTrackFrame(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcVideoFrame& Frame) override;
		virtual void		OnVideoTrackState(EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcTrackState State) override;
		virtual void		OnVideoTrackEncodedFrame(EpicRtcStringView ParticipantId, EpicRtcVideoTrackInterface* VideoTrack, const EpicRtcEncodedVideoFrame& EncodedFrame) override;
		virtual EpicRtcBool Enabled() const override;
		// End IPixelStreaming2VideoTrackObserver

		// Begin IPixelStreaming2DataTrackObserver
		virtual void OnDataTrackState(EpicRtcDataTrackInterface* DataTrack, const EpicRtcTrackState State) override;
		virtual void OnDataTrackMessage(EpicRtcDataTrackInterface* DataTrack, const EpicRtcDataFrame& DataFrame) override;
		virtual void OnDataTrackError(EpicRtcDataTrackInterface* DataTrack, const EpicRtcErrorCode Error) override;
		// End IPixelStreaming2DataTrackObserver

	private:
		// Begin EpicRtc Classes
		TRefCountPtr<EpicRtcConferenceInterface> EpicRtcConference;
		TRefCountPtr<EpicRtcSessionInterface>	 EpicRtcSession;
		TRefCountPtr<EpicRtcRoomInterface>		 EpicRtcRoom;
		// End EpicRtc Classes

		// Begin EpicRtc Observers
		TRefCountPtr<FEpicRtcSessionObserver>			SessionObserver;
		TRefCountPtr<FEpicRtcRoomObserver>				RoomObserver;
		TRefCountPtr<FEpicRtcAudioTrackObserverFactory> AudioTrackObserverFactory;
		TRefCountPtr<FEpicRtcVideoTrackObserverFactory> VideoTrackObserverFactory;
		TRefCountPtr<FEpicRtcDataTrackObserverFactory>	DataTrackObserverFactory;
		// End EpicRtc Observers

	private:
		FMockPlayer(FMockPlayerConfig Config);

		TSharedPtr<FMockVideoSink>				   VideoSink;
		TSharedPtr<FEpicRtcDataTrack>			   DataTrack;
		TRefCountPtr<EpicRtcPlatformInterface>	   Platform;
		TSharedPtr<FEpicRtcTickConferenceTask>     TickConferenceTask;
		TSharedPtr<IPixelStreaming2DataProtocol>   ToStreamerProtocol;

		TArray<EpicRtcVideoEncoderInitializerInterface*> EpicRtcVideoEncoderInitializers;
		TArray<EpicRtcVideoDecoderInitializerInterface*> EpicRtcVideoDecoderInitializers;

		EpicRtcSessionState SessionState = EpicRtcSessionState::Disconnected;

		FUtf8String		TargetStreamer;
		FUtf8String		PlayerName;
		static uint32_t PlayerId;

		EMediaDirection AudioDirection;
		EMediaDirection VideoDirection;

		bool bHasLocalAudioTrack = false;
		bool bHasRemoteAudioTrack = false;
		bool bHasLocalVideoTrack = false;
		bool bHasRemoteVideoTrack = false;
	};
} // namespace UE::PixelStreaming2

#endif // WITH_TESTS
