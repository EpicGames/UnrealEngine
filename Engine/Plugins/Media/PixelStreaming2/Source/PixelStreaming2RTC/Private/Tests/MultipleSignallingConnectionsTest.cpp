// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "PixelStreaming2Servers.h"
#include "TestUtils.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER(FWaitForStreamConnected, double, TimeoutSeconds, TSharedPtr<IPixelStreaming2Streamer>, OutStreamer, TSharedPtr<bool>, bIsConnected, TSharedPtr<bool>, bIsDisconnected);
	bool FWaitForStreamConnected::Update()
	{
		if (!OutStreamer.IsValid())
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Streamer not found");
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting for streamer to dis/connect to signalling server.");
			return true;
		}

		if (*bIsDisconnected.Get() == true)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Streamer should not be Disconnected");
		}

		if (*bIsConnected.Get() == true)
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "Streamer is Connected as expected");
			return true;
		}

		return false;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FWaitForStreamDisconnected, double, TimeoutSeconds, TSharedPtr<IPixelStreaming2Streamer>, OutStreamer, TSharedPtr<bool>, bIsStateChanged);
	bool FWaitForStreamDisconnected::Update()
	{
		if (!OutStreamer.IsValid())
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Streamer not found");
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting for streamer to dis/connect to signalling server.");
			return true;
		}

		if (*bIsStateChanged.Get() == true)
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "Streamer is Disconnected as expected");
			return true;
		}

		return false;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForServerOrTimeout, TSharedPtr<UE::PixelStreaming2Servers::IServer>, Server);
	bool FWaitForServerOrTimeout::Update()
	{
		return Server->IsTimedOut() || Server->IsReady();
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDisconnectStreamer, TSharedPtr<IPixelStreaming2Streamer>, Streamer);
	bool FDisconnectStreamer::Update()
	{
		Streamer->StopStreaming();
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupServer, TSharedPtr<UE::PixelStreaming2Servers::IServer>, Server);
	bool FCleanupServer::Update()
	{
		Server->Stop();
		Server.Reset();
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupStreamer, TSharedPtr<IPixelStreaming2Streamer>, Streamer);
	bool FCleanupStreamer::Update()
	{
		Streamer->StopStreaming();
		Streamer.Reset();
		return true;
	}

	class FCheckNumConnected : public IAutomationLatentCommand
	{
	public:
		FCheckNumConnected(double InTimeoutSeconds,
			TSharedPtr<UE::PixelStreaming2Servers::IServer> InSignallingServer,
			uint16 InNumExpected)
			: TimeoutSeconds(InTimeoutSeconds)
			, SignallingServer(InSignallingServer)
			, NumExpected(InNumExpected)
			, bRequestedNumStreamers(false)
			, bHasNumStreamers(false)
			, NumStreamers(0)
		{}
		virtual ~FCheckNumConnected() = default;
		virtual bool Update() override;

	private:
		double TimeoutSeconds;
		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer;
		uint16 NumExpected;
		bool bRequestedNumStreamers;
		bool bHasNumStreamers;
		uint16 NumStreamers;
	};

	bool FCheckNumConnected::Update()
	{
		if (!bRequestedNumStreamers)
		{
			bRequestedNumStreamers = true;
			SignallingServer->GetNumStreamers([this](uint16 InNumStreamers) {
				NumStreamers = InNumStreamers;
				bHasNumStreamers = true;
				});
		}

		if (bHasNumStreamers)
		{
			if (NumStreamers == NumExpected)
			{
				UE_LOGF(LogPixelStreaming2RTC, Log, "Expected %d streamers and found %d", NumExpected, NumStreamers);
				return true;
			}
			else
			{
				bRequestedNumStreamers = false;
			}
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			if (bHasNumStreamers)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Expected %d streamers but found %d", NumExpected, NumStreamers);
			}
			else
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting for number of streamers to be retrieved.");
			}
			return true;
		}

		return false;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2MultipleSignallingConnectionsTest, "System.Plugins.PixelStreaming2.MultipleSignallingConnectionsTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter);
	bool FPS2MultipleSignallingConnectionsTest::RunTest(const FString& Parameters)
	{
		UE_LOGF(LogPixelStreaming2RTC, Log, "----------- ConnectAndDisconnectMultipleStreamersEmbeddedCirrus -----------");

		int32 StreamerPort = TestUtils::NextStreamerPort();
		const int HttpPort = 85;

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = UE::PixelStreaming2Servers::MakeSignallingServer();
		UE::PixelStreaming2Servers::FLaunchArgs LaunchArgs;
		LaunchArgs.bPollUntilReady = true;
		LaunchArgs.ReconnectionTimeoutSeconds = 30.0f;
		LaunchArgs.ReconnectionIntervalSeconds = 2.0f;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--HttpPort=%d --StreamerPort=%d"), HttpPort, StreamerPort);

		SignallingServer->OnReady.AddLambda([this](TMap<UE::PixelStreaming2Servers::EEndpoint, FURL> Endpoints) {
			TestTrue("Got server OnReady.", true);
			});

		SignallingServer->OnFailedToReady.AddLambda([this]() {
			TestTrue("Server was not ready.", false);
			});

		bool bLaunched = SignallingServer->Launch(LaunchArgs);
		UE_LOGF(LogPixelStreaming2RTC, Log, "Embedded cirrus launched: %ls", bLaunched ? TEXT("true") : TEXT("false"));
		TestTrue("Embedded cirrus launched.", bLaunched);

		if (!bLaunched)
		{
			return false;
		}

		// make streamer and connect to signalling server websocket
		FString StreamerName1 = FString("Streamer1");
		FString StreamerName2 = FString("Streamer2");
		TSharedPtr<IPixelStreaming2Streamer> Streamer1 = CreateStreamer(StreamerName1, StreamerPort);
		TSharedPtr<IPixelStreaming2Streamer> Streamer2 = CreateStreamer(StreamerName2, StreamerPort);

		TSharedPtr<bool> stream1Connected = MakeShared<bool>(false);
		TSharedPtr<bool> stream2Connected = MakeShared<bool>(false);
		TSharedPtr<bool> stream1Disconnected = MakeShared<bool>(false);
		TSharedPtr<bool> stream2Disconnected = MakeShared<bool>(false);
		Streamer1->OnStreamingStarted().AddLambda([stream1Connected](IPixelStreaming2Streamer*) { *stream1Connected.Get() = true; });
		Streamer2->OnStreamingStarted().AddLambda([stream2Connected](IPixelStreaming2Streamer*) { *stream2Connected.Get() = true; });
		Streamer1->OnStreamingStopped().AddLambda([stream1Disconnected](IPixelStreaming2Streamer*) { *stream1Disconnected.Get() = true; });
		Streamer2->OnStreamingStopped().AddLambda([stream2Disconnected](IPixelStreaming2Streamer*) { *stream2Disconnected.Get() = true; });

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer1]() { Streamer1->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer2]() { Streamer2->StartStreaming(); }))

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForServerOrTimeout(SignallingServer));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForStreamConnected(5.0f, Streamer1, stream1Connected, stream1Disconnected));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForStreamConnected(5.0f, Streamer2, stream2Connected, stream2Disconnected));
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumConnected(5.0f, SignallingServer, 2));

		ADD_LATENT_AUTOMATION_COMMAND(FDisconnectStreamer(Streamer1));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForStreamDisconnected(5.0f, Streamer1, stream1Disconnected));
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumConnected(5.0f, SignallingServer, 1));

		ADD_LATENT_AUTOMATION_COMMAND(FDisconnectStreamer(Streamer2));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForStreamDisconnected(5.0f, Streamer2, stream2Disconnected));
		ADD_LATENT_AUTOMATION_COMMAND(FCheckNumConnected(5.0f, SignallingServer, 0));

		ADD_LATENT_AUTOMATION_COMMAND(FCleanupStreamer(Streamer1));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupStreamer(Streamer2));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupServer(SignallingServer));

		return true;
	}
} // UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS
