// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPixelStreaming2Module.h"
#include "IPixelStreaming2InputModule.h"
#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "Misc/AutomationTest.h"
#include "PixelStreaming2InputEnums.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Tests/AutomationCommon.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2ProtocolTestAddMessage, "System.Plugins.PixelStreaming2.FPS2ProtocolTestAddMessage", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2ProtocolTestAddMessage::RunTest(const FString& Parameters)
	{
		// need to be able to accept codec to handshake otherwise setting local description fails when generating an answer
		SetCodec(EVideoCodec::VP8);

		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString									 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer>	 Streamer = CreateStreamer(StreamerName, StreamerPort);
		TSharedPtr<IPixelStreaming2InputHandler> InputHandler = Streamer->GetInputHandler().Pin();

		const FString CustomMessageName = FString(TEXT("CustomMessage"));

		// Define our message and add it to the protocol
		TSharedPtr<IPixelStreaming2InputMessage> Message = InputHandler->GetToStreamerProtocol()->Add(CustomMessageName, { EPixelStreaming2MessageTypes::Uint16 });
		// Define a handler function
		const TFunction<void(FString, FMemoryReader)> Handler = [this](FString, FMemoryReader Ar) { /* Do nothing */ };
		// Add it to the streamer
		InputHandler->RegisterMessageHandler(CustomMessageName, Handler);

		TSharedPtr<FMockPlayer> Player = CreatePlayer();
		Player->GetToStreamerProtocol()->Add(CustomMessageName, { EPixelStreaming2MessageTypes::Uint16 });
		TSharedPtr<FMockVideoSink> VideoSink = Player->GetVideoSink();

		TSharedPtr<bool>					  bComplete = MakeShared<bool>(false);
		TFunction<void(const TArray<uint8>&)> Callback = [bComplete, CustomMessageName](const TArray<uint8>& RawBuffer) {
			const uint8 Type = RawBuffer[0];
			if (Type == 255 && RawBuffer.Num() > 1)
			{
				const size_t  DescriptorSize = (RawBuffer.Num() - 1) / sizeof(TCHAR);
				const TCHAR*  DescPtr = reinterpret_cast<const TCHAR*>(RawBuffer.GetData() + 1);
				const FString JsonRaw(DescriptorSize, DescPtr);

				TSharedPtr<FJsonObject>		   JsonParsed;
				TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonRaw);
				if (FJsonSerializer::Deserialize(JsonReader, JsonParsed))
				{
					double Direction = JsonParsed->GetNumberField(TEXT("Direction"));
					if (!(Direction == static_cast<double>(EPixelStreaming2MessageDirection::ToStreamer)))
					{
						return;
					}

					if (JsonParsed->HasField(CustomMessageName))
					{
						*bComplete.Get() = true;
					}
					else
					{
						UE_LOGF(LogPixelStreaming2RTC, Error, "Expected custom message definition to be in the received protocol.");
					}
				}
			}
		};

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		Player->OnMessageReceived.AddLambda(Callback);

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check streaming started"), [bStreamingStarted]() { return *bStreamingStarted.Get(); }, 5.0))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, PlayerPort, StreamerName]() { Player->Connect(PlayerPort, StreamerName); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check player connected"), [Player]() { return Player->IsConnected(); }, 5.0));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelOrTimeout(5.0, Player));
		ADD_LATENT_AUTOMATION_COMMAND(FSendCustomMessageToStreamer(Player, CustomMessageName, 1337));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelMessageOrTimeout(15.0, Player, Callback, bComplete));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { Player }));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2ProtocolTestUseCustomMessage, "System.Plugins.PixelStreaming2.FPS2ProtocolTestUseCustomMessage", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2ProtocolTestUseCustomMessage::RunTest(const FString& Parameters)
	{
		// need to be able to accept codec to handshake otherwise setting local description fails when generating an answer
		SetCodec(EVideoCodec::VP8);

		int32 StreamerPort = TestUtils::NextStreamerPort();
		int32 PlayerPort = TestUtils::NextPlayerPort();

		TSharedPtr<UE::PixelStreaming2Servers::IServer> SignallingServer = CreateSignallingServer(StreamerPort, PlayerPort);

		FString									 StreamerName(FString::Printf(TEXT("MockStreamer%d"), StreamerPort));
		TSharedPtr<IPixelStreaming2Streamer>	 Streamer = CreateStreamer(StreamerName, StreamerPort);
		TSharedPtr<IPixelStreaming2InputHandler> InputHandler = Streamer->GetInputHandler().Pin();

		// Define our message and add it to the protocol
		const FString							 CustomMessageName = FString(TEXT("CustomMessage"));
		TSharedPtr<IPixelStreaming2InputMessage> Message = InputHandler->GetToStreamerProtocol()->Add(CustomMessageName, { EPixelStreaming2MessageTypes::Uint16 });

		// Define a handler function
		TSharedPtr<bool>							  bComplete = MakeShared<bool>(false);
		const TFunction<void(FString, FMemoryReader)> Handler = [this, bComplete](FString, FMemoryReader Ar) {
			*bComplete.Get() = true;
			uint16 Out;
			Ar << Out;
			TestTrue(TEXT("Expected message content to be 1337."), Out == 1337);
		};
		// Add it to the streamer
		InputHandler->RegisterMessageHandler(CustomMessageName, Handler);

		TFunction<void(const TArray<uint8>&)> Callback = [](const TArray<uint8>&) { /* Do nothing */ };

		TSharedPtr<FMockPlayer> Player = CreatePlayer();
		Player->GetToStreamerProtocol()->Add(CustomMessageName, { EPixelStreaming2MessageTypes::Uint16 });
		TSharedPtr<FMockVideoSink> VideoSink = Player->GetVideoSink();

		TSharedPtr<bool> bStreamingStarted = MakeShared<bool>(false);
		Streamer->OnStreamingStarted().AddLambda([bStreamingStarted](IPixelStreaming2Streamer*) {
			*(bStreamingStarted.Get()) = true;
		});

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Streamer]() { Streamer->StartStreaming(); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check streaming started"), [bStreamingStarted]() { return *bStreamingStarted.Get(); }, 5.0))
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteLambda([Player, PlayerPort, StreamerName]() { Player->Connect(PlayerPort, StreamerName); }))
		ADD_LATENT_AUTOMATION_COMMAND(FCheckLambdaOrTimeout(TEXT("check player connected"), [Player]() { return Player->IsConnected(); }, 5.0));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelOrTimeout(5.0, Player));
		ADD_LATENT_AUTOMATION_COMMAND(FSendCustomMessageToStreamer(Player, CustomMessageName, 1337));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForDataChannelMessageOrTimeout(15.0, Player, Callback, bComplete));
		ADD_LATENT_AUTOMATION_COMMAND(FCleanupAllPlayers(SignallingServer, Streamer, { Player }));
		return true;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS
