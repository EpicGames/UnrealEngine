// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestUtils.h"

#include "EpicRtcStreamer.h"
#include "EpicRtcVideoBufferI420.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "IPixelStreaming2Module.h"
#include "Logging.h"
#include "PixelCaptureInputFrameI420.h"
#include "PixelStreaming2PluginSettings.h"
#include "UtilsAsync.h"
#include "VideoProducer.h"
#include "SocketSubsystem.h"
#include "SocketUtils.h"

#if WITH_TESTS

namespace UE::PixelStreaming2
{
	int32 TestUtils::NextStreamerPort()
	{
		// Start of IANA un-registerable ports (49152 - 65535)
		static int NextStreamerPort = 49152;
		NextStreamerPort = GetNextAvailablePort(NextStreamerPort);
		return NextStreamerPort;
	}

	int32 TestUtils::NextPlayerPort()
	{
		// Half of IANA un-registerable ports (49152 - 65535)
		static int NextPlayerPort = 57344;
		NextPlayerPort = GetNextAvailablePort(NextPlayerPort);
		return NextPlayerPort;
	}

	/* ---------- Latent Automation Commands ----------- */

	bool FWaitSeconds::Update()
	{
		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > WaitSeconds)
		{
			return true;
		}
		return false;
	}

	bool FSendSolidColorFrame::Update()
	{
		TSharedPtr<FPixelCaptureBufferI420> Buffer = MakeShared<FPixelCaptureBufferI420>(FrameConfig.Width, FrameConfig.Height);

		uint8_t* yData = Buffer->GetMutableDataY();
		uint8_t* uData = Buffer->GetMutableDataU();
		uint8_t* vData = Buffer->GetMutableDataV();
		for (int y = 0; y < Buffer->GetHeight(); ++y)
		{
			for (int x = 0; x < Buffer->GetWidth(); ++x)
			{
				const int x2 = x / 2;
				const int y2 = y / 2;

				yData[x + (y * Buffer->GetStrideY())] = FrameConfig.Y;
				uData[x2 + (y2 * Buffer->GetStrideUV())] = FrameConfig.U;
				vData[x2 + (y2 * Buffer->GetStrideUV())] = FrameConfig.V;
			}
		}

		VideoProducer->PushFrame(FPixelCaptureInputFrameI420(Buffer));
		return true;
	}

	bool FSendCustomMessageToStreamer::Update()
	{
		UE_LOGF(LogPixelStreaming2RTC, Log, "FSendCustomMessageToStreamer: %ls", *MessageType);
		if (Player->DataChannelAvailable())
		{
			if (!Player->SendMessage(MessageType, Body))
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Data channel send message failed.");
			}
		}
		else
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "No DataChannel on player.");
		}

		return true;
	}

	bool FSendDataChannelMessageToStreamer::Update()
	{
		UE_LOGF(LogPixelStreaming2RTC, Log, "SendDataChannelMessageToStreamer: %ls, %ls", *MessageType, *Body);
		if (Player->DataChannelAvailable())
		{
			if (!Player->SendMessage(MessageType, Body))
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Data channel send message failed.");
			}
		}
		else
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "No DataChannel on player.");
		}

		return true;
	}

	bool FSendDataChannelMessageFromStreamer::Update()
	{
		UE_LOGF(LogPixelStreaming2RTC, Log, "SendDataChannelMessageFromStreamer: %ls, %ls", *MessageType, *Body);
		if (Streamer)
		{
			Streamer->SendAllPlayersMessage(MessageType, Body);
		}
		else
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "No DataChannel on player.");
		}

		return true;
	}

	bool FWaitForFrameReceived::Update()
	{
		if (VideoSink && VideoSink->HasReceivedFrame())
		{
			UE_LOGF(LogPixelStreaming2RTC, Log, "Successfully received streamed frame.");

			TRefCountPtr<EpicRtcVideoBufferInterface> Buffer = VideoSink->GetReceivedBuffer();

			FString WidthTestString = FString::Printf(TEXT("Expected frame res=%dx%d, actual res=%dx%d"),
				FrameConfig.Width,
				FrameConfig.Height,
				Buffer->GetWidth(),
				Buffer->GetHeight());

			if (FrameConfig.Width != Buffer->GetWidth() || FrameConfig.Height != Buffer->GetHeight())
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "%ls", *WidthTestString);
			}
			else
			{
				UE_LOGF(LogPixelStreaming2RTC, Log, "%ls", *WidthTestString);
			}

			FEpicRtcVideoBuffer* const BaseFrameBuffer = static_cast<FEpicRtcVideoBuffer*>(Buffer.GetReference());
			if (!BaseFrameBuffer)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Invalid frame buffer");
				return true;
			}

			if (BaseFrameBuffer->GetBufferFormat() != PixelCaptureBufferFormat::FORMAT_I420)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "Invalid Pixel Format");
				return true;
			}

			uint8_t* DataY = reinterpret_cast<uint8_t*>(Buffer->GetData());
			uint8_t* DataU = DataY + Buffer->GetWidth() * Buffer->GetHeight();
			uint8_t* DataV = DataU + ((Buffer->GetWidth() + 1) / 2) * ((Buffer->GetHeight() + 1) / 2);

			/* ----- Test the pixels of the received frame ---- */

			// Due this frame being a single solid color we "should" only need to look at a single element.
			FString PixelTestString = FString::Printf(TEXT("Expected solid color frame.| Expect: Y=%d, Actual: Y=%d | Expected: U=%d, Actual: U=%d | Expected: V=%d, Actual: V=%d"),
				FrameConfig.Y,
				DataY[0],
				FrameConfig.U,
				DataU[0],
				FrameConfig.V,
				DataV[0]);

			const int YDelta = FMath::Max(FrameConfig.Y, DataY[0]) - FMath::Min(FrameConfig.Y, DataY[0]);
			const int UDelta = FMath::Max(FrameConfig.U, DataU[0]) - FMath::Min(FrameConfig.U, DataU[0]);
			const int VDelta = FMath::Max(FrameConfig.V, DataV[0]) - FMath::Min(FrameConfig.V, DataV[0]);
			const int Tolerance = 10;

			// Match pixel values within a tolerance as compression can result in color variations, but not much as this is a solid color.
			if (YDelta > Tolerance || UDelta > Tolerance || VDelta > Tolerance)
			{
				UE_LOGF(LogPixelStreaming2RTC, Error, "%ls", *PixelTestString);
			}
			else
			{
				UE_LOGF(LogPixelStreaming2RTC, Log, "%ls", *PixelTestString);
			}

			// So we can use this sink for this test again if we want.
			VideoSink->ResetReceivedFrame();

			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting to receive a frame of video through the video sink.");
			return true;
		}
		return false;
	}

	bool FWaitForDataChannelOrTimeout::Update()
	{
		if (OutPlayer->DataChannelAvailable())
		{
			return true;
		}

		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Timed out waiting to data channel to be available.");
			return true;
		}
		return false; // Not connected or timed out so run this latent test again next frame
	}

	bool FWaitForDataChannelMessageOrTimeout::Update()
	{
		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Player timed out waiting for a datachannel message.");
			return true;
		}
		return *bComplete.Get(); // Not connected or timed out so run this latent test again next frame
	}

	bool FWaitForBoolOrTimeout::Update()
	{
		double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > WaitSeconds)
		{
			if ((*bCheck.Get()) != bExpectedValue)
			{
				UE_LOGFMT(LogPixelStreaming2RTC, Error, "{0} failed. Expected [{1}] but got [{2}]", CheckName, bExpectedValue, *bCheck.Get());
			}
			return true;
		}

		if ((*bCheck.Get()) == bExpectedValue)
		{
			return true;
		}
		return false; // We keep updating this until the timeout is complete and then we check that there's no tracks added
	}

	bool FCleanupAllPlayers::Update()
	{
		bool bPlayersReset = false;
		for (TSharedPtr<FMockPlayer>& Player : OutPlayers)
		{
			if (Player)
			{
				Player.Reset();
				bPlayersReset = true;
			}
		}

		if (bPlayersReset)
		{
			// If the players have been reset, we need to wait a frame as their websockets need to tick to properly close
			return false;
		}

		if (OutStreamer)
		{
			OutStreamer->StopStreaming();
			OutStreamer.Reset();
		}

		if (OutSignallingServer)
		{
			OutSignallingServer->Stop();
			OutSignallingServer.Reset();
		}

		// Restore media directions back to default
		SetMediaDirection(EMediaType::Audio, EMediaDirection::Bidirectional);
		SetMediaDirection(EMediaType::Video, EMediaDirection::Bidirectional);
		return true;
	}

	bool FExecuteLambda::Update()
	{
		Func();
		return true;
	}

	bool FCheckLambdaOrTimeout::Update()
	{
		const double DeltaTime = FPlatformTime::Seconds() - StartTime;
		if (DeltaTime > TimeoutSeconds)
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Timed out waiting for lambda [{0}] to return true.", CheckName);
			return true;
		}

		return Func();
	}

	/* ---------- Utility functions ----------- */

	void SetCodec(EVideoCodec Codec)
	{
		// Set codec
		UE::PixelStreaming2::DoOnGameThreadAndWait(MAX_uint32, [Codec]() {
			UPixelStreaming2PluginSettings::CVarEncoderCodec.AsVariable()->Set(*UE::PixelStreaming2::GetCVarStringFromEnum(Codec));
		});
	}

	void SetMediaDirection(EMediaType MediaType, EMediaDirection Direction)
	{
		bool bTransmit = Direction == EMediaDirection::SendOnly || Direction == EMediaDirection::Bidirectional;
		bool bReceive = Direction == EMediaDirection::RecvOnly || Direction == EMediaDirection::Bidirectional;
		DoOnGameThreadAndWait(MAX_uint32, [MediaType, bTransmit, bReceive]() {
			if (MediaType == EMediaType::Audio)
			{
				UPixelStreaming2PluginSettings::CVarWebRTCDisableTransmitAudio->SetWithCurrentPriority(!bTransmit);
				UPixelStreaming2PluginSettings::CVarWebRTCDisableReceiveAudio->SetWithCurrentPriority(!bReceive);
			}
			else if (MediaType == EMediaType::Video)
			{
				UPixelStreaming2PluginSettings::CVarWebRTCDisableTransmitVideo->SetWithCurrentPriority(!bTransmit);
				UPixelStreaming2PluginSettings::CVarWebRTCDisableReceiveVideo->SetWithCurrentPriority(!bReceive);
			}
		});
	}

	TSharedPtr<IPixelStreaming2Streamer> CreateStreamer(const FString& StreamerName, int StreamerPort)
	{
		TSharedPtr<IPixelStreaming2Streamer> OutStreamer = IPixelStreaming2Module::Get().CreateStreamer(StreamerName);
		OutStreamer->SetVideoProducer(FVideoProducerBase::Create());
		OutStreamer->SetConnectionURL(FString::Printf(TEXT("ws://127.0.0.1:%d"), StreamerPort));

		OutStreamer->OnStreamingStarted().AddLambda([](IPixelStreaming2Streamer*) {
			UE_LOGF(LogPixelStreaming2RTC, Verbose, "CreateStreamer: Streamer Connected");
		});

		return OutStreamer;
	}

	TSharedPtr<FMockPlayer> CreatePlayer(FMockPlayerConfig Config)
	{
		TSharedPtr<FMockPlayer> OutPlayer = FMockPlayer::Create(Config);

		return OutPlayer;
	}

	TSharedPtr<UE::PixelStreaming2Servers::IServer> CreateSignallingServer(int StreamerPort, int PlayerPort)
	{
		// Make signalling server
		TSharedPtr<UE::PixelStreaming2Servers::IServer> OutSignallingServer = UE::PixelStreaming2Servers::MakeSignallingServer();

		UE::PixelStreaming2Servers::FLaunchArgs LaunchArgs;
		LaunchArgs.ProcessArgs = FString::Printf(TEXT("--StreamerPort=%d --HttpPort=%d"), StreamerPort, PlayerPort);
		bool bLaunchedSignallingServer = OutSignallingServer->Launch(LaunchArgs);
		if (!bLaunchedSignallingServer)
		{
			UE_LOGF(LogPixelStreaming2RTC, Error, "Failed to launch signalling server.");
		}
		UE_LOGF(LogPixelStreaming2RTC, Log, "Signalling server launched=%ls", bLaunchedSignallingServer ? TEXT("true") : TEXT("false"));
		return OutSignallingServer;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_TESTS
