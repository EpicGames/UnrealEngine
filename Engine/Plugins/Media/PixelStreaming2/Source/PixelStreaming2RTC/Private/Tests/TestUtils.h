// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixelStreaming2Streamer.h"
#include "Misc/AutomationTest.h"
#include "MockPlayer.h"
#include "PixelStreaming2Servers.h"
#include "VideoProducer.h"
#include "Video/VideoConfig.h"

#if WITH_TESTS

namespace UE::PixelStreaming2
{
	enum class EMediaType : uint8
	{
		Audio,
		Video
	};

	namespace TestUtils
	{
		int32 NextStreamerPort();
		int32 NextPlayerPort();
	} // namespace TestUtils

	// Equivalent to DEFINE_LATENT_AUTOMATION_COMMAND_FOUR_PARAMETER, but instead we define a custom constructor
	class FWaitForDataChannelMessageOrTimeout : public IAutomationLatentCommand
	{
	public:
		FWaitForDataChannelMessageOrTimeout(double InTimeoutSeconds, TSharedPtr<FMockPlayer> InPlayer, TFunction<void(const TArray<uint8>&)>& InCallback, TSharedPtr<bool> InbComplete)
			: TimeoutSeconds(InTimeoutSeconds)
			, Player(InPlayer)
			, Callback(InCallback)
			, bComplete(InbComplete)
		{
			MessageReceivedHandle = Player->OnMessageReceived.AddLambda([MessageCallback = this->Callback](const TArray<uint8>& RawBuffer) {
				MessageCallback(RawBuffer);
			});
		}

		virtual ~FWaitForDataChannelMessageOrTimeout()
		{
			Player->OnMessageReceived.Remove(MessageReceivedHandle);
		}
		virtual bool Update() override;

	private:
		double								  TimeoutSeconds;
		TSharedPtr<FMockPlayer>				  Player;
		TFunction<void(const TArray<uint8>&)> Callback;
		TSharedPtr<bool>					  bComplete;
		FDelegateHandle						  MessageReceivedHandle;
	};

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitSeconds, double, WaitSeconds);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FSendSolidColorFrame, TSharedPtr<IPixelStreaming2VideoProducer>, VideoProducer, FMockVideoFrameConfig, FrameConfig);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FWaitForFrameReceived, double, TimeoutSeconds, TSharedPtr<FMockVideoSink>, VideoSink, FMockVideoFrameConfig, FrameConfig);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FCleanupAllPlayers, TSharedPtr<UE::PixelStreaming2Servers::IServer>, OutSignallingServer, TSharedPtr<IPixelStreaming2Streamer>, OutStreamer, TArray<TSharedPtr<FMockPlayer>>, OutPlayers);
	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FWaitForDataChannelOrTimeout, double, TimeoutSeconds, TSharedPtr<FMockPlayer>, OutPlayer);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FSendDataChannelMessageToStreamer, TSharedPtr<FMockPlayer>, Player, FString, MessageType, const FString, Body);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FSendDataChannelMessageFromStreamer, TSharedPtr<IPixelStreaming2Streamer>, Streamer, FString, MessageType, const FString, Body);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FSendCustomMessageToStreamer, TSharedPtr<FMockPlayer>, Player, FString, MessageType, uint16, Body);
	DEFINE_LATENT_AUTOMATION_COMMAND_FIVE_PARAMETER(FWaitForBoolOrTimeout, const FString, CheckName, double, WaitSeconds, TSharedPtr<IPixelStreaming2Streamer>, Streamer, TSharedPtr<bool>, bCheck, bool, bExpectedValue);
	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FExecuteLambda, const TFunction<void()>, Func);
	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FCheckLambdaOrTimeout, const FString, CheckName, const TFunction<bool()>, Func, double, TimeoutSeconds);

	TSharedPtr<IPixelStreaming2Streamer>			CreateStreamer(const FString& StreamerName, int StreamerPort);
	TSharedPtr<FMockPlayer>							CreatePlayer(FMockPlayerConfig Config = { .AudioDirection = EMediaDirection::RecvOnly, .VideoDirection = EMediaDirection::RecvOnly });
	TSharedPtr<UE::PixelStreaming2Servers::IServer> CreateSignallingServer(int StreamerPort, int PlayerPort);
	void											SetCodec(EVideoCodec Codec);
	void											SetMediaDirection(EMediaType Media, EMediaDirection Direction);
} // namespace UE::PixelStreaming2

#endif // WITH_TESTS