// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "UtilsCodecs.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2H264ProfileLevelToString, "System.Plugins.PixelStreaming2.FPS2H264ProfileLevelToString", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2H264ProfileLevelToString::RunTest(const FString& Parameters)
	{
		// Standard conversion
		TestEqual("EH264Profile::ConstrainedBaseline and EH264Level::Level_3_1 returns \"42e01f\"", "42e01f", *H264ProfileLevelToString(EH264Profile::ConstrainedBaseline, EH264Level::Level_3_1));
		TestEqual("EH264Profile::Baseline and EH264Level::Level_1 returns \"42000a\"", "42000a", *H264ProfileLevelToString(EH264Profile::Baseline, EH264Level::Level_1));
		TestEqual("EH264Profile::Main and EH264Level::Level_3_1 returns \"4d001f\"", "4d001f", *H264ProfileLevelToString(EH264Profile::Main, EH264Level::Level_3_1));
		TestEqual("EH264Profile::ConstrainedHigh and EH264Level::Level_4_2 returns \"640c2a\"", "640c2a", *H264ProfileLevelToString(EH264Profile::ConstrainedHigh, EH264Level::Level_4_2));
		TestEqual("EH264Profile::High and EH264Level::Level_4_2 returns \"64002a\"", "64002a", *H264ProfileLevelToString(EH264Profile::High, EH264Level::Level_4_2));

		// Special case for Level 1b
		TestEqual("EH264Profile::ConstrainedBaseline and EH264Level::Level_1b returns \"42f00b\"", "42f00b", *H264ProfileLevelToString(EH264Profile::ConstrainedBaseline, EH264Level::Level_1b));
		TestEqual("EH264Profile::Baseline and EH264Level::Level_1b returns \"42100b\"", "42100b", *H264ProfileLevelToString(EH264Profile::Baseline, EH264Level::Level_1b));
		TestEqual("EH264Profile::Main and EH264Level::Level_1b returns \"4d100b\"", "4d100b", *H264ProfileLevelToString(EH264Profile::Main, EH264Level::Level_1b));

		// Test known invalid cases
		TestFalse("EH264Profile::High and EH264Level::Level_1b doesn't return a valid profile", H264ProfileLevelToString(EH264Profile::High, EH264Level::Level_1b).IsSet());
		TestFalse("EH264Profile::ConstrainedHigh and EH264Level::Level_1b doesn't return a valid profile", H264ProfileLevelToString(EH264Profile::ConstrainedHigh, EH264Level::Level_1b).IsSet());
		TestFalse("EH264Profile::(255) and EH264Level::Level_3_1 returns doesn't return a valid profile", H264ProfileLevelToString(static_cast<EH264Profile>(255), EH264Level::Level_3_1).IsSet());

		return true;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS
