// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodecUtils.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VP8FrameReceivedTest, "System.Plugins.PixelStreaming2.FPS2VP8FrameReceivedTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VP8FrameReceivedTest::RunTest(const FString& Parameters)
	{
		SetCodec(EVideoCodec::VP8);
		DoFrameReceiveTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VP8FrameResizeTest, "System.Plugins.PixelStreaming2.FPS2VP8FrameResizeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VP8FrameResizeTest::RunTest(const FString& Parameters)
	{
		SetCodec(EVideoCodec::VP8);
		DoFrameResizeMultipleTimesTest();
		return true;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS