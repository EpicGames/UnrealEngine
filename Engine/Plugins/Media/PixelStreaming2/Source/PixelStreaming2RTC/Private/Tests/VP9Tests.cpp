// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodecUtils.h"
#include "TestUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::PixelStreaming2
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VP9FrameReceivedTest, "System.Plugins.PixelStreaming2.FPS2VP9FrameReceivedTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VP9FrameReceivedTest::RunTest(const FString& Parameters)
	{
		SetCodec(EVideoCodec::VP9);
		DoFrameReceiveTest();
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPS2VP9FrameResizeTest, "System.Plugins.PixelStreaming2.FPS2VP9FrameResizeTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ProductFilter)
	bool FPS2VP9FrameResizeTest::RunTest(const FString& Parameters)
	{
		SetCodec(EVideoCodec::VP9);
		DoFrameResizeMultipleTimesTest();
		return true;
	}
} // namespace UE::PixelStreaming2

#endif // WITH_DEV_AUTOMATION_TESTS