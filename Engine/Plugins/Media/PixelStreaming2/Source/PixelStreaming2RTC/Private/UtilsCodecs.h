// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EpicRtcContainers.h"
#include "Logging.h"
#include "UtilsVideo.h"

#include "epic_rtc/core/video/video_common.h"
#include "epic_rtc/core/video/video_rate_control.h"

namespace UE::PixelStreaming2
{
	// Helper array for all scalability modes. EScalabilityMode::None must always be the last entry
	const TArray<EScalabilityMode> AllScalabilityModes = {
		EScalabilityMode::L1T1,
		EScalabilityMode::L1T2,
		EScalabilityMode::L1T3,
		EScalabilityMode::L2T1,
		EScalabilityMode::L2T1h,
		EScalabilityMode::L2T1_KEY,
		EScalabilityMode::L2T2,
		EScalabilityMode::L2T2h,
		EScalabilityMode::L2T2_KEY,
		EScalabilityMode::L2T2_KEY_SHIFT,
		EScalabilityMode::L2T3,
		EScalabilityMode::L2T3h,
		EScalabilityMode::L2T3_KEY,
		EScalabilityMode::L3T1,
		EScalabilityMode::L3T1h,
		EScalabilityMode::L3T1_KEY,
		EScalabilityMode::L3T2,
		EScalabilityMode::L3T2h,
		EScalabilityMode::L3T2_KEY,
		EScalabilityMode::L3T3,
		EScalabilityMode::L3T3h,
		EScalabilityMode::L3T3_KEY,
		EScalabilityMode::S2T1,
		EScalabilityMode::S2T1h,
		EScalabilityMode::S2T2,
		EScalabilityMode::S2T2h,
		EScalabilityMode::S2T3,
		EScalabilityMode::S2T3h,
		EScalabilityMode::S3T1,
		EScalabilityMode::S3T1h,
		EScalabilityMode::S3T2,
		EScalabilityMode::S3T2h,
		EScalabilityMode::S3T3,
		EScalabilityMode::S3T3h,
		EScalabilityMode::None
	};

	// Make sure EpicRtcVideoScalabilityMode and EScalabilityMode match up
	static_assert(EpicRtcVideoScalabilityMode::L1T1 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L1T1));
	static_assert(EpicRtcVideoScalabilityMode::L1T2 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L1T2));
	static_assert(EpicRtcVideoScalabilityMode::L1T3 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L1T3));
	static_assert(EpicRtcVideoScalabilityMode::L2T1 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T1));
	static_assert(EpicRtcVideoScalabilityMode::L2T1h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T1h));
	static_assert(EpicRtcVideoScalabilityMode::L2T1Key == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T1_KEY));
	static_assert(EpicRtcVideoScalabilityMode::L2T2 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T2));
	static_assert(EpicRtcVideoScalabilityMode::L2T2h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T2h));
	static_assert(EpicRtcVideoScalabilityMode::L2T2Key == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T2_KEY));
	static_assert(EpicRtcVideoScalabilityMode::L2T2KeyShift == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T2_KEY_SHIFT));
	static_assert(EpicRtcVideoScalabilityMode::L2T3 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T3));
	static_assert(EpicRtcVideoScalabilityMode::L2T3h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T3h));
	static_assert(EpicRtcVideoScalabilityMode::L2T3Key == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L2T3_KEY));
	static_assert(EpicRtcVideoScalabilityMode::L3T1 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T1));
	static_assert(EpicRtcVideoScalabilityMode::L3T1h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T1h));
	static_assert(EpicRtcVideoScalabilityMode::L3T1Key == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T1_KEY));
	static_assert(EpicRtcVideoScalabilityMode::L3T2 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T2));
	static_assert(EpicRtcVideoScalabilityMode::L3T2h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T2h));
	static_assert(EpicRtcVideoScalabilityMode::L3T2Key == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T2_KEY));
	static_assert(EpicRtcVideoScalabilityMode::L3T3 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T3));
	static_assert(EpicRtcVideoScalabilityMode::L3T3h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T3h));
	static_assert(EpicRtcVideoScalabilityMode::L3T3Key == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::L3T3_KEY));
	static_assert(EpicRtcVideoScalabilityMode::S2T1 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S2T1));
	static_assert(EpicRtcVideoScalabilityMode::S2T1h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S2T1h));
	static_assert(EpicRtcVideoScalabilityMode::S2T2 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S2T2));
	static_assert(EpicRtcVideoScalabilityMode::S2T2h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S2T2h));
	static_assert(EpicRtcVideoScalabilityMode::S2T3 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S2T3));
	static_assert(EpicRtcVideoScalabilityMode::S2T3h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S2T3h));
	static_assert(EpicRtcVideoScalabilityMode::S3T1 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S3T1));
	static_assert(EpicRtcVideoScalabilityMode::S3T1h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S3T1h));
	static_assert(EpicRtcVideoScalabilityMode::S3T2 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S3T2));
	static_assert(EpicRtcVideoScalabilityMode::S3T2h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S3T2h));
	static_assert(EpicRtcVideoScalabilityMode::S3T3 == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S3T3));
	static_assert(EpicRtcVideoScalabilityMode::S3T3h == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::S3T3h));
	static_assert(EpicRtcVideoScalabilityMode::None == static_cast<EpicRtcVideoScalabilityMode>(EScalabilityMode::None));

	using namespace UE::AVCodecCore::H264;
	inline FString H264LevelToString(EH264Level Level)
	{
		switch (Level)
		{
			case EH264Level::Level_1b:
				return TEXT("Level_1b");
			case EH264Level::Level_1:
				return TEXT("Level_1");
			case EH264Level::Level_1_1:
				return TEXT("Level_1_1");
			case EH264Level::Level_1_2:
				return TEXT("Level_1_2");
			case EH264Level::Level_1_3:
				return TEXT("Level_1_3");
			case EH264Level::Level_2:
				return TEXT("Level_2");
			case EH264Level::Level_2_1:
				return TEXT("Level_2_1");
			case EH264Level::Level_2_2:
				return TEXT("Level_2_2");
			case EH264Level::Level_3:
				return TEXT("Level_3");
			case EH264Level::Level_3_1:
				return TEXT("Level_3_1");
			case EH264Level::Level_3_2:
				return TEXT("Level_3_2");
			case EH264Level::Level_4:
				return TEXT("Level_4");
			case EH264Level::Level_4_1:
				return TEXT("Level_4_1");
			case EH264Level::Level_4_2:
				return TEXT("Level_4_2");
			case EH264Level::Level_5:
				return TEXT("Level_5");
			case EH264Level::Level_5_1:
				return TEXT("Level_5_1");
			case EH264Level::Level_5_2:
				return TEXT("Level_5_2");
			default:
				checkNoEntry();
				return TEXT("");
		}
	}

	inline TOptional<FString> H264ProfileLevelToString(EH264Profile Profile, EH264Level Level)
	{
		FString ProfileString;
		if (Level == EH264Level::Level_1b)
		{
			switch (Profile)
			{
				case EH264Profile::ConstrainedBaseline:
					ProfileString = TEXT("42f00b");
					break;
				case EH264Profile::Baseline:
					ProfileString = TEXT("42100b");
					break;
				case EH264Profile::Main:
					ProfileString = TEXT("4d100b");
					break;
				// Level 1b is not allowed for other profiles.
				default:
					return TOptional<FString>();
			}
		}
		else
		{
			FString ProfileIdcIopString;
			switch (Profile)
			{
				case EH264Profile::ConstrainedBaseline:
					ProfileIdcIopString = TEXT("42e0");
					break;
				case EH264Profile::Baseline:
					ProfileIdcIopString = TEXT("4200");
					break;
				case EH264Profile::Main:
					ProfileIdcIopString = TEXT("4d00");
					break;
				case EH264Profile::ConstrainedHigh:
					ProfileIdcIopString = TEXT("640c");
					break;
				case EH264Profile::High:
					ProfileIdcIopString = TEXT("6400");
					break;
				case EH264Profile::High444:
					ProfileIdcIopString = TEXT("f400");
					break;
				// Unrecognized profile.
				default:
					return TOptional<FString>();
			}

			ProfileString = FString::Printf(TEXT("%s%02x"), *ProfileIdcIopString, static_cast<uint32>(Level));
		}

		return ProfileString;
	}

	inline FEpicRtcParameterPairArray* CreateH264Format(EH264Profile Profile, EH264Level Level)
	{
		TOptional<FString> ProfileString = H264ProfileLevelToString(Profile, Level);
		if (!ProfileString.IsSet())
		{
			UE_LOGFMT(LogPixelStreaming2RTC, Error, "Unable to create H264 profile string from profile ({0}) and level ({1})", StaticEnum<EH264Profile>()->GetNameStringByValue((int32)Profile), H264LevelToString(Level));
			return nullptr;
		}

		return new FEpicRtcParameterPairArray(
			{
				new FEpicRtcParameterPair(new FEpicRtcString(TEXT("profile-level-id")), new FEpicRtcString(*ProfileString)),  //
				new FEpicRtcParameterPair(new FEpicRtcString(TEXT("packetization-mode")), new FEpicRtcString(TEXT("1"))),	  //
				new FEpicRtcParameterPair(new FEpicRtcString(TEXT("level-asymmetry-allowed")), new FEpicRtcString(TEXT("1"))) //
			});
	}

	using namespace UE::AVCodecCore::VP9;
	inline FEpicRtcParameterPairArray* CreateVP9Format(EProfile Profile)
	{
		FString ProfileIdString = FString::Printf(TEXT("%d"), EnumToUnderlyingType(Profile));
		return new FEpicRtcParameterPairArray(
			{
				new FEpicRtcParameterPair(new FEpicRtcString(TEXT("profile-id")), new FEpicRtcString(ProfileIdString)) //
			});
	}

	inline EpicRtcVideoCodec ToEpicRtcVideoCodec(EVideoCodec Codec)
	{
		switch (Codec)
		{
			case EVideoCodec::VP8:
				return EpicRtcVideoCodec::VP8;
			case EVideoCodec::VP9:
				return EpicRtcVideoCodec::VP9;
			case EVideoCodec::H264:
				return EpicRtcVideoCodec::H264;
			case EVideoCodec::AV1:
				return EpicRtcVideoCodec::AV1;
			default:
				checkNoEntry();
				return EpicRtcVideoCodec::VP8;
		}
	}
} // namespace UE::PixelStreaming2