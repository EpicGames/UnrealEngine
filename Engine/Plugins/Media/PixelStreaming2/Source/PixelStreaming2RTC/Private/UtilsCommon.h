// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CommandLine.h"
namespace UE::PixelStreaming2
{
	static const FString SFU_PLAYER_ID_PREFIX = FString(TEXT("SFU_"));

	inline bool IsSFU(const FString& InPlayerId)
	{
		return InPlayerId.StartsWith(SFU_PLAYER_ID_PREFIX);
	}
} // namespace UE::PixelStreaming2
