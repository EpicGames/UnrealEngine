// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "IPixelStreaming2Streamer.h"
#include "Framework/Application/SlateApplication.h"

/**
 * The IPixelStreaming2RTCModule interface manages the core functionality of the Pixel Streaming system.
 * This class provides access to streamers, video and audio producers, and handles starting and stopping the streaming process.
 * It also allows interaction with the signalling server and manages the lifecycle of streamers within the Pixel Streaming system.
 */
class IPixelStreaming2RTCModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.
	 * Beware calling this during the shutdown phase, though. Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPixelStreaming2RTCModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixelStreaming2RTCModule>("PixelStreaming2RTC");
	}

	/**
	 * Checks to see if this module is loaded.
	 *
	 * @return True if the module is loaded.
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PixelStreaming2RTC");
	}

	/**
	 * Event fired when internal streamer is initialized and the methods on this module are ready for use.
	 */
	DECLARE_EVENT_OneParam(IPixelStreaming2RTCModule, FReadyEvent, IPixelStreaming2RTCModule&);

	/**
	 * A getter for the OnReady event. Intent is for users to call IPixelStreaming2RTCModule::Get().OnReady().AddXXX.
	 * @return The bindable OnReady event.
	 */
	virtual FReadyEvent& OnReady() = 0;

	/**
	 * Is the PixelStreaming2 module actually ready to use? Is the streamer created.
	 * @return True if Pixel Streaming module methods are ready for use.
	 */
	virtual bool IsReady() = 0;
};
