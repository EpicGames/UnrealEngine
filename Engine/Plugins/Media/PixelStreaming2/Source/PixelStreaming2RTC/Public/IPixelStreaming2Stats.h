// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#define UE_API PIXELSTREAMING2RTC_API

/**
 * Add stats that Pixel Streaming can display on screen or be used internally.
 */

class IPixelStreaming2Stats
{
public:
	/**
	 * Singleton-like access to the Pixel Streaming stats public API.
	 * @return Returns singleton instance of Pixel Streaming stats.
	 */
	static UE_API IPixelStreaming2Stats& Get();

	/**
	 * Graphs a value in a real-time plot that is visible when `stat PixelStreaming2Graphs` is used.
	 * If a graph of this name has not been graphed before this will also create the graph internally.
	 * @param InName The name of the stat to graph, this is used in the graph title and must be unique.
	 * @param InSamples The number of samples (along the x-axis) we wish to graph.
	 * @param InMinRange The minimum value we expect for this value we are graphing.
	 * @param InMaxRange The maximum value we expect for this value we are graphing.
	 * @param InRefValue A reference value that we draw in the graph as a horizontal line, this can be useful to see if we are within some threshold.
	 */
	virtual void GraphValue(FName InName, float Value, int InSamples, float InMinRange, float InMaxRange, float InRefValue = 0.0f) = 0;
};

#undef UE_API
