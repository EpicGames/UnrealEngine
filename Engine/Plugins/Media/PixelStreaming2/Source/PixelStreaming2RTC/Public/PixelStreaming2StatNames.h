// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

/**
 * Stat Names used by Pixel Streaming.
 */
namespace PixelStreaming2StatNames
{
	PIXELSTREAMING2RTC_API extern const FName Bitrate;
	PIXELSTREAMING2RTC_API extern const FName BitrateMegabits;
	PIXELSTREAMING2RTC_API extern const FName TargetBitrateMegabits;
	PIXELSTREAMING2RTC_API extern const FName MeanSendDelay;
	PIXELSTREAMING2RTC_API extern const FName SourceFps;
	PIXELSTREAMING2RTC_API extern const FName Fps;
	PIXELSTREAMING2RTC_API extern const FName MeanEncodeTime;
	PIXELSTREAMING2RTC_API extern const FName EncodedFramesPerSecond;
	PIXELSTREAMING2RTC_API extern const FName DecodedFramesPerSecond;
	PIXELSTREAMING2RTC_API extern const FName MeanQPPerSecond;
	PIXELSTREAMING2RTC_API extern const FName FramesSentPerSecond;
	PIXELSTREAMING2RTC_API extern const FName FramesReceivedPerSecond;
	PIXELSTREAMING2RTC_API extern const FName JitterBufferDelay;
	PIXELSTREAMING2RTC_API extern const FName FramesSent;
	PIXELSTREAMING2RTC_API extern const FName FramesReceived;
	PIXELSTREAMING2RTC_API extern const FName FramesPerSecond;
	PIXELSTREAMING2RTC_API extern const FName FramesDecoded;
	PIXELSTREAMING2RTC_API extern const FName FramesDropped;
	PIXELSTREAMING2RTC_API extern const FName FramesCorrupted;
	PIXELSTREAMING2RTC_API extern const FName PartialFramesLost;
	PIXELSTREAMING2RTC_API extern const FName FullFramesLost;
	PIXELSTREAMING2RTC_API extern const FName HugeFramesSent;
	PIXELSTREAMING2RTC_API extern const FName JitterBufferTargetDelay;
	PIXELSTREAMING2RTC_API extern const FName InterruptionCount;
	PIXELSTREAMING2RTC_API extern const FName TotalInterruptionDuration;
	PIXELSTREAMING2RTC_API extern const FName FreezeCount;
	PIXELSTREAMING2RTC_API extern const FName PauseCount;
	PIXELSTREAMING2RTC_API extern const FName TotalFreezesDuration;
	PIXELSTREAMING2RTC_API extern const FName TotalPausesDuration;
	PIXELSTREAMING2RTC_API extern const FName FirCount;
	PIXELSTREAMING2RTC_API extern const FName PliCount;
	PIXELSTREAMING2RTC_API extern const FName NackCount;
	PIXELSTREAMING2RTC_API extern const FName RetransmittedBytesSent;
	PIXELSTREAMING2RTC_API extern const FName TargetBitrate;
	PIXELSTREAMING2RTC_API extern const FName TotalEncodeBytesTarget;
	PIXELSTREAMING2RTC_API extern const FName KeyFramesEncoded;
	PIXELSTREAMING2RTC_API extern const FName FrameWidth;
	PIXELSTREAMING2RTC_API extern const FName FrameHeight;
	PIXELSTREAMING2RTC_API extern const FName BytesSent;
	PIXELSTREAMING2RTC_API extern const FName BytesReceived;
	PIXELSTREAMING2RTC_API extern const FName QPSum;
	PIXELSTREAMING2RTC_API extern const FName TotalEncodeTime;
	PIXELSTREAMING2RTC_API extern const FName TotalPacketSendDelay;
	PIXELSTREAMING2RTC_API extern const FName FramesEncoded;
	PIXELSTREAMING2RTC_API extern const FName AvgSendDelay;
	PIXELSTREAMING2RTC_API extern const FName MessagesSent;
	PIXELSTREAMING2RTC_API extern const FName MessagesReceived;
	PIXELSTREAMING2RTC_API extern const FName PacketsLost;
	PIXELSTREAMING2RTC_API extern const FName Jitter;
	PIXELSTREAMING2RTC_API extern const FName RoundTripTime;
	PIXELSTREAMING2RTC_API extern const FName KeyFramesDecoded;
	PIXELSTREAMING2RTC_API extern const FName AudioLevel;
	PIXELSTREAMING2RTC_API extern const FName TotalSamplesDuration;
	PIXELSTREAMING2RTC_API extern const FName AvailableOutgoingBitrate;
	PIXELSTREAMING2RTC_API extern const FName AvailableIncomingBitrate;
	PIXELSTREAMING2RTC_API extern const FName RetransmittedBytesReceived;
	PIXELSTREAMING2RTC_API extern const FName RetransmittedPacketsReceived;
	PIXELSTREAMING2RTC_API extern const FName DataChannelBytesSent;
	PIXELSTREAMING2RTC_API extern const FName DataChannelBytesReceived;
	PIXELSTREAMING2RTC_API extern const FName DataChannelMessagesSent;
	PIXELSTREAMING2RTC_API extern const FName DataChannelMessagesReceived;
	PIXELSTREAMING2RTC_API extern const FName InputController;
	PIXELSTREAMING2RTC_API extern const FName PacketsSent;
	PIXELSTREAMING2RTC_API extern const FName EncoderImplementation;
	PIXELSTREAMING2RTC_API extern const FName DecoderImplementation;
	PIXELSTREAMING2RTC_API extern const FName MimeType;
	PIXELSTREAMING2RTC_API extern const FName Channels;
	PIXELSTREAMING2RTC_API extern const FName ClockRate;
	PIXELSTREAMING2RTC_API extern const FName QualityLimitationReason;
	PIXELSTREAMING2RTC_API extern const FName Rid;

} // namespace PixelStreaming2StatNames
