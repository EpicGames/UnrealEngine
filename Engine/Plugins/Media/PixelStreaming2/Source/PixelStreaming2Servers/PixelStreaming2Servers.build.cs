// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class PixelStreaming2Servers : ModuleRules
	{
		public PixelStreaming2Servers(ReadOnlyTargetRules Target) : base(Target)
		{
			// This is so for game projects using our public headers don't have to include extra modules they might not know about.
			PublicDependencyModuleNames.AddRange(new string[] {});

			// NOTE: General rule is not to access the private folder of another module
			PrivateIncludePaths.AddRange(new string[] {});

			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"Engine",
				"Json",
				"WebSocketNetworking",
				"WebSockets",
				"Sockets"
			});

			PrivateDependencyModuleNames.AddRange(new string[] {});
		}
	}
}