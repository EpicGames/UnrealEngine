// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2Servers.h"
#include "SignallingServer.h"

namespace UE::PixelStreaming2Servers
{
	TSharedPtr<IServer> MakeSignallingServer()
	{
		return MakeShared<FSignallingServer>();
	}

	TSharedPtr<FMonitoredProcess> DownloadPixelStreaming2Servers(bool bSkipIfPresent)
	{
		return Utils::DownloadPixelStreaming2Servers(bSkipIfPresent);
	}
} // namespace UE::PixelStreaming2Servers
