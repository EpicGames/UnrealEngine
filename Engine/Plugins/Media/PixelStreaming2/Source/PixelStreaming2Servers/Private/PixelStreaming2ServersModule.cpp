// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2ServersModule.h"

FPixelStreaming2ServersModule& FPixelStreaming2ServersModule::Get()
{
	return FModuleManager::LoadModuleChecked<FPixelStreaming2ServersModule>("PixelStreaming2Servers");
}

void FPixelStreaming2ServersModule::StartupModule()
{
	// Any startup logic require goes here.
}

IMPLEMENT_MODULE(FPixelStreaming2ServersModule, PixelStreaming2Servers)
