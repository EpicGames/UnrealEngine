// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FPixelStreaming2ServersModule : public IModuleInterface
{
public:
	virtual ~FPixelStreaming2ServersModule() = default;
	virtual void StartupModule() override;

public:
	static FPixelStreaming2ServersModule& Get();

};