// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreaming2Servers.h"
#include "Tickable.h"

namespace UE::PixelStreaming2Servers
{

	// Base class for all servers, provides functionality for servers to be polled for readiness while they start up.
	class FServerBase : public IServer, public FTickableGameObject
	{
	public:
		FServerBase();
		virtual ~FServerBase() override;

		/* Begin IServer */
		virtual void Stop() = 0;
		virtual bool HasLaunched() override;
		virtual bool Launch(FLaunchArgs& InLaunchArgs) override;
		virtual bool IsReady() override;
		virtual bool IsTimedOut() override;
		virtual void GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived) = 0;
		/* End IServer */

		/* Begin FTickableGameObject */
		virtual bool IsTickableWhenPaused() const override;
		virtual bool IsTickableInEditor() const override;
		virtual void Tick(float DeltaTime) override;
		virtual bool IsAllowedToTick() const override;
		TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(PixelStreaming2Servers, STATGROUP_Tickables); }
		/* End FTickableGameObject */
	
	protected:
		/**
		 * Implemented by derived types. Implementation specific but somehow the server has been tested to see if it ready for connections.
		 * @return	True if the server is able to be connected to.
		 **/
		virtual bool TestConnection() = 0;

		/**
		 * Implemented by derived types. The actual implementation of how this specified server is launched.
		 * @param LaunchArgs	The launch arguments to use.
		 * @param OutEndpoints	The output endpoints the user can expect to use with this server.
		 * @return True if the server was able to launch.
		 **/
		virtual bool LaunchImpl(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndpoints) = 0;

	protected:
		FLaunchArgs LaunchArgs;
		bool bHasLaunched = false;
		TMap<EEndpoint, FURL> Endpoints;
		FThreadSafeBool bIsReady = false;
		float PollingStartedSeconds = 0.0f;
		float LastReconnectionTimeSeconds = 0.0f;
		bool bAllowedToTick = true;
		bool bTimedOut = false;
		bool bPollUntilReady = false;

	};

} // UE::PixelStreaming2Servers