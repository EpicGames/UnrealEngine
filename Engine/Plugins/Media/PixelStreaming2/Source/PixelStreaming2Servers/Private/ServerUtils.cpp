// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerUtils.h"
#include "HAL/PlatformFileManager.h"
#include "PixelStreaming2Servers.h"
#include "PixelStreaming2ServersModule.h"
#include "Logging.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonReader.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Misc/Paths.h"
#include "SocketUtils.h"
#include "GenericPlatform/GenericPlatformFile.h"

namespace UE::PixelStreaming2Servers::Utils
{

	TSharedPtr<FMonitoredProcess> LaunchChildProcess(FString ExecutableAbsPath, FString Args, FString LogPrefix, bool bRunAsScript)
	{
		// Check if the binary actually exists
		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
		if (!FileManager.FileExists(*ExecutableAbsPath))
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Cannot start child process - the specified file did not exist. File=%ls", *ExecutableAbsPath);
			return TSharedPtr<FMonitoredProcess>();
		}

		if (bRunAsScript)
		{
// Get the executable we will use to run the scripts (e.g. cmd.exe on Windows)
#if PLATFORM_WINDOWS
			Args = FString::Printf(TEXT("/c \"\"%s\" %s\""), *ExecutableAbsPath, *Args);
			ExecutableAbsPath = TEXT("cmd.exe");

#elif PLATFORM_LINUX
			Args = FString::Printf(TEXT(" -- \"%s\" %s --nosudo"), *ExecutableAbsPath, *Args);
			ExecutableAbsPath = TEXT("/usr/bin/bash");
#elif PLATFORM_MAC
			Args = FString::Printf(TEXT(" -- \"%s\" %s --nosudo"), *ExecutableAbsPath, *Args);
			ExecutableAbsPath = TEXT("/bin/bash");
#else
			UE_LOGF(LogPixelStreaming2Servers, Error, "Unsupported platform for Pixel Streaming.");
			return TSharedPtr<FMonitoredProcess>();
#endif
		}

		TSharedPtr<FMonitoredProcess> ChildProcess = MakeShared<FMonitoredProcess>(
			ExecutableAbsPath,
			Args,
			true,
#if PLATFORM_MAC
			// Pipes cause UE to lockup when destroying on Mac
			false
#else
			true
#endif
		);
		// Bind to output so we can capture the output in the log
		ChildProcess->OnOutput().BindLambda([LogPrefix](FString Output) {
			UE_LOGF(LogPixelStreaming2Servers, Log, "%ls - %ls", *LogPrefix, *Output);
		});
		// Run the child process
		UE_LOGF(LogPixelStreaming2Servers, Log, "Launch child process - %ls %ls", *ExecutableAbsPath, *Args);
		ChildProcess->Launch();
		return ChildProcess;
	}

	bool ExtractValueFromArgs(FString ArgsString, FString ArgKey, FString FallbackValue, FString& OutValue)
	{
		// Tokenize string in single whitespace " ".
		TArray<FString> ArgTokens;
		ArgsString.ParseIntoArray(ArgTokens, TEXT(" "), true);

		for (FString& Token : ArgTokens)
		{
			Token.TrimStartAndEndInline();

			if (!Token.StartsWith(ArgKey, ESearchCase::Type::CaseSensitive))
			{
				continue;
			}

			// We have a matching token for our search "key" - split on it.
			FString RightStr;
			if (!Token.Split(TEXT("="), nullptr, &RightStr))
			{
				continue;
			}

			OutValue = RightStr;
			return true;
		}
		OutValue = FallbackValue;
		return false;
	}

	FString QueryOrSetProcessArgs(FLaunchArgs& LaunchArgs, FString ArgKey, FString FallbackArgValue)
	{
		FString OutValue;
		bool	bExtractedValue = ExtractValueFromArgs(LaunchArgs.ProcessArgs, ArgKey, FallbackArgValue, OutValue);

		// No key was present so we will inject our own.
		if (!bExtractedValue)
		{
			LaunchArgs.ProcessArgs += FString::Printf(TEXT(" %s%s"), *ArgKey, *FallbackArgValue);
		}

		return OutValue;
	}

	bool GetResourcesDir(FString& OutResourcesDir)
	{
#if WITH_EDITOR
		OutResourcesDir = FPaths::EnginePluginsDir() / TEXT("Media") / TEXT("PixelStreaming2") / TEXT("Resources");
#else
		OutResourcesDir = FPaths::ProjectDir() / TEXT("Samples") / TEXT("PixelStreaming2");
#endif // WITH_EDITOR

		OutResourcesDir = FPaths::ConvertRelativePathToFull(OutResourcesDir);

		if (FPaths::DirectoryExists(OutResourcesDir))
		{
			return true;
		}
		return false;
	}

	bool GetWebServersDir(FString& OutWebServersAbsPath)
	{
		bool bResourceDirExists = GetResourcesDir(OutWebServersAbsPath);

		if (!bResourceDirExists)
		{
			return false;
		}

		OutWebServersAbsPath = OutWebServersAbsPath / TEXT("WebServers");

		if (FPaths::DirectoryExists(OutWebServersAbsPath))
		{
			return true;
		}
		return false;
	}

	bool GetDownloadedServer(FString& OutAbsPath, FString ServerDirectoryName)
	{
		bool bServersDirExists = GetWebServersDir(OutAbsPath);

		if (!bServersDirExists)
		{
			return false;
		}

#if PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC
		// Now add the {ServerDirectoryName}/plaform_script/{os_specific_path}/{run_local.bat|sh}
		OutAbsPath = OutAbsPath / ServerDirectoryName / TEXT("platform_scripts");
#if PLATFORM_WINDOWS
		OutAbsPath = OutAbsPath / TEXT("cmd") / TEXT("run_local.bat");
#else
		OutAbsPath = OutAbsPath / TEXT("bash") / TEXT("run_local.sh");
#endif

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (PlatformFile.FileExists(*OutAbsPath))
		{
			return true;
		}
#else
		UE_LOGF(LogPixelStreaming2Servers, Error, "Unsupported platform for Pixel Streaming scripts.");
#endif
		return false;
	}

	TSharedPtr<FMonitoredProcess> DownloadPixelStreaming2Servers(bool bSkipIfPresent)
	{
		FString OutScriptPath;
		if (bSkipIfPresent && GetDownloadedServer(OutScriptPath, FString(TEXT("SignallingWebServer"))))
		{
			UE_LOGF(LogPixelStreaming2Servers, Log, "Found pixel streaming servers, skipping download.");
			// empty process
			return TSharedPtr<FMonitoredProcess>();
		}

		bool bHasWebServersDir = GetWebServersDir(OutScriptPath);

		if (!bHasWebServersDir)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Could not download ps servers, no PixelStreaming2/Resources/WebServers directory found.");
		}

#if PLATFORM_WINDOWS || PLATFORM_LINUX || PLATFORM_MAC
		FString Args = TEXT("");

#if PLATFORM_WINDOWS
		OutScriptPath = OutScriptPath / TEXT("get_ps_servers.bat");
#else
		OutScriptPath = OutScriptPath / TEXT("get_ps_servers.sh");
#endif
		return LaunchChildProcess(OutScriptPath, Args, FString(TEXT("Download ps servers")), true /*bRunAsScript*/);
#else
		UE_LOGF(LogPixelStreaming2Servers, Error, "Unsupported platform for Pixel Streaming scripts.");
		// empty process
		return TSharedPtr<FMonitoredProcess>();
#endif
	}

	bool PopulateCirrusEndPoints(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndPoints)
	{
		FPixelStreaming2ServersModule& Module = FPixelStreaming2ServersModule::Get();

		// Query for ports, or set them if they don't exist
		int FallbackStreamerPort = UE::PixelStreaming2::GetNextAvailablePort();
		int FallbackSFUPort = UE::PixelStreaming2::GetNextAvailablePort();
		int FallbackHttpPort = UE::PixelStreaming2::GetNextAvailablePort();
		int FallbackHttpsPort = UE::PixelStreaming2::GetNextAvailablePort();
		if (FallbackStreamerPort == -1)
		{
			UE_LOGFMT(LogPixelStreaming2Servers, Warning, "Failed to find an available port for streamer connections");
			return false;
		}
		if (FallbackSFUPort == -1)
		{
			UE_LOGFMT(LogPixelStreaming2Servers, Warning, "Failed to find an available port for SFU connections");
			return false;
		}
		if (FallbackHttpPort == -1)
		{
			UE_LOGFMT(LogPixelStreaming2Servers, Warning, "Failed to find an available port for http connections");
			return false;
		}
		FString StreamerPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--StreamerPort="), FString::FromInt(FallbackStreamerPort));
		FString SFUPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--SFUPort="), FString::FromInt(FallbackSFUPort));
		FString HttpPort = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--HttpPort="), FString::FromInt(FallbackHttpPort));
		FString ServeHttps = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--ServeHttps="), TEXT("false"));
		bool	bServeHttps = ServeHttps == TEXT("true");

		// Construct endpoint urls

		// Streamer
		FURL SignallingStreamerUrl;
		SignallingStreamerUrl.Protocol = TEXT("ws");
		SignallingStreamerUrl.Host = TEXT("127.0.0.1");
		SignallingStreamerUrl.Port = FCString::Atoi(*StreamerPort);
		SignallingStreamerUrl.Map = FString();

		// Players
		FURL PlayersUrl;
		PlayersUrl.Protocol = TEXT("ws");
		PlayersUrl.Host = TEXT("127.0.0.1");
		PlayersUrl.Port = FCString::Atoi(*HttpPort);
		PlayersUrl.Map = FString();

		// Webserver
		FURL WebServerUrl;
		WebServerUrl.Protocol = bServeHttps ? TEXT("https") : TEXT("http");
		WebServerUrl.Host = TEXT("127.0.0.1");
		WebServerUrl.Port = FCString::Atoi(*HttpPort);
		WebServerUrl.Map = FString();

		// SFU
		FURL SFUUrl;
		SFUUrl.Protocol = TEXT("ws");
		SFUUrl.Host = TEXT("127.0.0.1");
		SFUUrl.Port = FCString::Atoi(*SFUPort);
		SFUUrl.Map = FString();

		OutEndPoints.Add(EEndpoint::Signalling_Streamer, SignallingStreamerUrl);
		OutEndPoints.Add(EEndpoint::Signalling_Players, PlayersUrl);
		OutEndPoints.Add(EEndpoint::Signalling_SFU, SFUUrl);
		OutEndPoints.Add(EEndpoint::Signalling_Webserver, WebServerUrl);

		return true;
	}

	FString ToString(FURL Url)
	{
		return FString::Printf(TEXT("%s://%s:%d"), *(Url.Protocol), *(Url.Host), Url.Port);
	}

	FString ToString(TArrayView<uint8> UTF8Bytes)
	{
		FUTF8ToTCHAR Converted((const ANSICHAR*)UTF8Bytes.GetData(), UTF8Bytes.Num());
		FString		 OutString(Converted.Length(), Converted.Get());
		return OutString;
	}

	FString ToString(TSharedPtr<FJsonObject> JSONObj)
	{
		FString Res;
		auto	JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Res);
		bool	bSerialized = FJsonSerializer::Serialize(JSONObj.ToSharedRef(), JsonWriter);
		if (!bSerialized)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Failed to stringify JSON object.");
		}
		return Res;
	}

	TSharedPtr<FJsonObject> ToJSON(const FString& InString)
	{
		TSharedPtr<FJsonObject> OutJSON = MakeShared<FJsonObject>();
		const auto				JsonReader = TJsonReaderFactory<TCHAR>::Create(InString);
		if (FJsonSerializer::Deserialize(JsonReader, OutJSON))
		{
			return OutJSON;
		}
		return nullptr;
	}

} // namespace UE::PixelStreaming2Servers::Utils