// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignallingServer.h"
#include "ServerUtils.h"
#include "Logging.h"
#include "Misc/Paths.h"
#include "WebSocketServerWrapper.h"

namespace UE::PixelStreaming2Servers
{
	static const FString LEGACY_NAME = "_LEGACY_";

	FSignallingServer::FSignallingServer()
	{
		StreamerMessageHandlers.Add("endpointId").BindRaw(this, &FSignallingServer::OnStreamerIdMessage);
		StreamerMessageHandlers.Add("ping").BindRaw(this, &FSignallingServer::OnStreamerPingMessage);
		StreamerMessageHandlers.Add("disconnectPlayer").BindRaw(this, &FSignallingServer::OnStreamerDisconnectMessage);

		PlayerMessageHandlers.Add("listStreamers").BindRaw(this, &FSignallingServer::OnPlayerListStreamersMessage);
		PlayerMessageHandlers.Add("subscribe").BindRaw(this, &FSignallingServer::OnPlayerSubscribeMessage);
		PlayerMessageHandlers.Add("unsubscribe").BindRaw(this, &FSignallingServer::OnPlayerUnsubscribeMessage);
		PlayerMessageHandlers.Add("stats").BindRaw(this, &FSignallingServer::OnPlayerStatsMessage);
		PlayerMessageHandlers.Add("ping").BindRaw(this, &FSignallingServer::OnPlayerPingMessage);
	}

	void FSignallingServer::Stop()
	{
		if (StreamersWS)
		{
			StreamersWS->Stop();
			StreamersWS.Reset();
		}
		if (Probe)
		{
			Probe.Reset();
		}
	}

	bool FSignallingServer::TestConnection()
	{
		if (bIsReady)
		{
			return true;
		}
		else
		{
			const bool bConnected = Probe && Probe->Probe();
			if (bConnected)
			{
				// Close the websocket connection so others can use it
				Probe->Close();
				Probe.Reset();
				// Note: even after closing the client WS of the probe above it will take another tick to remove the connection
				// from the ws server so if you query number of streamers during this time we do have to remove the probe from that count
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	// Returns true if the origin string contains a port (e.g. "http://localhost:8080", "http://[::1]:8080")
	// by checking for a colon after the host portion of the URL.
	// Handles IPv6 addresses in brackets (e.g. "http://[::1]") where colons inside brackets are not ports.
	static bool OriginHasPort(const FString& InOrigin)
	{
		int32 SchemeEnd;
		if (InOrigin.FindChar(TEXT('/'), SchemeEnd) && InOrigin.Mid(SchemeEnd).StartsWith(TEXT("//")))
		{
			FString AfterScheme = InOrigin.Mid(SchemeEnd + 3);

			// IPv6: skip past the bracketed address before looking for a port colon
			if (AfterScheme.StartsWith(TEXT("[")))
			{
				int32 BracketEnd;
				if (AfterScheme.FindChar(TEXT(']'), BracketEnd))
				{
					return AfterScheme.Mid(BracketEnd + 1).StartsWith(TEXT(":"));
				}
				return false;
			}

			// IPv4 / hostname: a colon after the scheme means a port
			return AfterScheme.Contains(TEXT(":"));
		}
		return false;
	}

	EWebsocketConnectionFilterResult FSignallingServer::FilterConnection(FString Origin, FString ClientIP)
	{
		// If no allowed origins configured, accept all connections
		if (AllowedOrigins.Num() == 0)
		{
			return EWebsocketConnectionFilterResult::ConnectionAccepted;
		}

		// Wildcard allows all origins we normalize to by adding":*" even if the user passes *
		if (AllowedOrigins.Contains(TEXT("*:*")))
		{
			return EWebsocketConnectionFilterResult::ConnectionAccepted;
		}

		// Normalize: if the origin has no port, append ":*" so it can match patterns with ports
		FString NormalizedOrigin = Origin;
		if (!OriginHasPort(NormalizedOrigin))
		{
			NormalizedOrigin += TEXT(":*");
		}

		// Check if the origin matches any allowed pattern (supports wildcards e.g. http://192.168.0.*:8080)
		for (const FString& AllowedOrigin : AllowedOrigins)
		{
			if (NormalizedOrigin.MatchesWildcard(AllowedOrigin, ESearchCase::IgnoreCase))
			{
				return EWebsocketConnectionFilterResult::ConnectionAccepted;
			}
		}

		UE_LOGFMT(LogPixelStreaming2Servers, Warning, "Rejected WebSocket connection from origin '{0}' (client IP: {1}). Allowed origins: [{2}]", Origin, ClientIP, FString::Join(AllowedOrigins, TEXT(", ")));
		return EWebsocketConnectionFilterResult::ConnectionRefused;
	}

	bool FSignallingServer::LaunchImpl(FLaunchArgs& InLaunchArgs, TMap<EEndpoint, FURL>& OutEndpoints)
	{
		Utils::PopulateCirrusEndPoints(InLaunchArgs, OutEndpoints);
		FURL PlayersURL = OutEndpoints[EEndpoint::Signalling_Players];
		FURL StreamerURL = OutEndpoints[EEndpoint::Signalling_Streamer];

		// Parse allowed origins for CORS filtering
		FString AllowedOriginsArg = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--AllowedOrigins="), TEXT(""));
		if (!AllowedOriginsArg.IsEmpty())
		{
			AllowedOriginsArg.ParseIntoArray(AllowedOrigins, TEXT(","), true);
			for (FString& AllowedOrigin : AllowedOrigins)
			{
				AllowedOrigin.TrimStartAndEndInline();
				// Normalize: if no port specified, append ":*" so it matches origins with any port
				if (!OriginHasPort(AllowedOrigin))
				{
					AllowedOrigin += TEXT(":*");
				}
			}
			UE_LOGFMT(LogPixelStreaming2Servers, Log, "CORS filtering enabled. Allowed origins: {0}", AllowedOriginsArg);
		}

		// Parse IPv6 setting — players WS gets dual-stack, streamers WS stays IPv4-only
		FString EnableIPv6Str = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--EnableIPv6="), TEXT("false"));
		IWebSocketServer::ENetworkProtocol PlayerProtocol = IWebSocketServer::ENetworkProtocol::IPv4;
		if (EnableIPv6Str == TEXT("true"))
		{
			PlayerProtocol = IWebSocketServer::ENetworkProtocol::IPv4 | IWebSocketServer::ENetworkProtocol::IPv6;
		}

		/*
		 * --------------- Streamers websocket server ---------------
		 */
		StreamersWS = MakeUnique<FWebSocketServerWrapper>();
		// Do not need to enable IPv6 on StreamerWS as it is always local
		if (AllowedOrigins.Num() > 0)
		{
			FWebSocketFilterConnectionCallback FilterCallback;
			FilterCallback.BindRaw(this, &FSignallingServer::FilterConnection);
			StreamersWS->SetFilterConnectionCallback(MoveTemp(FilterCallback));
		}
		StreamersWS->OnMessage.AddRaw(this, &FSignallingServer::OnStreamerMessage);
		StreamersWS->OnOpenConnection.AddRaw(this, &FSignallingServer::OnStreamerConnected);
		StreamersWS->OnClosedConnection.AddRaw(this, &FSignallingServer::OnStreamerDisconnected);
		bool bLaunchedStreamerServer = StreamersWS->Launch(StreamerURL.Port);

		if (!bLaunchedStreamerServer)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Failed to launch websocket server for streamers on port=%d", StreamerURL.Port);
			return false;
		}

		FString						 ServeHttpsString = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--ServeHttps="), TEXT("false"));
		bool						 bServeHttps = ServeHttpsString == TEXT("true");
		FWebSocketServerCertificates Certificates;
		if (bServeHttps)
		{
			FString CertificatePath = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--CertificatePath="), TEXT(""));
			if (!CertificatePath.IsEmpty())
			{
				Certificates.SetCertificateFilePath(CertificatePath);
			}

			FString PrivateKeyPath = Utils::QueryOrSetProcessArgs(InLaunchArgs, TEXT("--PrivateKeyPath="), TEXT(""));
			if (!PrivateKeyPath.IsEmpty())
			{
				Certificates.SetPrivateKeyFilePath(PrivateKeyPath);
			}
		}

		/*
		 * --------------- Players websocket server ---------------
		 */
		PlayersWS = MakeUnique<FWebSocketServerWrapper>();
		PlayersWS->SetNetworkProtocol(PlayerProtocol);
		if (AllowedOrigins.Num() > 0)
		{
			FWebSocketFilterConnectionCallback FilterCallback;
			FilterCallback.BindRaw(this, &FSignallingServer::FilterConnection);
			PlayersWS->SetFilterConnectionCallback(MoveTemp(FilterCallback));
		}
		PlayersWS->EnableWebServer(GenerateDirectoriesToServe(), bServeHttps, Certificates);
		PlayersWS->OnMessage.AddRaw(this, &FSignallingServer::OnPlayerMessage);
		PlayersWS->OnOpenConnection.AddRaw(this, &FSignallingServer::OnPlayerConnected);
		PlayersWS->OnClosedConnection.AddRaw(this, &FSignallingServer::OnPlayerDisconnected);
		bool bLaunchedPlayerServer = PlayersWS->Launch(PlayersURL.Port);

		if (!bLaunchedPlayerServer)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Failed to launch websocket server for players on port=%d", PlayersURL.Port);
			return false;
		}

		/*
		 * --------------- Websocket probe ---------------
		 */

		if (bPollUntilReady)
		{
			TArray<FString> Protocols;
			Protocols.Add(FString(TEXT("binary")));
			Probe = MakeUnique<FWebSocketProbe>(StreamerURL, Protocols);
		}

		return true;
	}

	FString FSignallingServer::GetPathOnDisk()
	{
		return FString();
	}

	TArray<FWebSocketHttpMount> FSignallingServer::GenerateDirectoriesToServe() const
	{
		FString ServersDir;
		bool	bServersDirExists = Utils::GetWebServersDir(ServersDir);
		if (bServersDirExists)
		{
			ServersDir = ServersDir / TEXT("SignallingWebServer");
			bServersDirExists = FPaths::DirectoryExists(ServersDir);
		}

		TArray<FWebSocketHttpMount> MountsArr;

#if WITH_EDITOR
		// If server directory doesn't exist we will serve a known directory that gives the user a message
		// telling them to run the `get_ps_servers` script.
		if (!bServersDirExists)
		{
			FString OutResourcesDir;
			bool	bResourcesDirExists = Utils::GetResourcesDir(OutResourcesDir);
			FString NotFoundDir = OutResourcesDir / TEXT("NotFound");

			if (bResourcesDirExists && FPaths::DirectoryExists(NotFoundDir))
			{
				FWebSocketHttpMount Mount;
				Mount.SetPathOnDisk(NotFoundDir);
				Mount.SetWebPath(FString(TEXT("/")));
				Mount.SetDefaultFile(FString(TEXT("not_found.html")));
				MountsArr.Add(Mount);
				return MountsArr;
			}
		}
#endif // WITH_EDITOR

		// Add /Public
		FWebSocketHttpMount PublicMount;
		PublicMount.SetPathOnDisk(ServersDir / TEXT("www"));
		PublicMount.SetWebPath(FString(TEXT("/")));
		PublicMount.SetDefaultFile(FString(TEXT("player.html")));
		MountsArr.Add(PublicMount);

		// Todo (Luke.Bermingham): Expose way for user to specify what directories to serve.

		return MountsArr;
	}

	TSharedRef<FJsonObject> FSignallingServer::CreateConfigJSON() const
	{
		// Todo (Luke): Parse `iceServers` from the process args `--peerConnectionOptions`
		TArray<TSharedPtr<FJsonValue>> IceServersArr;

		TSharedPtr<FJsonObject> PeerConnectionOptionsJSON = MakeShared<FJsonObject>();
		PeerConnectionOptionsJSON->SetArrayField(FString(TEXT("iceServers")), IceServersArr);

		TSharedRef<FJsonObject> ConfigJSON = MakeShared<FJsonObject>();
		ConfigJSON->SetStringField(FString(TEXT("type")), FString(TEXT("config")));
		ConfigJSON->SetObjectField(FString(TEXT("peerConnectionOptions")), PeerConnectionOptionsJSON);
		return ConfigJSON;
	}

	TSharedPtr<FJsonObject> FSignallingServer::ParseMessage(const FString& InMessage, FString& OutMessageType) const
	{
		TSharedPtr<FJsonObject> JSONObj = Utils::ToJSON(InMessage);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Failed to parse message: %ls", *InMessage);
			return nullptr;
		}

		if (!JSONObj->TryGetStringField(TEXT("type"), OutMessageType))
		{
			UE_LOGF(LogPixelStreaming2Servers, Warning, "Incoming message did not contain a 'type' field: %ls", *InMessage);
			return nullptr;
		}

		return JSONObj;
	}

	void FSignallingServer::SubscribePlayer(uint16 PlayerConnectionId, const FString& StreamerName)
	{
		UE_LOGF(LogPixelStreaming2Servers, Log, "Subscribing player %d to streamer %ls", PlayerConnectionId, *StreamerName);

		uint16 StreamerConnectionId;
		if (!StreamersWS->GetNamedConnection(StreamerName, StreamerConnectionId))
		{
			UE_LOGF(LogPixelStreaming2Servers, Log, "Streamer name %ls does not exist", *StreamerName);
			return;
		}

		if (!StreamersWS->GetConnections().Contains(StreamerConnectionId))
		{
			UE_LOGF(LogPixelStreaming2Servers, Log, "Streamer %d does not exist", StreamerConnectionId);
			return;
		}

		if (PlayerSubscriptions.Contains(PlayerConnectionId))
		{
			// unsubscribe first
			UnsubscribePlayer(PlayerConnectionId);
		}

		// We don't want to make the connections shared to prevent someone accidentally holding on to it. So we use it raw here
		FWebSocketConnection* PlayerWS = (*PlayersWS->GetConnections().Find(PlayerConnectionId)).Get();
		bool				  bUESendsOffer = !PlayerWS->GetUrlArgs().Contains(TEXT("OfferToReceive=true"));

		// Send "playerConnected" message to streamer which kicks off making a new RTC connection
		TSharedRef<FJsonObject> OnPlayerConnectedJSON = MakeShared<FJsonObject>();
		OnPlayerConnectedJSON->SetStringField(TEXT("type"), "playerConnected");
		OnPlayerConnectedJSON->SetStringField(TEXT("playerId"), FString::FromInt(PlayerConnectionId));
		OnPlayerConnectedJSON->SetBoolField(TEXT("dataChannel"), true);
		OnPlayerConnectedJSON->SetBoolField(TEXT("sfu"), false);
		OnPlayerConnectedJSON->SetBoolField(TEXT("sendOffer"), bUESendsOffer);
		SendStreamerMessage(StreamerConnectionId, OnPlayerConnectedJSON);

		PlayerSubscriptions.Add(PlayerConnectionId, StreamerConnectionId);
	}

	void FSignallingServer::UnsubscribePlayer(uint16 PlayerConnectionId)
	{
		if (PlayerSubscriptions.Contains(PlayerConnectionId))
		{
			const uint16 StreamerConnectionId = PlayerSubscriptions[PlayerConnectionId];
			UE_LOGF(LogPixelStreaming2Servers, Log, "Unsubscribing player %d from streamer %d", PlayerConnectionId, StreamerConnectionId);

			// Send "playerDisconnected" message to streamer
			TSharedRef<FJsonObject> OnPlayerDisconnectedJSON = MakeShared<FJsonObject>();
			OnPlayerDisconnectedJSON->SetStringField(TEXT("type"), "playerDisconnected");
			OnPlayerDisconnectedJSON->SetStringField(TEXT("playerId"), FString::FromInt(PlayerConnectionId));
			SendStreamerMessage(StreamerConnectionId, OnPlayerDisconnectedJSON);

			PlayerSubscriptions.Remove(PlayerConnectionId);
		}
	}

	void FSignallingServer::SendPlayerMessage(uint16 PlayerId, TSharedPtr<FJsonObject> JSONObj, bool bLog)
	{
		const FString MessageString = Utils::ToString(JSONObj);
		if (bLog)
		{
			UE_LOGF(LogPixelStreaming2Servers, Log, "Sending to player id=%d: %ls", PlayerId, *MessageString);
		}
		PlayersWS->Send(PlayerId, MessageString);
	}

	void FSignallingServer::SendStreamerMessage(uint16 StreamerId, TSharedPtr<FJsonObject> JSONObj, bool bLog)
	{
		const FString MessageString = Utils::ToString(JSONObj);
		if (bLog)
		{
			UE_LOGF(LogPixelStreaming2Servers, Log, "Sending to streamer id=%d: %ls", StreamerId, *MessageString);
		}
		StreamersWS->Send(StreamerId, MessageString);
	}

	void FSignallingServer::OnStreamerConnected(uint16 ConnectionId)
	{
		UE_LOGF(LogPixelStreaming2Servers, Log, "Streamer websocket connected, id=%d", ConnectionId);

		// Send a config message to the streamer passing ICE servers to be used.
		TSharedPtr<FJsonObject> JSONObj = CreateConfigJSON();
		SendStreamerMessage(ConnectionId, JSONObj);

		// request the streamer id
		TSharedRef<FJsonObject> idJSON = MakeShared<FJsonObject>();
		idJSON->SetStringField(TEXT("type"), "identify");
		SendStreamerMessage(ConnectionId, idJSON);

		StreamersWS->NameConnection(ConnectionId, LEGACY_NAME);
	}

	void FSignallingServer::OnStreamerDisconnected(uint16 ConnectionId)
	{
		UE_LOGF(LogPixelStreaming2Servers, Log, "Streamer websocket disconnected, id=%d", ConnectionId);

		for (auto& ConnectionPair : PlayerSubscriptions)
		{
			const uint16 StreamerConnectionId = ConnectionPair.Key;
			const uint16 PlayerConnectionId = ConnectionPair.Value;
			if (StreamerConnectionId == ConnectionId)
			{
				UnsubscribePlayer(PlayerConnectionId);
			}
		}
	}

	void FSignallingServer::OnStreamerMessage(uint16 ConnectionId, TArrayView<uint8> Message)
	{
		const FString Msg = Utils::ToString(Message);

		FString					MsgType;
		TSharedPtr<FJsonObject> JSONObj = ParseMessage(Msg, MsgType);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Failed to parse incoming streamer message.");
			return;
		}

		if (auto* Handler = StreamerMessageHandlers.Find(MsgType))
		{
			UE_LOGFMT(LogPixelStreaming2Servers, Log, "From Streamer id={0}: {1}", ConnectionId, Msg);
			Handler->Execute(ConnectionId, JSONObj);
		}
		else
		{
			// All other message types require a `playerId` field to be valid.
			uint16 PlayerConnectionId;
			if (!JSONObj->TryGetNumberField(TEXT("playerId"), PlayerConnectionId))
			{
				UE_LOGF(LogPixelStreaming2Servers, Warning, "Message did not contain a field called 'playerId' - message=%ls", *Msg);
				return;
			}

			// As message are going to the player they don't actually need the playerId field, the field exists only so we know who to send it to.
			JSONObj->RemoveField(TEXT("playerId"));
			UE_LOGFMT(LogPixelStreaming2Servers, Log, "Forwarding from Streamer [{0}] to Player [{1}]: {2}", ConnectionId, PlayerConnectionId, Msg);
			SendPlayerMessage(PlayerConnectionId, JSONObj, false /* don't log outgoing message as we just did it here */);
		}
	}

	void FSignallingServer::OnPlayerConnected(uint16 ConnectionId)
	{
		// Send config to newly connected player, which kicks off making a new RTC connection
		TSharedPtr<FJsonObject> ConfigJSON = CreateConfigJSON();
		SendPlayerMessage(ConnectionId, ConfigJSON);
	}

	void FSignallingServer::OnPlayerDisconnected(uint16 ConnectionId)
	{
		UnsubscribePlayer(ConnectionId);
	}

	void FSignallingServer::OnPlayerMessage(uint16 ConnectionId, TArrayView<uint8> Message)
	{
		const FString Msg = Utils::ToString(Message);

		FString					MsgType;
		TSharedPtr<FJsonObject> JSONObj = ParseMessage(Msg, MsgType);
		if (!JSONObj)
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Failed to parse incoming player message.");
			return;
		}

		if (auto* Handler = PlayerMessageHandlers.Find(MsgType))
		{
			UE_LOGFMT(LogPixelStreaming2Servers, Log, "From Player id={0}: {1}", ConnectionId, Msg);
			Handler->Execute(ConnectionId, JSONObj);
		}
		else
		{
			if (!PlayerSubscriptions.Contains(ConnectionId))
			{
				TArray<FString> StreamerConnections = StreamersWS->GetConnectionNames();
				if (StreamerConnections.Num() == 0)
				{
					UE_LOGFMT(LogPixelStreaming2Servers, Error, "Player {0} sent a message, but no streamers were connected", ConnectionId);
					return;
				}

				UE_LOGFMT(LogPixelStreaming2Servers, Log, "Player {0} attempted to send an outgoing message without having subscribed first. Defaulting to {1}", ConnectionId, StreamerConnections[0]);
				SubscribePlayer(ConnectionId, StreamerConnections[0]);
			}

			// Add player id to any messages going to streamer so streamer knows who sent it
			JSONObj->SetStringField(TEXT("playerId"), FString::FromInt(ConnectionId));
			UE_LOGFMT(LogPixelStreaming2Servers, Log, "Forwarding from Player [{0}] to Streamer [{1}]: {2}", ConnectionId, PlayerSubscriptions[ConnectionId], Msg);
			SendStreamerMessage(PlayerSubscriptions[ConnectionId], JSONObj, false /* don't log outgoing message as we just did it here */);
		}
	}

	void FSignallingServer::OnStreamerIdMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		FString StreamerName;
		if (JSONObj->TryGetStringField(TEXT("id"), StreamerName))
		{
			StreamersWS->NameConnection(ConnectionId, StreamerName);
			StreamersWS->RemoveName(LEGACY_NAME);
		}
	}

	void FSignallingServer::OnStreamerPingMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		const double			UnixTime = FDateTime::UtcNow().ToUnixTimestamp();
		TSharedRef<FJsonObject> PongJSON = MakeShared<FJsonObject>();
		PongJSON->SetStringField(TEXT("type"), "pong");
		PongJSON->SetNumberField(TEXT("time"), UnixTime);
		SendStreamerMessage(ConnectionId, PongJSON);
	}

	void FSignallingServer::OnStreamerDisconnectMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		uint16 PlayerConnectionId;
		if (!JSONObj->TryGetNumberField(TEXT("playerId"), PlayerConnectionId))
		{
			UE_LOGF(LogPixelStreaming2Servers, Warning, "Disconnect message did not contain a field called 'playerId'");
			return;
		}

		// TODO this might get called anyway from OnClosedConnection
		UnsubscribePlayer(PlayerConnectionId);
		PlayersWS->Close(PlayerConnectionId);
	}

	void FSignallingServer::OnPlayerPingMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		const double			UnixTime = FDateTime::UtcNow().ToUnixTimestamp();
		TSharedRef<FJsonObject> PongJSON = MakeShared<FJsonObject>();
		PongJSON->SetStringField(TEXT("type"), "pong");
		PongJSON->SetNumberField(TEXT("time"), UnixTime);
		SendPlayerMessage(ConnectionId, PongJSON);
	}

	void FSignallingServer::OnPlayerListStreamersMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		TSharedRef<FJsonObject>		   listJSON = MakeShared<FJsonObject>();
		const TArray<FString>		   Names = StreamersWS->GetConnectionNames();
		TArray<TSharedPtr<FJsonValue>> JsonNames;
		for (const FString& Name : Names)
		{
			JsonNames.Add(MakeShared<FJsonValueString>(Name));
		}
		listJSON->SetStringField(TEXT("type"), TEXT("streamerList"));
		listJSON->SetArrayField(TEXT("ids"), JsonNames);
		SendPlayerMessage(ConnectionId, listJSON);
	}

	void FSignallingServer::OnPlayerSubscribeMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		FString StreamerName;
		if (!JSONObj->TryGetStringField(TEXT("streamerId"), StreamerName))
		{
			UE_LOGF(LogPixelStreaming2Servers, Error, "Player %d subscribe message missing streamerId.", ConnectionId);
		}
		else
		{
			SubscribePlayer(ConnectionId, StreamerName);
		}
	}

	void FSignallingServer::OnPlayerUnsubscribeMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		UnsubscribePlayer(ConnectionId);
	}

	void FSignallingServer::OnPlayerStatsMessage(uint16 ConnectionId, TSharedPtr<FJsonObject> JSONObj)
	{
		UE_LOGF(LogPixelStreaming2Servers, Log, "Player %d stats = \n %ls", ConnectionId, *Utils::ToString(JSONObj.ToSharedRef()));
	}

	void FSignallingServer::GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived)
	{
		if (StreamersWS)
		{
			int NConnections = StreamersWS->Count();
			// If the probe is currently connected, it is not a streamer, so don't count it.
			if (Probe && Probe->IsConnected()) 
			{
				NConnections = NConnections - 1;
			}
			OnNumStreamersReceived(NConnections);
		}
		else
		{
			// Streamers websocket server went out of scope, so we can assume no streamers are connected.
			OnNumStreamersReceived(0);
		}
	}

} // namespace UE::PixelStreaming2Servers
