// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketUtils.h"

#include "Logging.h"
#include "SocketSubsystem.h"

namespace UE::PixelStreaming2
{
	static FThreadSafeCounter NextGeneratedPort = 0;

	int	GetNextAvailablePort(TOptional<int> StartingPort)
	{
		int32					  CandidateNextPort = StartingPort.Get((4000 + NextGeneratedPort.Increment()) % 65535);
		int32					  NumRemainingPorts = 65535 - CandidateNextPort;
		ISocketSubsystem*		  SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		TSharedRef<FInternetAddr> LocalHostAddr = SocketSubsystem->CreateInternetAddr(FNetworkProtocolTypes::IPv4);
		LocalHostAddr->SetPort(CandidateNextPort);

		// The following logic is done inside a scope to ensure that the FUniqueSocket is destroyed and unbound before we return the port number
		{
			FUniqueSocket Socket = SocketSubsystem->CreateUniqueSocket(NAME_Stream, TEXT("DummySocket"), FNetworkProtocolTypes::IPv4);
			int32		  BoundPort = SocketSubsystem->BindNextPort(Socket.Get(), LocalHostAddr.Get(), NumRemainingPorts, 1);
			if (BoundPort == 0)
			{
				UE_LOGFMT(LogPixelStreaming2Servers, Warning, "Failed to find an available port!");
				return -1;
			}
			CandidateNextPort = BoundPort;
		}
		return CandidateNextPort;
	}
} // namespace UE::PixelStreaming2
