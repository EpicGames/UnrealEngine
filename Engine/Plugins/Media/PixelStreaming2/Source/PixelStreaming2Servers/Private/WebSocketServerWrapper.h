// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWebSocketServer.h"
#include "INetworkingWebSocket.h"
#include "Tickable.h"
#include "HAL/ThreadSafeBool.h"

namespace UE::PixelStreaming2Servers
{

	struct FWebSocketConnection
	{
	public:
		FWebSocketConnection(INetworkingWebSocket* InSocketConnection);
		~FWebSocketConnection();
		uint16			GetId() const;
		TArray<FString> GetUrlArgs() const;
		bool			Send(FString Message) const;
		void			SetCallbacks();

	public:
		FWebSocketPacketReceivedCallBack OnPacketReceivedCallback;
		FWebSocketInfoCallBack			 OnClosedCallback;

	private:
		static FThreadSafeCounter IdGenerator;
		uint16					  Id;
		TArray<FString>			  UrlArgs;
		INetworkingWebSocket*	  SocketConnection;
	};

	class FWebSocketServerWrapper : public FTickableGameObject
	{
	public:
		FWebSocketServerWrapper();
		virtual ~FWebSocketServerWrapper();
		void EnableWebServer(TArray<FWebSocketHttpMount> InDirectoriesToServe, bool bServeHttps, const FWebSocketServerCertificates& Certificates);
		void SetFilterConnectionCallback(FWebSocketFilterConnectionCallback InFilterConnectionCallback);
		void SetNetworkProtocol(IWebSocketServer::ENetworkProtocol Protocol) { NetworkProtocol = Protocol; }
		bool Launch(uint16 Port);
		void Stop();
		bool IsLaunched() const;
		bool HasConnections() const;
		bool Close(uint16 ConnectionId);
		/**
		 * Send message to a particular websocket connection.
		 * Note: This message is sent as binary using the websocket protocol because of the WS implementation that is used.
		 * @return True if the message able to be sent.
		 */
		bool											Send(uint16 ConnectionId, FString Message) const;
		bool											Send(const FString& ConnectionName, FString Message) const;
		bool											GetFirstConnection(uint16& OutConnectionId) const;
		TMap<uint16, TUniquePtr<FWebSocketConnection>>& GetConnections() { return Connections; }

		void			NameConnection(uint16 ConnectionId, const FString& Name);
		void			RemoveName(const FString& Name);
		bool			GetNamedConnection(const FString& Name, uint16& OutConnectionId) const;
		TArray<FString> GetConnectionNames() const;

		/* Begin FTickableGameObject */
		virtual bool IsTickableWhenPaused() const { return true; }
		virtual bool IsTickableInEditor() const { return true; }
		virtual void Tick(float DeltaTime) override;
		virtual bool IsAllowedToTick() const { return true; }
		TStatId		 GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FWebSocketServerWrapper, STATGROUP_Tickables); }
		/* End FTickableGameObject */

		int32 Count() const { return Connections.Num(); }

	public:
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewWebSocketConnection, uint16 /*Connection Id*/);
		FOnNewWebSocketConnection OnOpenConnection;

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnClosedWebSocketConnection, uint16 /*Connection Id*/);
		FOnClosedWebSocketConnection OnClosedConnection;

		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMessageWebSocketConnection, uint16 /*Connection Id*/, TArrayView<uint8> /*Message*/);
		FOnMessageWebSocketConnection OnMessage;

	protected:
		virtual void OnConnectionOpened(INetworkingWebSocket* Socket);
		virtual void OnPacketReceived(void* Data, int32 Size, uint16 ConnectionId);
		virtual void OnConnectionClosed(uint16 ConnectionId);

	private:
		FThreadSafeBool								   bLaunched;
		TUniquePtr<IWebSocketServer>				   WSServer;
		FWebSocketClientConnectedCallBack			   OnClientConnectedCallback;
		TMap<uint16, TUniquePtr<FWebSocketConnection>> Connections;
		TMap<FString, uint16>						   NamedConnections;
		bool										   bEnableWebServer = false;
		IWebSocketServer::ENetworkProtocol			   NetworkProtocol = IWebSocketServer::ENetworkProtocol::IPv4;
		bool										   bServeHttps = false;
		FWebSocketServerCertificates				   Certificates;
		TArray<FWebSocketHttpMount>					   DirectoriesToServe;
		FWebSocketFilterConnectionCallback			   FilterConnectionCallback;
	};

} // namespace UE::PixelStreaming2Servers
