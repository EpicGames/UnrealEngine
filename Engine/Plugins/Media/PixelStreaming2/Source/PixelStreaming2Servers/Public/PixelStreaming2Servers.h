// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Containers/Map.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/MonitoredProcess.h"

/*
 * Utility namespace for launching Pixel Streaming servers signaling or selective forwarding.
 * This utility can launch embedded servers from C++ classes in this module,
 * from known server binaries and from the Github releases of the servers.
 */
namespace UE::PixelStreaming2Servers
{
	/**
	 * Configuration to control behaviour when launching any of the Pixel Streaming servers.
	 **/
	struct FLaunchArgs
	{
		// Arguments passed to the actual server when its process is started.
		FString ProcessArgs = TEXT("");

		// If true poll until ready
		bool bPollUntilReady = false;

		// Reconnection timeout in seconds
		float ReconnectionTimeoutSeconds = 30.0f;

		// Reconnect interval in seconds.
		float ReconnectionIntervalSeconds = 2.0f;
	};

	/**
	 * Endpoints for the various Pixel Streaming servers.
	 **/
	enum class PIXELSTREAMING2SERVERS_API EEndpoint
	{
		// The websocket signalling url between the server and the UE streamer - e.g. ws://localhost:8888
		Signalling_Streamer,

		// The websocket signalling url between the server and the players (aka. web browsers) - e.g. ws://localhost:80
		Signalling_Players,

		// The websocket signalling url between the server and the SFU server - e.g. ws://localhost:8889
		Signalling_SFU,

		// The http url for the webserver hosted within the signalling server - e.g. http://localhost
		Signalling_Webserver
	};

	// ---------------------------------------------------------------------------------------------
	typedef TMap<EEndpoint, FURL> FEndpoints;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReady, const FEndpoints& /* Endpoint urls */);
	// ---------------------------------------------------------------------------------------------

	/**
	 * Interface for all Pixel Streaming servers.
	 **/
	class IServer
	{
	public:
		virtual ~IServer() = default;

		/**
		 * Immediately stops the server.
		 */
		virtual void Stop() = 0;

		/**
		 * @return	The absolute path to the root directory that the server was launched from.
		 **/
		virtual FString GetPathOnDisk() = 0;

		/**
		 * @return	True if the server has been launched. Note: Launched does not necessarily mean it is connectible yet. Bind to OnReady for that.
		 **/
		virtual bool HasLaunched() = 0;

		/**
		 * Launch the server in a child process using the supplied launch arguments.
		 * @param LaunchArgs	The launch arguments to control how the server is launched, including what args to pass to the child process.
		 * @return True if the server was able to start launching, this can fail when launching child process servers where files must exist on disk.
		 **/
		virtual bool Launch(FLaunchArgs& InLaunchArgs) = 0;

		/**
		 * @return	True if the server has been connected to and is ready for new connections.
		 **/
		virtual bool IsReady() = 0;

		/**
		 * @return	True if the server has timed out while trying to establish a connection.
		 **/
		virtual bool IsTimedOut() = 0;

		/**
		 * Asynchronously queries the numbers of streamers connected to the signalling server.
		 * @param OnNumStreamersReceived	Callback fired when the query for the number of streamers returns.
		 */
		virtual void GetNumStreamers(TFunction<void(uint16)> OnNumStreamersReceived) = 0;

	public:
		// Delegate fired when the server is ready for connections, first parameter is a map of all supported endpoints and their urls.
		FOnReady OnReady;

		DECLARE_MULTICAST_DELEGATE(FOnFailedToReady);
		/* Can fire when the server is unable to be contacted or connecting to it timed out. */
		FOnFailedToReady OnFailedToReady;
	};

	/* -------------- Static utility methods for working with Pixel Streaming servers. ----------------- */
	/**
	 * Creates a native C++ signalling server (similar to cirrus.js) with no dependencies launched inside the Unreal Engine process.
	 * Note: Calling this method does not launch the server. You should call Launch() yourself
	 * once you have bound to appropriate delegates such as OnReady.
	 * @return	The embedded signalling server.
	 **/
	PIXELSTREAMING2SERVERS_API TSharedPtr<IServer> MakeSignallingServer();

	/**
	 * Download the Pixel Streaming servers using the `get_ps_servers` scripts.
	 * @param bSkipIfPresent Servers will not be downloaded if they are already present.
	 * @return The child process that is used to download the servers.
	 **/
	PIXELSTREAMING2SERVERS_API TSharedPtr<FMonitoredProcess> DownloadPixelStreaming2Servers(bool bSkipIfPresent);

} // namespace UE::PixelStreaming2Servers
