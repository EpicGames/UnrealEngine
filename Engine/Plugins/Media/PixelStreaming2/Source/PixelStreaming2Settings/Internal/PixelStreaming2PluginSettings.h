// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "PixelStreaming2SettingsEnums.h"

#include "AVConfig.h"
#include "Video/VideoConfig.h"
#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsH264.h"

#include "epic_rtc/core/connection_config.h"

#include "PixelStreaming2PluginSettings.generated.h"

#define UE_API PIXELSTREAMING2SETTINGS_API

namespace UE::PixelStreaming2
{
	// This variable exists to back CVarEncoderCodec and should only be used by the FEpicRtcVideo(Encoder/Decoder)Initializer. When doing software fallback,
	// the encoder initializer will set post an async task to set the CVarEncoderCodec. However, we cannot be certain that this task will execute before
	// the decoder initializer reads the cvar again. By having both use this variable, we can be certain that there will be no race conditions between them. 
	extern PIXELSTREAMING2SETTINGS_API FString GSelectedCodec;

	template <typename TEnumType>
	TEnumType GetEnumFromCVar(const TAutoConsoleVariable<FString>& CVar)
	{
		uint64 OutEnum = StaticEnum<TEnumType>()->GetIndexByNameString(CVar.GetValueOnAnyThread());
		checkf(OutEnum != INDEX_NONE, TEXT("CVar was not containing valid enum string"));
		return static_cast<TEnumType>(StaticEnum<TEnumType>()->GetValueByIndex(OutEnum));
	}

	template <typename TEnumType>
	TEnumType GetEnumFromString(const FString& EnumString)
	{
		uint64 OutEnum = StaticEnum<TEnumType>()->GetIndexByNameString(EnumString);
		checkf(OutEnum != INDEX_NONE, TEXT("EnumString was not containing valid enum string"));
		return static_cast<TEnumType>(StaticEnum<TEnumType>()->GetValueByIndex(OutEnum));
	}

	template <typename TEnumType>
	FString GetCVarStringFromEnum(TEnumType Value)
	{
		return StaticEnum<TEnumType>()->GetNameStringByValue((int32)Value);
	}
} // namespace UE::PixelStreaming2

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EPortAllocatorFlags : uint32
{
	None = static_cast<uint32>(EpicRtcPortAllocatorOptions::None) UMETA(Hidden),
	DisableUdp = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableUdp),
	DisableStun = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableStun),
	DisableRelay = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableRelay),
	DisableTcp = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableTcp),
	EnableIPV6 = static_cast<uint32>(EpicRtcPortAllocatorOptions::EnableIPV6),
	EnableSharedSocket = static_cast<uint32>(EpicRtcPortAllocatorOptions::EnableSharedSocket),
	EnableStunRetransmitAttribute = static_cast<uint32>(EpicRtcPortAllocatorOptions::EnableStunRetransmitAttribute),
	DisableAdapterEnumeration = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableAdapterEnumeration),
	DisableDefaultLocalCandidate = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableDefaultLocalCandidate),
	DisableUdpRelay = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableUdpRelay),
	DisableCostlyNetworks = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableCostlyNetworks),
	EnableIPV6OnWifi = static_cast<uint32>(EpicRtcPortAllocatorOptions::EnableIPV6OnWifi),
	EnableAnyAddressPort = static_cast<uint32>(EpicRtcPortAllocatorOptions::EnableAnyAddressPort),
	DisableLinkLocalNetworks = static_cast<uint32>(EpicRtcPortAllocatorOptions::DisableLinkLocalNetworks)
};
ENUM_CLASS_FLAGS(EPortAllocatorFlags);

/** Pixel Streaming can limit who can send input (keyboard, mouse, etc). */
UENUM()
enum class EInputControllerMode : uint8
{
	Any, // Any peer can control input.
	Host // Only the "host" peer can control input.
};

// Config loaded/saved to an .ini file.
// It is also exposed through the plugin settings page in editor.
UCLASS(MinimalAPI, config = Game, defaultconfig, meta = (DisplayName = "PixelStreaming2"))
class UPixelStreaming2PluginSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UE_API virtual ~UPixelStreaming2PluginSettings();

public:
	// clang-format off
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		ConfigRestartRequired = true,
		DisplayName = "Enable Pixel Streaming 2 Toolbar",
		ToolTip = "Adds the Pixel Streaming 2 toolbar to the main viewport toolbar."
		))
	bool EnablePixelStreamingToolbar = true;

	static UE_API TAutoConsoleVariable<bool> CVarLogStats;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Log Pixel Streaming Stats",
		ToolTip = "Whether to show PixelStreaming stats in the log (default: false)."
		))
	bool LogStats = false;

	static UE_API TAutoConsoleVariable<FString> CVarEpicRtcLogFilter;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Log messages from EpicRtc to filter",
		ToolTip = "Double forward slash (\"//\") separated list of regex patterns to filter from the EpicRtc logs (default: \"\")."
		))
	FString EpicRtcLogFilter = "";

	static UE_API TAutoConsoleVariable<bool> CVarDisableLatencyTester;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Disable Latency Tester",
		ToolTip = "If true disables latency tester being triggerable."
		))
	bool DisableLatencyTester = false;

	static UE_API TAutoConsoleVariable<FString> CVarInputController;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Input Controller Mode",
		ToolTip = "If true disables latency tester being triggerable."
		))
	EInputControllerMode InputController = EInputControllerMode::Any;

	static UE_API TAutoConsoleVariable<bool> CVarDecoupleFramerate;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Decouple Frame Rate",
		ToolTip = "Whether we should only stream as fast as we render or at some fixed interval. Coupled means only stream what we render."
		))
	bool DecoupleFramerate = false;

	static UE_API TAutoConsoleVariable<float> CVarDecoupleWaitFactor;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Decouple Wait Factor",
		ToolTip = "Frame rate factor to wait for a captured frame when streaming in decoupled mode. Higher factor waits longer but may also result in higher latency."
		))
	float DecoupleWaitFactor = 1.0f;

	static UE_API TAutoConsoleVariable<float> CVarSignalingReconnectInterval;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Signaling Reconnect Interval",
		ToolTip = "Changes the number of seconds between attempted reconnects to the signaling server. This is useful for reducing the log spam produced from attempted reconnects. A value <= 0 results in no reconnect. Default: 2.0s"
		))
	float SignalingReconnectInterval = 2.0f;

	static UE_API TAutoConsoleVariable<float> CVarSignalingMaxReconnectAttempts;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Signaling Max Reconnect Attempts",
		ToolTip = "Changes the number of attempts that will be made to reconnect to the signalling server. This is useful for triggering application shutdowns if this value is exceeded. A value of < 0 results in unlimited attempts. Default: -1"
		))
	int32 SignalingMaxReconnectAttempts = -1;

	static UE_API TAutoConsoleVariable<float> CVarSignalingKeepAliveInterval;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Signaling Reconnect Interval",
		ToolTip = "Frame rate factor to wait for a captured frame when streaming in decoupled mode. Higher factor waits longer but may also result in higher latency."
		))
	float SignalingKeepAliveInterval = 30.0f;

	static UE_API TAutoConsoleVariable<bool> CVarUseMediaCapture;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Use Media Capture",
		ToolTip = "Use Media Capture from MediaIOFramework to capture frames rather than Pixel Streamings internal backbuffer sources."
		))
	bool UseMediaCapture = true;

	static UE_API TAutoConsoleVariable<bool> CVarInitializeDefaultStreamer;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Initialize Default Streamer",
		ToolTip = "If the default Pixel Streaming 2 streamer should be initialized. You may wish to set this to false if you only plan on receiving streams. Default: true"
		))
	bool InitializeDefaultStreamer = true;

	static UE_API TAutoConsoleVariable<FString> CVarDefaultStreamerID;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Default Streamer ID",
		ToolTip = "Default Streamer ID to be used when not specified elsewhere.",
		EditCondition="InitializeDefaultStreamer"
		))
	FString DefaultStreamerID = TEXT("DefaultStreamer");

	static UE_API TAutoConsoleVariable<FString> CVarDefaultStreamerType;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Default Streamer Type",
		ToolTip = "Default Streamer Type to be used when not specified elsewhere. This value should match a type registered by an IPixelStreaming2StreamerFactory",
		GetOptions = "GetDefaultStreamerTypeOptions",
		EditCondition="InitializeDefaultStreamer"
		))
	FString DefaultStreamerType = TEXT("DefaultRtc");

	static UE_API TAutoConsoleVariable<bool> CVarAutoStartStream;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Automatically Start Streaming",
		ToolTip = "Configure the PixelStreaming2 plugin to automatically start streaming once loaded (if not in editor). You may wish to set this value to false and manually call StartStreaming at a later point from your c++ code. Default: true",
		EditCondition="InitializeDefaultStreamer"
		))
	bool AutoStartStream = true;

	static UE_API TAutoConsoleVariable<FString> CVarConnectionURL;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Default Connection URL",
		ToolTip = "Default URL to connect to. This can be a URL to a signalling server or some other endpoint with the format (protocol)://(host):(port)"
		))
	FString ConnectionURL;

	static UE_API FAutoConsoleVariableDeprecated CVarSignallingURL;

	static UE_API TAutoConsoleVariable<bool> CVarCaptureUseFence;
	UPROPERTY(config, EditAnywhere, Category = "PixelStreaming", meta = (
		DisplayName = "Capture Using Fence",
		ToolTip = "Whether the texture copy we do during image capture should use a fence or not (non-fenced is faster but less safe)."
		))
	bool CaptureUseFence = true;

	static UE_API TAutoConsoleVariable<bool> CVarDebugDumpAudio;

	// Begin Cursor Settings
	/**
	 * Pixel streaming always requires various software cursors so they will be
	 * visible in the video stream sent to the browser to allow the user to
	 * click and interact with UI elements.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cursor")
	FSoftClassPath DefaultCursorClassName;
	UPROPERTY(config, EditAnywhere, Category = "Cursor")
	FSoftClassPath TextEditBeamCursorClassName;

	/**
	 * Pixel Streaming can have a server-side cursor (where the cursor itself
	 * is shown as part of the video), or a client-side cursor (where the cursor
	 * is shown by the browser). In the latter case we need to turn the UE4
	 * cursor invisible.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Cursor")
	FSoftClassPath HiddenCursorClassName;

	// Begin Encoder Settings
	static UE_API TAutoConsoleVariable<int32> CVarEncoderTargetBitrate;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Target Bitrate",
		ToolTip = "Target bitrate (bps). Ignore the bitrate WebRTC wants (not recommended). Set to -1 to disable. Default -1.",
		ClampMin = -1))
	int32 EncoderTargetBitrate = -1;

	static UE_API TAutoConsoleVariable<int32>	 CVarEncoderMinQuality;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Encoder Minimum Quality",
		ToolTip = "0-100, Higher values result in a better minimum quality but higher average bitrates. Default 0 - i.e. no limit on a minimum Quality.",
		ClampMin = 0, ClampMax = 100))
	int32 EncoderMinQuality = 0;

	static UE_API TAutoConsoleVariable<int32>	CVarEncoderMaxQuality;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Encoder Maximum Quality",
		ToolTip = "0-100, Lower values result in lower average bitrates but reduces maximum quality. Default 100 - i.e. no limit on a maximum QP.",
		ClampMin = 0, ClampMax = 100))
	int32 EncoderMaxQuality = 100;

	// not directly connected to cvar due to string to enum conversion
	static UE_API TAutoConsoleVariable<FString> CVarEncoderQualityPreset;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Default Encoding Quality Profile",
		ToolTip = "PixelStreaming Encoder presets that affecting Quality vs Bitrate. Supported modes are `ULTRA_LOW_QUALITY`, `LOW_QUALITY`, `DEFAULT`, `HIGH_QUALITY`, `LOSSLESS`"
		))
	EAVPreset QualityPreset = EAVPreset::Default;

	// not directly connected to cvar due to string to enum conversion
	static UE_API TAutoConsoleVariable<FString> CVarEncoderLatencyMode;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Default Encoding Quality Profile",
		ToolTip = "PixelStreaming Encoder presets that affecting Quality vs Latency. Supported modes are `ULTRA_LOW_LATENCY`, `LOW_LATENCY`, `DEFAULT`"
		))
	EAVLatencyMode LatencyMode = EAVLatencyMode::UltraLowLatency;

	static UE_API TAutoConsoleVariable<int32> CVarEncoderKeyframeInterval;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Default Keyframe Interval",
		ToolTip = "How many frames before a key frame is sent. Default: -1 which disables the sending of periodic key frames. Note: NVENC reqires a reinitialization when this changes."
		))
	int32 KeyframeInterval = -1;

	static UE_API TAutoConsoleVariable<int32> CVarEncoderMaxSessions;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Default Max Number of Encoding Session",
		ToolTip = "-1 implies no limit. Maximum number of concurrent hardware encoder sessions for Pixel Streaming. Note: Geforce gpus only support 8 concurrent sessions and will rollover to software encoding when that number is exceeded."
		))
	int32 MaxSessions = -1;

	static UE_API TAutoConsoleVariable<bool> CVarEncoderEnableSimulcast;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Enable Simulcast",
		ToolTip = "Enables simulcast. When enabled, the encoder will encode at full resolution, 1/2 resolution and 1/4 resolution simultaneously. Note: Simulcast is only supported with `H264` and `VP8` and you must use the SFU from the infrastructure to fully utilise this functionality.",
		EditCondition="!WebRTCNegotiateCodecs",
		EditConditionHides
		))
	bool EnableSimulcast = false;

	static UE_API TAutoConsoleVariable<FString> CVarEncoderCodec;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Preferred Encoder Codec",
		ToolTip = "Preferred encoder codec signalled during connection establishment.",
		GetOptions = "GetVideoCodecOptions",
		EditCondition="!WebRTCNegotiateCodecs",
		EditConditionHides
		))
	FString Codec = UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::H264);

	static UE_API TAutoConsoleVariable<FString> CVarEncoderScalabilityMode;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings", meta = (
		DisplayName = "Default Scalability Mode",
		ToolTip = "Indicates number of spatial and temporal layers used, default: L1T1. For a full list of values refer to https://www.w3.org/TR/webrtc-svc/#scalabilitymodes*",
		GetOptions = "GetScalabilityModeOptions",
		EditCondition="!WebRTCNegotiateCodecs",
		EditConditionHides
		))
	FString ScalabilityMode = UE::PixelStreaming2::GetCVarStringFromEnum(EScalabilityMode::L1T1);

	static UE_API TAutoConsoleVariable<FString> CVarEncoderH264Profile;
	UPROPERTY(config, EditAnywhere, Category = "EncoderSettings|H264", meta = (
		DisplayName = "Default H264 profile",
		ToolTip = "PixelStreaming Encoder profile. Supported modes are `AUTO`, `BASELINE`, `MAIN`, `HIGH`, `HIGH444`, `PROGRESSIVE_HIGH` or `CONSTRAINED_HIGH`",
		ValidEnumValues = "Auto, Baseline, Main, High, High444, ProgressiveHigh, ConstrainedHigh"
		))
	EH264Profile H264Profile = EH264Profile::Baseline;

	static UE_API TAutoConsoleVariable<bool> CVarEncoderDebugDumpFrame;
	// End Encoder Settings

	// Begin WebRTC CVars
	static UE_API TAutoConsoleVariable<int32> CVarWebRTCFps;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default WebRtc FPS",
		ToolTip = "Framerate for WebRTC encoding. Default: 60"
		))
	int32 WebRTCFps = 60;

	static UE_API TAutoConsoleVariable<int32> CVarWebRTCStartBitrate;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default WebRtc Start Bitrate",
		ToolTip = "FStart bitrate (bps) that WebRTC will try begin the stream with. Must be between Min/Max bitrates. Default: 1000000"
		))
	int32 WebRTCStartBitrate = 1000000;

	static UE_API TAutoConsoleVariable<int32> CVarWebRTCMinBitrate;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default WebRtc Minimum Bitrate",
		ToolTip = "Min bitrate (bps) that WebRTC will not request below. Careful not to set too high otherwise WebRTC will just drop frames. Default: 100000"
		))
	int32 WebRTCMinBitrate = 100000;

	static UE_API TAutoConsoleVariable<int32> CVarWebRTCMaxBitrate;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default WebRtc Maximum Bitrate",
		ToolTip = "Max bitrate (bps) that WebRTC will not request above. Default: 40000000 aka 40 megabits/per second."
		))
	int32 WebRTCMaxBitrate = 40000000;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableReceiveAudio;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Disable receiving audio by default",
		ToolTip = "Disables receiving audio from the browser into UE."
		))
	bool WebRTCDisableReceiveAudio = false;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableReceiveVideo;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Disable receiving video by default",
		ToolTip = "Disables receiving video from the browser into UE."
		))
	bool WebRTCDisableReceiveVideo = false;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableTransmitAudio;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Disable transmiting audio by default",
		ToolTip = "Disables transmiting audio to the browser."
		))
	bool WebRTCDisableTransmitAudio = false;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableTransmitVideo;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Disable transmiting video by default",
		ToolTip = "Disables transmiting video to the browser."
		))
	bool WebRTCDisableTransmitVideo = false;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableAudioSync;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Disable Audio/Video track sychronisation",
		ToolTip = "Disables the synchronization of audio and video tracks in WebRTC. This can be useful in low latency usecases where synchronization is not required."
		))
	bool WebRTCDisableAudioSync = true;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCEnableFlexFec;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Enable Flexible Forward Error Correction",
		ToolTip = "Signals support for Flexible Forward Error Correction to WebRTC. This can cause a reduction in quality if total bitrate is low."
		))
	bool WebRTCEnableFlexFec = false;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableStats;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Disable WebRtc Stats",
		ToolTip = "Disables the collection of WebRTC stats."
		))
	bool WebRTCDisableStats = false;

	static UE_API TAutoConsoleVariable<float> CVarWebRTCStatsInterval;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Stats Interval",
		ToolTip = "Configures how often WebRTC stats are collected."
		))
	float WebRTCStatsInterval = 1.f;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCNegotiateCodecs;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "WebRtc Negotiate Codecs",
		ToolTip = "Whether PixelStreaming should send all its codecs during sdp handshake so peers can negotiate or just send a single selected codec."
		))
	bool WebRTCNegotiateCodecs = false;

	static UE_API TAutoConsoleVariable<FString> CVarWebRTCCodecPreferences;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "WebRtc Codec Preferences",
		ToolTip = "The preference order PixelStreaming will specify during sdp handshake",
		EditFixedSize,
		EditCondition="WebRTCNegotiateCodecs",
		EditConditionHides,
		GetOptions = "GetWebRTCCodecPreferencesOptions"
		))
	TArray<FString> WebRTCCodecPreferences = { 
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::AV1), 
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::H264), 
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP9), 
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP8)
	 };

	static UE_API TArray<EVideoCodec> GetCodecPreferences();

	static UE_API TAutoConsoleVariable<float> CVarWebRTCAudioGain;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default Audio Gain",
		ToolTip = "Sets the amount of gain to apply to audio. Default: 1.0"
		))
	float WebRTCAudioGain = 1.0f;

	static UE_API TAutoConsoleVariable<FString> CVarWebRTCPortAllocatorFlags;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		Bitmask,
		BitmaskEnum = "/Script/PixelStreaming2Settings.EPortAllocatorFlags",
		DisplayName = "WebRtc port allocation flags",
		ToolTip = "Sets the WebRTC port allocator flags. See "
		))
	int32 WebRTCPortAllocatorFlags = 0;

	static UE_API EPortAllocatorFlags GetPortAllocationFlags();

	static UE_API TAutoConsoleVariable<int> CVarWebRTCMinPort;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default WebRtc Min Port",
		ToolTip = "Sets the minimum usable port for the WebRTC port allocator. Default: 49152"
		))
	int WebRTCMinPort = 49152;

	static UE_API TAutoConsoleVariable<int> CVarWebRTCMaxPort;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings", meta = (
		DisplayName = "Default WebRtc Max Port",
		ToolTip = "Sets the maximum usable port for the WebRTC port allocator. Default: 65535"
		))
	int WebRTCMaxPort = 65535;

	static UE_API TAutoConsoleVariable<FString> CVarWebRTCFieldTrials;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings|Field Trials", meta = (
		DisplayName = "Manually defined WebRtc Field Trials",
		ToolTip = "Sets the WebRTC field trials string. Format:\"TRIAL1/VALUE1/TRIAL2/VALUE2/\" see https://webrtc.googlesource.com/src/+/HEAD/g3doc/field-trials.md"
		))
	FString WebRTCFieldTrials;

	static UE_API TAutoConsoleVariable<bool> CVarWebRTCDisableFrameDropper;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings|Field Trials", meta = (
		DisplayName = "Disable Frame Dropper",
		ToolTip = "Disables the WebRTC internal frame dropper using the field trial WebRTC-FrameDropper/Disabled/"
		))
	bool WebRTCDisableFrameDropper = false;

	static UE_API TAutoConsoleVariable<float> CVarWebRTCVideoPacingMaxDelay;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings|Field Trials", meta = (
		DisplayName = "Video Pacing Max Delay",
		ToolTip = "Enables the WebRTC-Video-Pacing field trial and sets the max delay (ms) parameter. Default: -1.0f (values below zero are discarded.)"
		))
	float WebRTCVideoPacingMaxDelay = -1.0f;

	static UE_API TAutoConsoleVariable<float> CVarWebRTCVideoPacingFactor;
	UPROPERTY(config, EditAnywhere, Category = "WebRtcSettings|Field Trials", meta = (
		DisplayName = "Video Pacing Factor",
		ToolTip = "Enables the WebRTC-Video-Pacing field trial and sets the video pacing factor parameter. Larger values are more lenient on larger bitrates. Default: -1.0f (values below zero are discarded.)"
		))
	float WebRTCVideoPacingFactor = -1.0f;

	// End WebRTC CVars

	// Begin EditorStreaming CVars
	static UE_API TAutoConsoleVariable<bool> CVarEditorUseRemoteSignallingServer;
	UPROPERTY(config, EditAnywhere, Category = "EditorStreaming", meta = (
		DisplayName = "Use Remote Signalling Server",
		ToolTip = "Enables the use of a remote signalling server. Default: false"
		))
	bool EditorUseRemoteSignallingServer = false;

	static UE_API TAutoConsoleVariable<bool> CVarAutoStreamPIE;
	UPROPERTY(config, EditAnywhere, Category = "EditorStreaming", meta = (
		DisplayName = "Auto Stream Play In Editor",
		ToolTip = "Automatically start streaming when playing in the editor. Default: true"
		))
	bool AutoStreamPlayInEditor = true;

	static UE_API TAutoConsoleVariable<bool> CVarEditorStartOnLaunch;
	UPROPERTY(config, EditAnywhere, Category = "EditorStreaming", meta = (
		DisplayName = "Start Editor Streamer On Launch",
		ToolTip = "Start Editor Streaming as soon as the Unreal Editor is launched. Default: false"
		))
	bool EditorStartOnLaunch = false;

	static UE_API TAutoConsoleVariable<FString> CVarEditorSource;
	UPROPERTY(config, EditAnywhere, Category = "EditorStreaming", meta = (
		DisplayName = "Editor Streamer Source",
		ToolTip = "Editor PixelStreaming source. Supported values are `Editor`, `LevelEditorViewport`. Default: Editor"
		))
	EPixelStreaming2EditorStreamTypes EditorSource = EPixelStreaming2EditorStreamTypes::Editor;
	// End EditorStreaming CVars

	// Begin HMD CVars
	static UE_API TAutoConsoleVariable<bool> CVarHMDEnable;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Enable HMD",
		ToolTip = "Enables HMD specific functionality for Pixel Streaming. Namely input handling and stereoscopic rendering. Default: false"
		))
	bool HMDEnable = false;

	static UE_API TAutoConsoleVariable<bool> CVarHMDMatchAspectRatio;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Match Aspect Ratio",
		ToolTip = "If true automatically resize the rendering resolution to match the aspect ratio determined by the HFoV and VFoV. Default: true"
		))
	bool HMDMatchAspectRatio = true;

	static UE_API TAutoConsoleVariable<bool> CVarHMDApplyEyePosition;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Apply Eye Position",
		ToolTip = "If true automatically position each eye's rendering by whatever amount WebXR reports for each left-right XRView. If false do no eye positioning. Default: true"
		))
	bool HMDAppleEyePosition = true;

	static UE_API TAutoConsoleVariable<bool> CVarHMDApplyEyeRotation;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Apply Eye Position",
		ToolTip = "If true automatically rotate each eye's rendering by whatever amount WebXR reports for each left-right XRView. If false do no eye rotation. Default: true"
		))
	bool HMDApplyEyeRotation = true;

	static UE_API TAutoConsoleVariable<float> CVarHMDHFOV;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Horizontal FOV Override",
		ToolTip = "Overrides the horizontal field of view for HMD rendering, values are in degrees and values less than 0.0f disable the override. Default: -1.0f"
		))
	float HMDHFOV = -1.0f;

	static UE_API TAutoConsoleVariable<float> CVarHMDVFOV;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Vertical FOV Override",
		ToolTip = "Overrides the vertical field of view for HMD rendering, values are in degrees and values less than 0.0f disable the override. Default: -1.0f"
		))
	float HMDVFOV = -1.0f;

	static UE_API TAutoConsoleVariable<float> CVarHMDIPD;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Interpupillary Distance Override",
		ToolTip = "Overrides the HMD IPD (interpupillary distance), values are in centimeters and values less than 0.0f disable the override. Default: -1.0f"
		))
	float HMDIPD = -1.0f;

	static UE_API TAutoConsoleVariable<float> CVarHMDProjectionOffsetX;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Horizontal Projection Offset Override",
		ToolTip = "Overrides the left/right eye projection matrix x-offset, values are in clip space and values less than 0.0f disable the override. Default: -1.0f"
		))
	float HMDProjectionOffsetX = -1.0f;

	static UE_API TAutoConsoleVariable<float> CVarHMDProjectionOffsetY;
	UPROPERTY(config, EditAnywhere, Category = "XR Streaming", meta = (
		DisplayName = "Vertical Projection Offset Override",
		ToolTip = "Overrides the left-right eye projection matrix y-offset, values are in clip space and values less than 0.0f disable the override. Default: -1.0f"
		))
	float HMDProjectionOffsetY = -1.0f;
	// End HMD CVars

	// Begin Input CVars
	static UE_API TAutoConsoleVariable<bool> CVarInputAllowConsoleCommands;
	UPROPERTY(config, EditAnywhere, Category = "Input", meta = (
		DisplayName = "Allow Commands",
		ToolTip = "If true browser can send consoleCommand payloads that execute in UE's console. Default: false"
		))
	bool InputAllowConsoleCommands = false;

	static UE_API TAutoConsoleVariable<FString> CVarInputKeyFilter;
	UPROPERTY(config, EditAnywhere, Category = "Input", meta = (
		DisplayName = "Key Filter",
		ToolTip = "Comma separated list of keys to ignore from streaming clients. Default: \"\""
		))
	FString InputKeyFilter = TEXT("");

	// End Input CVars

	// Begin Signalling Server CVars
	static UE_API TAutoConsoleVariable<FString> CVarAllowedOrigins;
	UPROPERTY(config, EditAnywhere, Category = "SignallingServer", meta = (
		DisplayName = "Allowed Origins",
		ToolTip = "Comma-separated list of allowed origins for WebSocket connection filtering. Supports wildcards (*). Use '*' to allow all origins. Leave empty to disable filtering. Examples: \"*\"  or \"https://example.com,http://192.168.0.*:8080\". Default: \"http://127.0.0.1,https://127.0.0.1\""
		))
	FString AllowedOrigins = TEXT("http://127.0.0.1,https://127.0.0.1");
	// End Signalling Server CVars

	// clang-format on

	// Begin UDeveloperSettings Interface
	UE_API virtual FName GetCategoryName() const override;

#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API void VerifyVideoSettings();
#endif
	// End UDeveloperSettings Interface

	// Begin UObject Interface
	UE_API virtual void PostInitProperties() override;
	// End UObject Interface

	struct FDelegates
	{
		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnScalabilityModeChanged, IConsoleVariable*);
		FOnScalabilityModeChanged OnScalabilityModeChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnSimulcastEnabledChanged, IConsoleVariable*);
		FOnSimulcastEnabledChanged OnSimulcastEnabledChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnCaptureUseFenceChanged, IConsoleVariable*);
		FOnCaptureUseFenceChanged OnCaptureUseFenceChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnUseMediaCaptureChanged, IConsoleVariable*);
		FOnUseMediaCaptureChanged OnUseMediaCaptureChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnWebRTCFpsChanged, IConsoleVariable*);
		FOnWebRTCFpsChanged OnWebRTCFpsChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnWebRTCBitrateChanged, IConsoleVariable*);
		FOnWebRTCBitrateChanged OnWebRTCBitrateChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnWebRTCDisableStatsChanged, IConsoleVariable*);
		FOnWebRTCDisableStatsChanged OnWebRTCDisableStatsChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnLogStatsChanged, IConsoleVariable*);
		FOnLogStatsChanged OnLogStatsChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnInputKeyFilterChanged, IConsoleVariable*);
		FOnInputKeyFilterChanged OnInputKeyFilterChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnEncoderDebugDumpFrameChanged, IConsoleVariable*);
		FOnEncoderDebugDumpFrameChanged OnEncoderDebugDumpFrameChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnDebugDumpAudioChanged, IConsoleVariable*);
		FOnDebugDumpAudioChanged OnDebugDumpAudioChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnDecoupleFramerateChanged, IConsoleVariable*);
		FOnDecoupleFramerateChanged OnDecoupleFramerateChanged;

		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnEpicRtcLogFilterChanged, IConsoleVariable*);
		FOnEpicRtcLogFilterChanged OnEpicRtcLogFilterChanged;
	};

	static UE_API FDelegates* Delegates();

private:
	UE_API void SetCVarAndPropertyFromValue(const FString& CVarName, FProperty* Property, const FString& Value);
	UE_API void SetCVarFromProperty(const FString& CVarName, FProperty* Property);

	UE_API void InitializeCVarsFromProperties();
	UE_API void ValidateCommandLineArgs();
	UE_API void ParseCommandlineArgs();
	UE_API void ParseLegacyCommandlineArgs();

	static UE_API FDelegates* DelegateSingleton;

	UFUNCTION()
	UE_API TArray<FString> GetVideoCodecOptions() const;

	UFUNCTION()
	UE_API TArray<FString> GetScalabilityModeOptions() const;

	UFUNCTION()
	UE_API TArray<FString> GetWebRTCCodecPreferencesOptions() const;

	UFUNCTION()
	UE_API TArray<FString> GetDefaultStreamerTypeOptions() const;
};

#undef UE_API
