// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class PixelStreaming2Settings : ModuleRules
	{
		public PixelStreaming2Settings(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"EngineSettings",
				"PixelStreaming2Core"
			});

			PublicDependencyModuleNames.AddRange(new string[]
			{
				"AVCodecsCore",
				"EpicRtc",
				"Slate"
			});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(new string[]
				{
					"UnrealEd"
				});
			}
		}
	}
}
