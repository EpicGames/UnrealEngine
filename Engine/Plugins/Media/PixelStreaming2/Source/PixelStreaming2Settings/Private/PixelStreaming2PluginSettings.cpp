// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixelStreaming2PluginSettings.h"

#include "IPixelStreaming2Streamer.h"
#include "Logging.h"
#include "Misc/CommandLine.h"
#include "UObject/ReflectedTypeAccessors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PixelStreaming2PluginSettings)

namespace
{
	template <typename TEnumType>
	static void CheckConsoleEnum(IConsoleVariable* InConsoleVariable)
	{
		FString ConsoleString = InConsoleVariable->GetString();
		if (StaticEnum<TEnumType>()->GetIndexByNameString(ConsoleString) == INDEX_NONE)
		{
			// Legacy CVar values were the enum values but underscores (LOW_LATENCY) instead of the camel case UENUM string (LowLatency). They are still valid we just need to remove the underscores when we check them.
			if (ConsoleString = ConsoleString.Replace(TEXT("_"), TEXT("")); StaticEnum<TEnumType>()->GetIndexByNameString(ConsoleString) != INDEX_NONE)
			{
				InConsoleVariable->Set(*ConsoleString, ECVF_SetByConsole);
			}
			else
			{
				FString ConsoleObjectName = IConsoleManager::Get().FindConsoleObjectName(InConsoleVariable);
				UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Invalid value {0} received for enum {1} of type {2}", ConsoleString, ConsoleObjectName, StaticEnum<TEnumType>()->GetName());
				InConsoleVariable->Set(*InConsoleVariable->GetDefaultValue(), ECVF_SetByConsole);
			}
		}
	}

	static void VerifyCVarVideoSettings(IConsoleVariable* /* We ignore the passed in console variable as this method is called by many different CVars */)
	{
		IConsoleVariable* SimulcastCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming2.Encoder.EnableSimulcast"));
		IConsoleVariable* CodecCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming2.Encoder.Codec"));
		IConsoleVariable* ScalabilityModeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("PixelStreaming2.Encoder.ScalabilityMode"));

		// Verify that the video codec and scalability mode strings correctly map to an enum
		CheckConsoleEnum<EVideoCodec>(CodecCVar);
		CheckConsoleEnum<EScalabilityMode>(ScalabilityModeCVar);

		if (SimulcastCVar->GetBool())
		{
			// Check that the selected codec supports simulcast
			FString Codec = CodecCVar->GetString();
			if (Codec != TEXT("H264") && Codec != TEXT("VP8"))
			{
				UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Selected codec doesn't support simulcast! Resetting default codec to {0}", CodecCVar->GetDefaultValue());
				CodecCVar->Set(*CodecCVar->GetDefaultValue(), ECVF_SetByConsole);
			}
		}

		FString Codec = CodecCVar->GetString();
		FString ScalabilityMode = ScalabilityModeCVar->GetString();
		if ((Codec == TEXT("H264") || Codec == TEXT("VP8"))
			&& (ScalabilityMode != TEXT("L1T1") && ScalabilityMode != TEXT("L1T2") && ScalabilityMode != TEXT("L1T3")))
		{
			UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Selected codec doesn't support the {0} scalability mode! Resetting scalability mode to {1}", ScalabilityMode, ScalabilityModeCVar->GetDefaultValue());
			ScalabilityModeCVar->Set(*ScalabilityModeCVar->GetDefaultValue(), ECVF_SetByConsole);
		}
	}

	static void VerifyCVarDefaultStreamerType(IConsoleVariable* CVar)
	{
		TArray<FString> AvailableFactoryTypes = IPixelStreaming2StreamerFactory::GetAvailableFactoryTypes();
		FString			SpecifiedFactory = CVar->GetString();

		if (AvailableFactoryTypes.Num() == 0)
		{
			// This code path executes when the cvar is initially set and no factories have been registered
			return;
		}

		bool bValid = false;
		for (const FString& AvailableFactory : AvailableFactoryTypes)
		{
			if (SpecifiedFactory == AvailableFactory)
			{
				bValid = true;
				break;
			}
		}

		if (!bValid)
		{
			UE_LOGFMT(LogPixelStreaming2Settings, Warning, "\"{0}\" isn't a registered streamer type. Valid types: [{1}]. Restoring to \"{2}\"", SpecifiedFactory, FString::Join(AvailableFactoryTypes, TEXT(",")), CVar->GetDefaultValue());
			CVar->SetWithCurrentPriority(*CVar->GetDefaultValue());
		}
	}

	FString ConsoleVariableToCommandArgValue(const FString InCVarName)
	{
		// CVars are . deliminated by section. To get their equivilent commandline arg for parsing
		// we need to remove the . and add a "="
		return InCVarName.Replace(TEXT("."), TEXT("")).Replace(TEXT("PixelStreaming2"), TEXT("PixelStreaming")).Append(TEXT("="));
	}

	FString ConsoleVariableToCommandArgParam(const FString InCVarName)
	{
		// CVars are . deliminated by section. To get their equivilent commandline arg parameter, we need to to remove the .
		return InCVarName.Replace(TEXT("."), TEXT("")).Replace(TEXT("PixelStreaming2"), TEXT("PixelStreaming"));
	}

	static void ParseLegacyCommandLineValue(const TCHAR* Match, TAutoConsoleVariable<FString>& CVar)
	{
		FString Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
		{
			CVar->Set(*Value, ECVF_SetByCommandline);
		}
	};

	static void ParseLegacyCommandLineOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
		{
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
			{
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
			{
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}

	static FString FindPropertyFromCVar(const TSet<TPair<FString, FString>> Set, const FString& Key)
	{
		for (const TPair<FString, FString>& Pair : Set)
		{
			if (Pair.Key == Key)
			{
				return Pair.Value;
			}
		}

		return "";
	}

	static FString FindCVarFromProperty(const TSet<TPair<FString, FString>> Set, const FString& Value)
	{
		for (const TPair<FString, FString>& Pair : Set)
		{
			if (Pair.Value == Value)
			{
				return Pair.Key;
			}
		}

		return "";
	}
} // namespace

// Map of Property Names to their commandline args as GetMetaData() is not avaliable in packaged projects
static const TSet<TPair<FString, FString>> GetCmdArg = {
	{ "PixelStreaming2.LogStats", "LogStats" },
	{ "PixelStreaming2.EpicRtcLogFilter", "EpicRtcLogFilter" },
	{ "PixelStreaming2.SendPlayerIdAsInteger", "SendPlayerIdAsInteger" },
	{ "PixelStreaming2.DisableLatencyTester", "DisableLatencyTester" },
	{ "PixelStreaming2.DecoupleFrameRate", "DecoupleFramerate" },
	{ "PixelStreaming2.DecoupleWaitFactor", "DecoupleWaitFactor" },
	{ "PixelStreaming2.SignalingReconnectInterval", "SignalingReconnectInterval" },
	{ "PixelStreaming2.SignalingMaxReconnectAttempts", "SignalingMaxReconnectAttempts" },
	{ "PixelStreaming2.SignalingKeepAliveInterval", "SignalingKeepAliveInterval" },
	{ "PixelStreaming2.UseMediaCapture", "UseMediaCapture" },
	{ "PixelStreaming2.InitializeDefaultStreamer", "InitializeDefaultStreamer" },
	{ "PixelStreaming2.ID", "DefaultStreamerID" },
	{ "PixelStreaming2.DefaultStreamerType", "DefaultStreamerType" },
	{ "PixelStreaming2.AutoStartStream", "AutoStartStream" },
	{ "PixelStreaming2.ConnectionURL", "ConnectionURL" },
	{ "PixelStreaming2.CaptureUseFence", "CaptureUseFence" },
	{ "PixelStreaming2.Encoder.Codec", "Codec" },
	{ "PixelStreaming2.Encoder.TargetBitrate", "EncoderTargetBitrate" },
	{ "PixelStreaming2.Encoder.MinQuality", "EncoderMinQuality" },
	{ "PixelStreaming2.Encoder.MaxQuality", "EncoderMaxQuality" },
	{ "PixelStreaming2.Encoder.ScalabilityMode", "ScalabilityMode" },
	{ "PixelStreaming2.Encoder.KeyframeInterval", "KeyframeInterval" },
	{ "PixelStreaming2.Encoder.MaxSessions", "MaxSessions" },
	{ "PixelStreaming2.Encoder.EnableSimulcast", "EnableSimulcast" },
	{ "PixelStreaming2.WebRTC.Fps", "WebRTCFps" },
	{ "PixelStreaming2.WebRTC.StartBitrate", "WebRTCStartBitrate" },
	{ "PixelStreaming2.WebRTC.MinBitrate", "WebRTCMinBitrate" },
	{ "PixelStreaming2.WebRTC.MaxBitrate", "WebRTCMaxBitrate" },
	{ "PixelStreaming2.WebRTC.DisableReceiveAudio", "WebRTCDisableReceiveAudio" },
	{ "PixelStreaming2.WebRTC.DisableReceiveVideo", "WebRTCDisableReceiveVideo" },
	{ "PixelStreaming2.WebRTC.DisableTransmitAudio", "WebRTCDisableTransmitAudio" },
	{ "PixelStreaming2.WebRTC.DisableTransmitVideo", "WebRTCDisableTransmitVideo" },
	{ "PixelStreaming2.WebRTC.DisableAudioSync", "WebRTCDisableAudioSync" },
	{ "PixelStreaming2.WebRTC.EnableFlexFec", "WebRTCEnableFlexFec" },
	{ "PixelStreaming2.WebRTC.DisableStats", "WebRTCDisableStats" },
	{ "PixelStreaming2.WebRTC.StatsInterval", "WebRTCStatsInterval" },
	{ "PixelStreaming2.WebRTC.NegotiateCodecs", "WebRTCNegotiateCodecs" },
	{ "PixelStreaming2.WebRTC.AudioGain", "WebRTCAudioGain" },
	{ "PixelStreaming2.WebRTC.PortAllocatorFlags", "WebRTCPortAllocatorFlags" },
	{ "PixelStreaming2.WebRTC.MinPort", "WebRTCMinPort" },
	{ "PixelStreaming2.WebRTC.MaxPort", "WebRTCMaxPort" },
	{ "PixelStreaming2.WebRTC.FieldTrials", "WebRTCFieldTrials" },
	{ "PixelStreaming2.WebRTC.DisableFrameDropper", "WebRTCDisableFrameDropper" },
	{ "PixelStreaming2.WebRTC.VideoPacing.MaxDelay", "WebRTCVideoPacingMaxDelay" },
	{ "PixelStreaming2.WebRTC.VideoPacing.Factor", "WebRTCVideoPacingFactor" },
	{ "PixelStreaming2.Editor.StartOnLaunch", "EditorStartOnLaunch" },
	{ "PixelStreaming2.Editor.AutoStreamPIE", "AutoStreamPlayInEditor" },
	{ "PixelStreaming2.Editor.UseRemoteSignallingServer", "EditorUseRemoteSignallingServer" },
	{ "PixelStreaming2.HMD.Enable", "HMDEnable" },
	{ "PixelStreaming2.HMD.MatchAspectRatio", "HMDMatchAspectRatio" },
	{ "PixelStreaming2.HMD.ApplyEyePosition", "HMDAppleEyePosition" },
	{ "PixelStreaming2.HMD.ApplyEyeRotation", "HMDApplyEyeRotation" },
	{ "PixelStreaming2.HMD.HFOV", "HMDHFOV" },
	{ "PixelStreaming2.HMD.VFOV", "HMDVFOV" },
	{ "PixelStreaming2.HMD.IPD", "HMDIPD" },
	{ "PixelStreaming2.HMD.ProjectionOffsetX", "HMDProjectionOffsetX" },
	{ "PixelStreaming2.HMD.ProjectionOffsetY", "HMDProjectionOffsetY" },
	{ "PixelStreaming2.AllowPixelStreamingCommands", "InputAllowConsoleCommands" },
	{ "PixelStreaming2.KeyFilter", "InputKeyFilter" },
	{ "PixelStreaming2.WebRTC.CodecPreferences", "WebRTCCodecPreferences" },
	{ "PixelStreaming2.InternalSignallingServer.AllowedOrigins", "AllowedOrigins" }
};

static const TSet<TPair<FString, FString>> GetMappedCmdArg = {
	{ "PixelStreaming2.InputController", "InputController" },
	{ "PixelStreaming2.Encoder.QualityPreset", "QualityPreset" },
	{ "PixelStreaming2.Encoder.LatencyMode", "LatencyMode" },
	{ "PixelStreaming2.Encoder.H264Profile", "H264Profile" },
	{ "PixelStreaming2.Editor.Source", "EditorSource" }
};

// Map a legacy cvar to its new property
static const TSet<TPair<FString, FString>> GetLegacyCmdArg = {
	{ "PixelStreaming2.Encoder.MinQp", "EncoderMaxQuality" },								   // Renamed to MaxQuality
	{ "PixelStreaming2.Encoder.MaxQp", "EncoderMinQuality" },								   // Renamed to MinQuality
	{ "PixelStreaming2.IP", "ConnectionURL" },												   // Moved to ConnectionURL
	{ "PixelStreaming2.Port", "ConnectionURL" },											   // Moved to ConnectionURL
	{ "PixelStreaming2.URL", "ConnectionURL" },												   // Renamed to ConnectionURL
	{ "PixelStreaming2.SignallingURL", "ConnectionURL" },									   // Renamed to ConnectionURL
	{ "AllowPixelStreamingCommands", "InputAllowConsoleCommands" },							   // Renamed to InputAllowConsoleCommands
	{ "PixelStreaming2.NegotiateCodecs", "WebRTCNegotiateCodecs" },							   // Renamed to PixelStreaming2.WebRTC.NegotiateCodecs
	{ "PixelStreaming2.EnableHMD", "HMDEnable" },											   // Renamed to PixelStreaming2.HMDEnable
	{ "Editor.PixelStreaming2.StartOnLaunch", "EditorStartOnLaunch" },						   // Renamed to PixelStreaming2.Editor.StartOnLaunch
	{ "Editor.PixelStreaming2.UseRemoteSignallingServer", "EditorUseRemoteSignallingServer" }, // Renamed to PixelStreaming2.Editor.UseRemoteSignallingServer
	{ "Editor.PixelStreaming2.Source", "EditorSource" }										   // Renamed to PixelStreaming2.Editor.Source
};

// Begin Pixel Streaming Plugin CVars
TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarLogStats(
	TEXT("PixelStreaming2.LogStats"),
	false,
	TEXT("Whether to show PixelStreaming stats in the log (default: false)."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates *Delegations = Delegates())
			{
				Delegations->OnLogStatsChanged.Broadcast(Var);
			}
		}),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEpicRtcLogFilter(
	TEXT("PixelStreaming2.EpicRtcLogFilter"),
	"",
	TEXT("Double forward slash (\"//\") separated list of regex patterns to filter from the EpicRtc logs (default: \"\")."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnEpicRtcLogFilterChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarDisableLatencyTester(
	TEXT("PixelStreaming2.DisableLatencyTester"),
	false,
	TEXT("If true disables latency tester being triggerable."),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarInputController(
	TEXT("PixelStreaming2.InputController"),
	TEXT("Any"),
	TEXT("Various modes of input control supported by Pixel Streaming, currently: \"Any\"  or \"Host\". Default: Any"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarDecoupleFramerate(
	TEXT("PixelStreaming2.DecoupleFramerate"),
	false,
	TEXT("Whether we should only stream as fast as we render or at some fixed interval. Coupled means only stream what we render."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnDecoupleFramerateChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarDecoupleWaitFactor(
	TEXT("PixelStreaming2.DecoupleWaitFactor"),
	1.25f,
	TEXT("Frame rate factor to wait for a captured frame when streaming in decoupled mode. Higher factor waits longer but may also result in higher latency."),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarSignalingReconnectInterval(
	TEXT("PixelStreaming2.SignalingReconnectInterval"),
	2.0f,
	TEXT("Changes the number of seconds between attempted reconnects to the signaling server. This is useful for reducing the log spam produced from attempted reconnects. A value <= 0 results in no reconnect. Default: 2.0s"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarSignalingMaxReconnectAttempts(
	TEXT("PixelStreaming2.SignalingMaxReconnectAttempts"),
	-1,
	TEXT("Changes the number of attempts that will be made to reconnect to the signalling server. This is useful for triggering application shutdowns if this value is exceeded. A value of < 0 results in unlimited attempts. Default: -1"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarSignalingKeepAliveInterval(
	TEXT("PixelStreaming2.SignalingKeepAliveInterval"),
	30.0f,
	TEXT("Changes the number of seconds between pings to the signaling server. This is useful for keeping the connection active. A value <= 0 results in no pings. Default: 30.0"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarUseMediaCapture(
	TEXT("PixelStreaming2.UseMediaCapture"),
	true,
	TEXT("Use Media Capture from MediaIOFramework to capture frames rather than Pixel Streamings internal backbuffer sources."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnUseMediaCaptureChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarInitializeDefaultStreamer(
	TEXT("PixelStreaming2.InitializeDefaultStreamer"),
	true,
	TEXT("If the default Pixel Streaming 2 streamer should be initialized. You may wish to set this to false if you only plan on receiving streams. Default: true"),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarDefaultStreamerID(
	TEXT("PixelStreaming2.ID"),
	TEXT("DefaultStreamer"),
	TEXT("Default Streamer ID to be used when not specified elsewhere."),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarDefaultStreamerType(
	TEXT("PixelStreaming2.DefaultStreamerType"),
	TEXT("DefaultRtc"),
	TEXT("Default Streamer Type to be used when not specified elsewhere."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) { VerifyCVarDefaultStreamerType(Var); }),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarAutoStartStream(
	TEXT("PixelStreaming2.AutoStartStream"),
	true,
	TEXT("Configure the PixelStreaming2 plugin to automatically start streaming once loaded (if not in editor). You may wish to set this value to false and manually call StartStreaming at a later point from your c++ code. Default: true"),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarConnectionURL(
	TEXT("PixelStreaming2.ConnectionURL"),
	TEXT(""),
	TEXT("Default URL to connect to. This can be a URL to a signalling server or some other endpoint with the format (protocol)://(host):(port)"),
	ECVF_Default);

FAutoConsoleVariableDeprecated UPixelStreaming2PluginSettings::CVarSignallingURL(TEXT("PixelStreaming2.SignallingURL"), TEXT("PixelStreaming2.ConnectionURL"), TEXT("5.6"));

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarCaptureUseFence(
	TEXT("PixelStreaming2.CaptureUseFence"),
	true,
	TEXT("Whether the texture copy we do during image capture should use a fence or not (non-fenced is faster but less safe)."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnCaptureUseFenceChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarDebugDumpAudio(
	TEXT("PixelStreaming2.DumpDebugAudio"),
	false,
	TEXT("Dumps mixed audio from PS2 to a file on disk for debugging purposes."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnDebugDumpAudioChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

// Begin Encoder CVars

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarEncoderTargetBitrate(
	TEXT("PixelStreaming2.Encoder.TargetBitrate"),
	-1,
	TEXT("Target bitrate (bps). Ignore the bitrate WebRTC wants (not recommended). Set to -1 to disable. Default -1."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarEncoderMinQuality(
	TEXT("PixelStreaming2.Encoder.MinQuality"),
	0,
	TEXT("0-100, Higher values result in a better minimum quality but higher average bitrates. Default 0 - i.e. no limit on a minimum Quality."),
	ECVF_Default);

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarEncoderMaxQuality(
	TEXT("PixelStreaming2.Encoder.MaxQuality"),
	100,
	TEXT("0-100, Lower values result in lower average bitrates but reduces maximum achievable quality. Default 100 - i.e. no limit on a maximum Quality."),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEncoderQualityPreset(
	TEXT("PixelStreaming2.Encoder.QualityPreset"),
	TEXT("Default"),
	TEXT("PixelStreaming encoder presets that affecting Quality vs Bitrate. Supported modes are: `ULTRA_LOW_QUALITY`, `LOW_QUALITY`, `DEFAULT`, `HIGH_QUALITY` or `LOSSLESS`"),
	FConsoleVariableDelegate::CreateStatic(&CheckConsoleEnum<EAVPreset>),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEncoderLatencyMode(
	TEXT("PixelStreaming2.Encoder.LatencyMode"),
	TEXT("UltraLowLatency"),
	TEXT("PixelStreaming encoder mode that affecting Quality vs Latency. Supported modes are: `ULTRA_LOW_LATENCY`, `LOW_LATENCY` or `DEFAULT`"),
	ECVF_Default);

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarEncoderKeyframeInterval(
	TEXT("PixelStreaming2.Encoder.KeyframeInterval"),
	-1,
	TEXT("How many frames before a key frame is sent. Default: -1 which disables the sending of periodic key frames. Note: NVENC reqires a reinitialization when this changes."),
	ECVF_Default);

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarEncoderMaxSessions(
	TEXT("PixelStreaming2.Encoder.MaxSessions"),
	-1,
	TEXT("-1 implies no limit. Maximum number of concurrent hardware encoder sessions for Pixel Streaming. Note GeForce gpus only support 8 concurrent sessions and will rollover to software encoding when that number is exceeded."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarEncoderEnableSimulcast(
	TEXT("PixelStreaming2.Encoder.EnableSimulcast"),
	false,
	TEXT("Enables simulcast. When enabled, the encoder will encode at full resolution, 1/2 resolution and 1/4 resolution simultaneously. Note: Simulcast is only supported with `H264` and `VP8` and you must use the SFU from the infrastructure to fully utilise this functionality."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			VerifyCVarVideoSettings(nullptr); 
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnSimulcastEnabledChanged.Broadcast(Var);
			}
		}),
	ECVF_Default);

FString UE::PixelStreaming2::GSelectedCodec = TEXT("H264");

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEncoderCodec(
	TEXT("PixelStreaming2.Encoder.Codec"),
	UE::PixelStreaming2::GSelectedCodec,
	TEXT("PixelStreaming default encoder codec. Supported values are: `H264`, `VP8`, `VP9` or `AV1`"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) { UE::PixelStreaming2::GSelectedCodec = Var->GetString(); VerifyCVarVideoSettings(nullptr); }),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEncoderScalabilityMode(
	TEXT("PixelStreaming2.Encoder.ScalabilityMode"),
	TEXT("L1T1"),
	TEXT("Indicates number of Spatial and temporal layers used, default: L1T1. For a full list of values refer to https://www.w3.org/TR/webrtc-svc/#scalabilitymodes*"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			VerifyCVarVideoSettings(nullptr); 
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnScalabilityModeChanged.Broadcast(Var);
			}
		}),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEncoderH264Profile(
	TEXT("PixelStreaming2.Encoder.H264Profile"),
	TEXT("Baseline"),
	TEXT("PixelStreaming encoder profile. Supported modes are: `AUTO`, `BASELINE`, `MAIN`, `HIGH`, `PROGRESSIVE_HIGH`, `CONSTRAINED_HIGH` or `HIGH444`"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarEncoderDebugDumpFrame(
	TEXT("PixelStreaming2.Encoder.DumpDebugFrames"),
	false,
	TEXT("Dumps frames from the encoder to a file on disk for debugging purposes."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var)
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnEncoderDebugDumpFrameChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

// Begin WebRTC CVars

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarWebRTCFps(
	TEXT("PixelStreaming2.WebRTC.Fps"),
	60,
	TEXT("Framerate for WebRTC encoding. Default: 60"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnWebRTCFpsChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);

// Note: 1 megabit is the maximum allowed in WebRTC for a start bitrate.
TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarWebRTCStartBitrate(
	TEXT("PixelStreaming2.WebRTC.StartBitrate"),
	1000000,
	TEXT("Start bitrate (bps) that WebRTC will try begin the stream with. Must be between Min/Max bitrates. Default: 1000000"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarWebRTCMinBitrate(
	TEXT("PixelStreaming2.WebRTC.MinBitrate"),
	100000,
	TEXT("Min bitrate (bps) that WebRTC will not request below. Careful not to set too high otherwise WebRTC will just drop frames. Default: 100000"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{ 
			if (FDelegates* Delegations = Delegates())
			{ 
				Delegations->OnWebRTCBitrateChanged.Broadcast(Var); 
			} 
		}),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> UPixelStreaming2PluginSettings::CVarWebRTCMaxBitrate(
	TEXT("PixelStreaming2.WebRTC.MaxBitrate"),
	40000000,
	TEXT("Max bitrate (bps) that WebRTC will not request above. Default: 40000000 aka 40 megabits/per second."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{ 
			if (FDelegates* Delegations = Delegates())
			{ 
				Delegations->OnWebRTCBitrateChanged.Broadcast(Var); 
			} 
		}),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableReceiveAudio(
	TEXT("PixelStreaming2.WebRTC.DisableReceiveAudio"),
	false,
	TEXT("Disables receiving audio from the browser into UE."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableReceiveVideo(
	TEXT("PixelStreaming2.WebRTC.DisableReceiveVideo"),
	true,
	TEXT("Disables receiving video from the browser into UE."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableTransmitAudio(
	TEXT("PixelStreaming2.WebRTC.DisableTransmitAudio"),
	false,
	TEXT("Disables transmission of UE audio to the browser."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableTransmitVideo(
	TEXT("PixelStreaming2.WebRTC.DisableTransmitVideo"),
	false,
	TEXT("Disables transmission of UE video to the browser."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableAudioSync(
	TEXT("PixelStreaming2.WebRTC.DisableAudioSync"),
	true,
	TEXT("Disables the synchronization of audio and video tracks in WebRTC. This can be useful in low latency usecases where synchronization is not required."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCEnableFlexFec(
	TEXT("PixelStreaming2.WebRTC.EnableFlexFec"),
	false,
	TEXT("Signals support for Flexible Forward Error Correction to WebRTC. This can cause a reduction in quality if total bitrate is low."),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableStats(
	TEXT("PixelStreaming2.WebRTC.DisableStats"),
	false,
	TEXT("Disables the collection of WebRTC stats."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var)
		{
			if (FDelegates* Delegations = Delegates())
			{ 
				Delegations->OnWebRTCDisableStatsChanged.Broadcast(Var); 
			} 
		}),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarWebRTCStatsInterval(
	TEXT("PixelStreaming2.WebRTC.StatsInterval"),
	1.f,
	TEXT("Configures how often WebRTC stats are collected in seconds. Values less than 0.0f disable stats collection. Default: 1.0f"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCNegotiateCodecs(
	TEXT("PixelStreaming2.WebRTC.NegotiateCodecs"),
	false,
	TEXT("Whether PS should send all its codecs during sdp handshake so peers can negotiate or just send a single selected codec."),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarWebRTCCodecPreferences(
	TEXT("PixelStreaming2.WebRTC.CodecPreferences"),
	TEXT("AV1,H264,VP9,VP8"),
	TEXT("A comma separated list of video codecs specifying the prefered order PS will signal during sdp handshake"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarWebRTCAudioGain(
	TEXT("PixelStreaming2.WebRTC.AudioGain"),
	1.0f,
	TEXT("Sets the amount of gain to apply to audio. Default: 1.0"),
	ECVF_Default);

// End WebRTC CVars

// Begin EditorStreaming CVars
TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarEditorStartOnLaunch(
	TEXT("PixelStreaming2.Editor.StartOnLaunch"),
	false,
	TEXT("Start Editor Streaming as soon as the Unreal Editor is launched. Default: false"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarAutoStreamPIE(
	TEXT("PixelStreaming2.Editor.AutoStreamPIE"),
	true,
	TEXT("Automatically start streaming when playing in the editor. Default: true"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarEditorUseRemoteSignallingServer(
	TEXT("PixelStreaming2.Editor.UseRemoteSignallingServer"),
	false,
	TEXT("Enables the use of a remote signalling server. Default: false"),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarEditorSource(
	TEXT("PixelStreaming2.Editor.Source"),
	TEXT("Editor"),
	TEXT("Editor PixelStreaming source. Supported values are `Editor`, `LevelEditorViewport`. Default: `Editor`"),
	FConsoleVariableDelegate::CreateStatic(&CheckConsoleEnum<EPixelStreaming2EditorStreamTypes>),
	ECVF_Default);
// End EditorStreaming CVars

// Begin HMD CVars
TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarHMDEnable(
	TEXT("PixelStreaming2.HMD.Enable"),
	false,
	TEXT("Enables HMD specific functionality for Pixel Streaming. Namely input handling and stereoscopic rendering. Default: false"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarHMDMatchAspectRatio(
	TEXT("PixelStreaming2.HMD.MatchAspectRatio"),
	true,
	TEXT("If true automatically resize the rendering resolution to match the aspect ratio determined by the HFoV and VFoV. Default: true"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarHMDApplyEyePosition(
	TEXT("PixelStreaming2.HMD.ApplyEyePosition"),
	true,
	TEXT("If true automatically position each eye's rendering by whatever amount WebXR reports for each left-right XRView. If false do no eye positioning. Default: true"),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarHMDApplyEyeRotation(
	TEXT("PixelStreaming2.HMD.ApplyEyeRotation"),
	true,
	TEXT("If true automatically rotate each eye's rendering by whatever amount WebXR reports for each left-right XRView. If false do no eye rotation. Default: true"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarHMDHFOV(
	TEXT("PixelStreaming2.HMD.HFOV"),
	-1.0f,
	TEXT("Overrides the horizontal field of view for HMD rendering, values are in degrees and values less than 0.0f disable the override. Default: -1.0f"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarHMDVFOV(
	TEXT("PixelStreaming2.HMD.VFOV"),
	-1.0f,
	TEXT("Overrides the vertical field of view for HMD rendering, values are in degrees and values less than 0.0f disable the override. Default: -1.0f"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarHMDIPD(
	TEXT("PixelStreaming2.HMD.IPD"),
	-1.0f,
	TEXT("Overrides the HMD IPD (interpupillary distance), values are in centimeters and values less than 0.0f disable the override. Default: -1.0f"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarHMDProjectionOffsetX(
	TEXT("PixelStreaming2.HMD.ProjectionOffsetX"),
	-1.0f,
	TEXT("Overrides the left/right eye projection matrix x-offset, values are in clip space and values less than 0.0f disable the override. Default: -1.0f"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarHMDProjectionOffsetY(
	TEXT("PixelStreaming2.HMD.ProjectionOffsetY"),
	-1.0f,
	TEXT("Overrides the left-right eye projection matrix y-offset, values are in clip space and values less than 0.0f disable the override. Default: -1.0f"),
	ECVF_Default);
// End HMD CVars

// Begin Input CVars
TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarInputAllowConsoleCommands(
	TEXT("PixelStreaming2.AllowPixelStreamingCommands"),
	false,
	TEXT("If true browser can send consoleCommand payloads that execute in UE's console. Default: false"),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarInputKeyFilter(
	TEXT("PixelStreaming2.KeyFilter"),
	"",
	TEXT("Comma separated list of keys to ignore from streaming clients. Default: \"\""),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Var) 
		{ 
			if (FDelegates* Delegations = Delegates())
			{
				Delegations->OnInputKeyFilterChanged.Broadcast(Var); 
			}
		}),
	ECVF_Default);
// End Input CVars

// Begin Signalling Server CVars
TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarAllowedOrigins(
	TEXT("PixelStreaming2.InternalSignallingServer.AllowedOrigins"),
	TEXT("http://127.0.0.1,https://127.0.0.1"),
	TEXT("Comma-separated list of allowed origins for WebSocket connection filtering. Supports wildcards (*). Use '*' to allow all origins. Leave empty to disable filtering."),
	ECVF_Default);
// End Signalling Server CVars

TArray<EVideoCodec> UPixelStreaming2PluginSettings::GetCodecPreferences()
{
	TArray<EVideoCodec> OutCodecPreferences;
	FString				StringOptions = UPixelStreaming2PluginSettings::CVarWebRTCCodecPreferences.GetValueOnAnyThread();
	if (StringOptions.IsEmpty())
	{
		return OutCodecPreferences;
	}

	TArray<FString> CodecArray;
	StringOptions.ParseIntoArray(CodecArray, TEXT(","), true);
	for (const FString& CodecString : CodecArray)
	{
		uint64 EnumIndex = StaticEnum<EVideoCodec>()->GetIndexByNameString(CodecString);
		checkf(EnumIndex != INDEX_NONE, TEXT("CVar was not containing valid enum string"));
		OutCodecPreferences.Add(static_cast<EVideoCodec>(StaticEnum<EVideoCodec>()->GetValueByIndex(EnumIndex)));
	}

	return OutCodecPreferences;
}

EPortAllocatorFlags UPixelStreaming2PluginSettings::GetPortAllocationFlags()
{
	EPortAllocatorFlags OutPortAllocatorFlags = EPortAllocatorFlags::None;
	FString				StringOptions = UPixelStreaming2PluginSettings::CVarWebRTCPortAllocatorFlags.GetValueOnAnyThread();
	if (StringOptions.IsEmpty())
	{
		return OutPortAllocatorFlags;
	}

	TArray<FString> FlagArray;
	StringOptions.ParseIntoArray(FlagArray, TEXT(","), true);
	int OptionCount = FlagArray.Num();
	while (OptionCount > 0)
	{
		FString Flag = FlagArray[OptionCount - 1];

		// Flags must match EpicRtc\Include\epic_rtc\core\connection_config.h
		if (Flag == "DISABLE_UDP")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableUdp;
		}
		else if (Flag == "DISABLE_STUN")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableStun;
		}
		else if (Flag == "DISABLE_RELAY")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableRelay;
		}
		else if (Flag == "DISABLE_TCP")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableTcp;
		}
		else if (Flag == "ENABLE_IPV6")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::EnableIPV6;
		}
		else if (Flag == "ENABLE_SHARED_SOCKET")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::EnableSharedSocket;
		}
		else if (Flag == "ENABLE_STUN_RETRANSMIT_ATTRIBUTE")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::EnableStunRetransmitAttribute;
		}
		else if (Flag == "DISABLE_ADAPTER_ENUMERATION")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableAdapterEnumeration;
		}
		else if (Flag == "DISABLE_DEFAULT_LOCAL_CANDIDATE")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableDefaultLocalCandidate;
		}
		else if (Flag == "DISABLE_UDP_RELAY")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableUdpRelay;
		}
		else if (Flag == "DISABLE_COSTLY_NETWORKS")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableCostlyNetworks;
		}
		else if (Flag == "ENABLE_IPV6_ON_WIFI")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::EnableIPV6OnWifi;
		}
		else if (Flag == "ENABLE_ANY_ADDRESS_PORTS")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::EnableAnyAddressPort;
		}
		else if (Flag == "DISABLE_LINK_LOCAL_NETWORKS")
		{
			OutPortAllocatorFlags |= EPortAllocatorFlags::DisableLinkLocalNetworks;
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Unknown port allocator flag: {0}", Flag);
		}
		OptionCount--;
	}

	return OutPortAllocatorFlags;
}

void SetPortAllocationCVarFromProperty(UObject* This, FProperty* Property)
{
	const FNumericProperty* EnumProperty = CastField<const FNumericProperty>(Property);
	void*					PropertyAddress = EnumProperty->ContainerPtrToValuePtr<void>(This);
	EPortAllocatorFlags		CurrentValue = static_cast<EPortAllocatorFlags>(EnumProperty->GetSignedIntPropertyValue(PropertyAddress));

	FString CVarString = TEXT("");

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableUdp))
	{
		CVarString += "DISABLE_UDP,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableStun))
	{
		CVarString += "DISABLE_STUN,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableRelay))
	{
		CVarString += "DISABLE_RELAY,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableTcp))
	{
		CVarString += "DISABLE_TCP,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::EnableIPV6))
	{
		CVarString += "ENABLE_IPV6,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::EnableSharedSocket))
	{
		CVarString += "ENABLE_SHARED_SOCKET,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::EnableStunRetransmitAttribute))
	{
		CVarString += "ENABLE_STUN_RETRANSMIT_ATTRIBUTE,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableAdapterEnumeration))
	{
		CVarString += "DISABLE_ADAPTER_ENUMERATION,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableDefaultLocalCandidate))
	{
		CVarString += "DISABLE_DEFAULT_LOCAL_CANDIDATE,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableUdpRelay))
	{
		CVarString += "DISABLE_UDP_RELAY,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableCostlyNetworks))
	{
		CVarString += "DISABLE_COSTLY_NETWORKS,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::EnableIPV6OnWifi))
	{
		CVarString += "ENABLE_IPV6_ON_WIFI,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::EnableAnyAddressPort))
	{
		CVarString += "ENABLE_ANY_ADDRESS_PORTS,";
	}

	if (static_cast<bool>(CurrentValue & EPortAllocatorFlags::DisableLinkLocalNetworks))
	{
		CVarString += "DISABLE_LINK_LOCAL_NETWORKS,";
	}

	UPixelStreaming2PluginSettings::CVarWebRTCPortAllocatorFlags.AsVariable()->Set(*CVarString, ECVF_SetByProjectSetting);
}

void SetPortAllocationCVarAndPropertyFromValue(UObject* This, FProperty* Property, const FString& Value)
{
	UPixelStreaming2PluginSettings::CVarWebRTCPortAllocatorFlags.AsVariable()->Set(*Value, ECVF_SetByCommandline);

	const FNumericProperty* EnumProperty = CastField<const FNumericProperty>(Property);
	int64*					PropertyAddress = EnumProperty->ContainerPtrToValuePtr<int64>(This);
	*PropertyAddress = static_cast<int64>(UPixelStreaming2PluginSettings::GetPortAllocationFlags());
}

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarWebRTCPortAllocatorFlags(
	TEXT("PixelStreaming2.WebRTC.PortAllocatorFlags"),
	TEXT(""),
	TEXT("Sets the WebRTC port allocator flags. Format:\"DISABLE_UDP,DISABLE_STUN,...\""),
	ECVF_Default);

TAutoConsoleVariable<int> UPixelStreaming2PluginSettings::CVarWebRTCMinPort(
	TEXT("PixelStreaming2.WebRTC.MinPort"),
	49152, // Default according to RFC5766
	TEXT("Sets the minimum usable port for the WebRTC port allocator. Default: 49152"),
	ECVF_Default);

TAutoConsoleVariable<int> UPixelStreaming2PluginSettings::CVarWebRTCMaxPort(
	TEXT("PixelStreaming2.WebRTC.MaxPort"),
	65535, // Default according to RFC5766
	TEXT("Sets the maximum usable port for the WebRTC port allocator. Default: 65535"),
	ECVF_Default);

TAutoConsoleVariable<FString> UPixelStreaming2PluginSettings::CVarWebRTCFieldTrials(
	TEXT("PixelStreaming2.WebRTC.FieldTrials"),
	TEXT(""),
	TEXT("Sets the WebRTC field trials string. Format:\"TRIAL1/VALUE1/TRIAL2/VALUE2/\""),
	ECVF_Default);

TAutoConsoleVariable<bool> UPixelStreaming2PluginSettings::CVarWebRTCDisableFrameDropper(
	TEXT("PixelStreaming2.WebRTC.DisableFrameDropper"),
	false,
	TEXT("Disables the WebRTC internal frame dropper using the field trial WebRTC-FrameDropper/Disabled/"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarWebRTCVideoPacingMaxDelay(
	TEXT("PixelStreaming2.WebRTC.VideoPacing.MaxDelay"),
	-1.0f,
	TEXT("Enables the WebRTC-Video-Pacing field trial and sets the max delay (ms) parameter. Default: -1.0f (values below zero are discarded.)"),
	ECVF_Default);

TAutoConsoleVariable<float> UPixelStreaming2PluginSettings::CVarWebRTCVideoPacingFactor(
	TEXT("PixelStreaming2.WebRTC.VideoPacing.Factor"),
	-1.0f,
	TEXT("Enables the WebRTC-Video-Pacing field trial and sets the video pacing factor parameter. Larger values are more lenient on larger bitrates. Default: -1.0f (values below zero are discarded.)"),
	ECVF_Default);

UPixelStreaming2PluginSettings::FDelegates* UPixelStreaming2PluginSettings::DelegateSingleton = nullptr;

UPixelStreaming2PluginSettings::~UPixelStreaming2PluginSettings()
{
	DelegateSingleton = nullptr;
}

FName UPixelStreaming2PluginSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UPixelStreaming2PluginSettings::GetSectionText() const
{
	return NSLOCTEXT("PixelStreaming2Plugin", "PixelStreaming2SettingsSection", "PixelStreaming2");
}

void UPixelStreaming2PluginSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!PropertyChangedEvent.Property)
	{
		return;
	}

	FString PropertyName = PropertyChangedEvent.Property->GetNameCPP();

	FString CVarName;
	if (CVarName = FindCVarFromProperty(GetCmdArg, PropertyName); !CVarName.IsEmpty())
	{
		if (PropertyName == "WebRTCPortAllocatorFlags")
		{
			SetPortAllocationCVarFromProperty(this, PropertyChangedEvent.Property);
		}
		else if (PropertyName == "Codec" || PropertyName == "ScalabilityMode" || PropertyName == "EnableSimulcast")
		{
			VerifyVideoSettings();
		}
		else
		{
			SetCVarFromProperty(CVarName, PropertyChangedEvent.Property);
		}
	}
	else if (CVarName = FindCVarFromProperty(GetMappedCmdArg, PropertyName); !CVarName.IsEmpty())
	{
		SetCVarFromProperty(CVarName, PropertyChangedEvent.Property);
	}
}

void UPixelStreaming2PluginSettings::VerifyVideoSettings()
{
	FProperty*	   SimulcastProperty = GetClass()->FindPropertyByName(TEXT("EnableSimulcast"));
	FBoolProperty* SimulcastBoolProperty = CastField<FBoolProperty>(SimulcastProperty);
	bool		   bSimulcastEnabled = SimulcastBoolProperty->GetPropertyValue_InContainer(this);

	FProperty*	  CodecProperty = GetClass()->FindPropertyByName(TEXT("Codec"));
	FStrProperty* CodecStrProperty = CastField<FStrProperty>(CodecProperty);
	FString		  CodecString = CodecStrProperty->GetPropertyValue_InContainer(this);

	FProperty*	  ScalabilityModeProperty = GetClass()->FindPropertyByName(TEXT("ScalabilityMode"));
	FStrProperty* ScalabilityModeStrProperty = CastField<FStrProperty>(ScalabilityModeProperty);
	FString		  ScalabilityModeString = ScalabilityModeStrProperty->GetPropertyValue_InContainer(this);

	if (bSimulcastEnabled)
	{
		if (CodecString != TEXT("H264") && CodecString != TEXT("VP8"))
		{
			UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Default codec ({0}) doesn't support simulcast! Resetting default codec to H.264", CodecString);
			CodecStrProperty->SetPropertyValue_InContainer(this, TEXT("H264"));
		}
	}

	CodecString = CodecStrProperty->GetPropertyValue_InContainer(this);
	if ((CodecString == TEXT("H264") || CodecString == TEXT("VP8"))
		&& (ScalabilityModeString != TEXT("L1T1") && ScalabilityModeString != TEXT("L1T2") && ScalabilityModeString != TEXT("L1T3")))
	{
		UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Default codec ({0}) doesn't support the {1} scalability mode! Resetting scalability mode to L1T1", CodecString, ScalabilityModeString);
		ScalabilityModeStrProperty->SetPropertyValue_InContainer(this, TEXT("L1T1"));
	}

	SetCVarFromProperty(FindCVarFromProperty(GetCmdArg, SimulcastProperty->GetNameCPP()), SimulcastProperty);
	SetCVarFromProperty(FindCVarFromProperty(GetCmdArg, CodecProperty->GetNameCPP()), CodecProperty);
	SetCVarFromProperty(FindCVarFromProperty(GetCmdArg, ScalabilityModeProperty->GetNameCPP()), ScalabilityModeProperty);
}
#endif

void UPixelStreaming2PluginSettings::SetCVarAndPropertyFromValue(const FString& CVarName, FProperty* Property, const FString& Value)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
	if (!CVar)
	{
		UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Failed to find CVar: {0}", CVarName);
		return;
	}

	if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property); ByteProperty != NULL && ByteProperty->Enum != NULL)
	{
		CVar->Set(FCString::Atoi(*Value), ECVF_SetByCommandline);
		ByteProperty->SetPropertyValue_InContainer(this, FCString::Atoi(*Value));
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [{2}] from command line", CVarName, Property->GetNameCPP(), FCString::Atoi(*Value));
	}
	else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		int64 EnumIndex = EnumProperty->GetEnum()->GetIndexByNameString(Value.Replace(TEXT("_"), TEXT("")));
		if (EnumIndex != INDEX_NONE)
		{
			CVar->Set(*EnumProperty->GetEnum()->GetNameStringByIndex(EnumIndex), ECVF_SetByCommandline);

			FNumericProperty* UnderlyingProp = EnumProperty->GetUnderlyingProperty();
			int64*			  PropertyAddress = EnumProperty->ContainerPtrToValuePtr<int64>(this);
			*PropertyAddress = EnumProperty->GetEnum()->GetValueByIndex(EnumIndex);

			UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [{2}] from command line", CVarName, Property->GetNameCPP(), EnumProperty->GetEnum()->GetNameStringByIndex(EnumIndex));
		}
		else
		{
			UE_LOGFMT(LogPixelStreaming2Settings, Warning, "{0} is not a valid enum value for {1}", Value, EnumProperty->GetEnum()->CppType);
		}
	}
	else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		bool bValue = false;
		if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
		{
			bValue = true;
		}
		else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
		{
			bValue = false;
		}
		CVar->Set(bValue, ECVF_SetByCommandline);
		BoolProperty->SetPropertyValue_InContainer(this, bValue);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [{2}] from command line", CVarName, Property->GetNameCPP(), bValue);
	}
	else if (FIntProperty* IntProperty = CastField<FIntProperty>(Property))
	{
		CVar->Set(FCString::Atoi(*Value), ECVF_SetByCommandline);
		IntProperty->SetPropertyValue_InContainer(this, FCString::Atoi(*Value));
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [{2}] from command line", CVarName, Property->GetNameCPP(), FCString::Atoi(*Value));
	}
	else if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
	{
		CVar->Set(FCString::Atof(*Value), ECVF_SetByCommandline);
		FloatProperty->SetPropertyValue_InContainer(this, FCString::Atof(*Value));
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [{2}] from command line", CVarName, Property->GetNameCPP(), FCString::Atof(*Value));
	}
	else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property))
	{
		CVar->Set(*Value, ECVF_SetByCommandline);
		StringProperty->SetPropertyValue_InContainer(this, *Value);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [\"{2}\"] from command line", CVarName, Property->GetNameCPP(), Value);
	}
	else if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
	{
		CVar->Set(*Value, ECVF_SetByCommandline);
		NameProperty->SetPropertyValue_InContainer(this, FName(*Value));
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [\"{2}\"] from command line", CVarName, Property->GetNameCPP(), Value);
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		// TODO (william.belcher): Only FString array properties are currently supported
		CVar->Set(*Value, ECVF_SetByCommandline);

		TArray<FString> StringArray;
		Value.ParseIntoArray(StringArray, TEXT(","), true);

		TArray<FString>& Array = *ArrayProperty->ContainerPtrToValuePtr<TArray<FString>>(this);
		Array = StringArray;

		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] and Property [{1}] to [\"{2}\"] from command line", CVarName, Property->GetNameCPP(), Value);
	}
}

void UPixelStreaming2PluginSettings::SetCVarFromProperty(const FString& CVarName, FProperty* Property)
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
	if (!CVar)
	{
		UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Failed to find CVar: {0}", CVarName);
		return;
	}

	if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property); ByteProperty != NULL && ByteProperty->Enum != NULL)
	{
		CVar->Set(ByteProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [{1}] from Property [{2}]", CVarName, ByteProperty->GetPropertyValue_InContainer(this), Property->GetNameCPP());
	}
	else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		void* PropertyAddress = EnumProperty->ContainerPtrToValuePtr<void>(this);
		int64 CurrentValue = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(PropertyAddress);
		CVar->Set(*EnumProperty->GetEnum()->GetNameStringByValue(CurrentValue), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [{1}] from Property [{2}]", CVarName, EnumProperty->GetEnum()->GetNameStringByValue(CurrentValue), Property->GetNameCPP());
	}
	else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		CVar->Set(BoolProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [{1}] from Property [{2}]", CVarName, BoolProperty->GetPropertyValue_InContainer(this), Property->GetNameCPP());
	}
	else if (FIntProperty* IntProperty = CastField<FIntProperty>(Property))
	{
		CVar->Set(IntProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [{1}] from Property [{2}]", CVarName, IntProperty->GetPropertyValue_InContainer(this), Property->GetNameCPP());
	}
	else if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
	{
		CVar->Set(FloatProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [{1}] from Property [{2}]", CVarName, FloatProperty->GetPropertyValue_InContainer(this), Property->GetNameCPP());
	}
	else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property))
	{
		CVar->Set(*StringProperty->GetPropertyValue_InContainer(this), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [\"{1}\"] from Property [{2}]", CVarName, StringProperty->GetPropertyValue_InContainer(this), Property->GetNameCPP());
	}
	else if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
	{
		CVar->Set(*NameProperty->GetPropertyValue_InContainer(this).ToString(), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [\"{1}\"] from Property [{2}]", CVarName, NameProperty->GetPropertyValue_InContainer(this), Property->GetNameCPP());
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		// TODO (william.belcher): Only FString array properties are currently supported
		TArray<FString> Array = *ArrayProperty->ContainerPtrToValuePtr<TArray<FString>>(this);
		CVar->Set(*FString::Join(Array, TEXT(",")), ECVF_SetByProjectSetting);
		UE_LOGFMT(LogPixelStreaming2Settings, Log, "Setting CVar [{0}] to [\"{1}\"] from Property [{2}]", CVarName, FString::Join(Array, TEXT(",")), Property->GetNameCPP());
	}
}

void UPixelStreaming2PluginSettings::InitializeCVarsFromProperties()
{
	UE_LOGFMT(LogPixelStreaming2Settings, Log, "Initializing CVars from ini");
	for (FProperty* Property = GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}

		// Handle the majority of commandline argument
		if (Property->GetNameCPP() == "WebRTCPortAllocatorFlags")
		{
			SetPortAllocationCVarFromProperty(this, Property);
			continue;
		}

		FString CVarName;
		if (CVarName = FindCVarFromProperty(GetCmdArg, Property->GetNameCPP()); !CVarName.IsEmpty())
		{
			SetCVarFromProperty(CVarName, Property);
			continue;
		}
		else if (CVarName = FindCVarFromProperty(GetMappedCmdArg, Property->GetNameCPP()); !CVarName.IsEmpty())
		{
			SetCVarFromProperty(CVarName, Property);
			continue;
		}
	}
}

void UPixelStreaming2PluginSettings::ValidateCommandLineArgs()
{
	FString CommandLine = FCommandLine::Get();

	TArray<FString> CommandArray;
	CommandLine.ParseIntoArray(CommandArray, TEXT(" "), true);

	for (FString Command : CommandArray)
	{
		Command.RemoveFromStart(TEXT("-"));
		if (!Command.StartsWith("PixelStreaming"))
		{
			continue;
		}

		// Get the pure command line arg from an arg that contains an '=', eg PixelStreamingURL=
		FString CurrentCommandLineArg = Command;
		if (Command.Contains("="))
		{
			Command.Split(TEXT("="), &CurrentCommandLineArg, nullptr);
		}

		bool bValidArg = false;
		for (const TPair<FString, FString>& Pair : GetCmdArg)
		{
			FString ValidCommandLineArg = ConsoleVariableToCommandArgParam(Pair.Key);
			if (CurrentCommandLineArg == ValidCommandLineArg)
			{
				bValidArg = true;
				break;
			}
		}

		if (!bValidArg)
		{
			for (const TPair<FString, FString>& Pair : GetMappedCmdArg)
			{
				FString ValidCommandLineArg = ConsoleVariableToCommandArgParam(Pair.Key);
				if (CurrentCommandLineArg == ValidCommandLineArg)
				{
					bValidArg = true;
					break;
				}
			}
		}

		if (!bValidArg)
		{
			for (const TPair<FString, FString>& Pair : GetLegacyCmdArg)
			{
				FString ValidCommandLineArg = ConsoleVariableToCommandArgParam(Pair.Key);
				if (CurrentCommandLineArg == ValidCommandLineArg)
				{
					bValidArg = true;
					break;
				}
			}
		}

		if (!bValidArg)
		{
			UE_LOGFMT(LogPixelStreaming2Settings, Warning, "Unknown PixelStreaming command line arg: {0}", CurrentCommandLineArg);
		}
	}
}

void UPixelStreaming2PluginSettings::ParseCommandlineArgs()
{
	UE_LOGFMT(LogPixelStreaming2Settings, Verbose, "Updating CVars and properties with command line args");
	for (const TPair<FString, FString>& Pair : GetCmdArg)
	{
		FString CVarString = Pair.Key;
		FString PropertyName = Pair.Value;

		FProperty* Property = GetClass()->FindPropertyByName(FName(*PropertyName));
		if (!Property || !Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}

		if (PropertyName == "WebRTCPortAllocatorFlags")
		{
			FString ConsoleString;
			if (FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(CVarString), ConsoleString))
			{
				SetPortAllocationCVarAndPropertyFromValue(this, Property, ConsoleString);
			}
			continue;
		}

		// Handle a directly parsable commandline
		FString ConsoleString;
		if (FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(CVarString), ConsoleString))
		{
			SetCVarAndPropertyFromValue(CVarString, Property, ConsoleString);
		}
		else if (FParse::Param(FCommandLine::Get(), *ConsoleVariableToCommandArgParam(CVarString)))
		{
			SetCVarAndPropertyFromValue(CVarString, Property, TEXT("true"));
		}
	}

	for (const TPair<FString, FString>& Pair : GetMappedCmdArg)
	{
		FString CVarString = Pair.Key;
		FString PropertyName = Pair.Value;

		FProperty* Property = GetClass()->FindPropertyByName(FName(*PropertyName));
		if (!Property || !Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}

		// Handle a directly parsable commandline
		FString ConsoleString;
		if (FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(CVarString), ConsoleString))
		{
			SetCVarAndPropertyFromValue(CVarString, Property, ConsoleString);
		}
	}
}

void UPixelStreaming2PluginSettings::ParseLegacyCommandlineArgs()
{
	FString SignallingServerIP;
	FString SignallingServerPort;

	for (const TPair<FString, FString>& Pair : GetLegacyCmdArg)
	{
		FString LegacyCVarString = Pair.Key;
		FString PropertyName = Pair.Value;

		FProperty* Property = GetClass()->FindPropertyByName(FName(*PropertyName));
		if (!Property || !Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}

		FString NewCVarString;
		if (FString CmdArgCVar = FindCVarFromProperty(GetCmdArg, PropertyName); !CmdArgCVar.IsEmpty())
		{
			NewCVarString = CmdArgCVar;
		}
		else if (FString MappedCmdArgCVar = FindCVarFromProperty(GetMappedCmdArg, PropertyName); !MappedCmdArgCVar.IsEmpty())
		{
			NewCVarString = MappedCmdArgCVar;
		}
		else
		{
			continue;
		}

		if (LegacyCVarString == "PixelStreaming2.Encoder.MinQp")
		{
			int32 MinQP;
			if (FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(LegacyCVarString), MinQP))
			{
				int32 ScaledQuality = 100.0f * (1.0f - (FMath::Clamp<int32>(MinQP, 0, 51) / 51.0f));
				SetCVarAndPropertyFromValue(NewCVarString, Property, FString::Printf(TEXT("%d"), ScaledQuality));
				UE_LOGFMT(LogPixelStreaming2Settings, Warning, "PixelStreamingEncoderMinQp is a legacy setting, converted to PixelStreamingEncoderMaxQuality={0}", CVarEncoderMaxQuality.GetValueOnAnyThread());
				continue;
			}
		}
		else if (LegacyCVarString == "PixelStreaming2.Encoder.MaxQp")
		{
			int32 MaxQP;
			if (FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(LegacyCVarString), MaxQP))
			{
				int32 ScaledQuality = 100.0f * (1.0f - (FMath::Clamp<int32>(MaxQP, 0, 51) / 51.0f));
				SetCVarAndPropertyFromValue(NewCVarString, Property, FString::Printf(TEXT("%d"), ScaledQuality));
				UE_LOGFMT(LogPixelStreaming2Settings, Warning, "PixelStreamingEncoderMaxQp is a legacy setting, converted to PixelStreamingEncoderMinQuality={0}", CVarEncoderMinQuality.GetValueOnAnyThread());
				continue;
			}
		}
		else if (LegacyCVarString == "PixelStreaming2.IP" || LegacyCVarString == "PixelStreaming2.Port")
		{
			if (LegacyCVarString == "PixelStreaming2.IP")
			{
				FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(LegacyCVarString), SignallingServerIP);
			}
			else if (LegacyCVarString == "PixelStreaming2.Port")
			{
				FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(LegacyCVarString), SignallingServerPort);
			}

			if (!SignallingServerIP.IsEmpty() && !SignallingServerPort.IsEmpty())
			{
				FString LegacyUrl = TEXT("ws://") + SignallingServerIP + TEXT(":") + SignallingServerPort;
				SetCVarAndPropertyFromValue(NewCVarString, Property, LegacyUrl);
				UE_LOGFMT(LogPixelStreaming2Settings, Warning, "PixelStreamingIP and PixelStreamingPort are legacy settings converted to -PixelStreamingConnectionURL={0}", CVarConnectionURL.GetValueOnAnyThread());
			}

			continue;
		}

		FString ConsoleString;
		if (FParse::Value(FCommandLine::Get(), *ConsoleVariableToCommandArgValue(LegacyCVarString), ConsoleString))
		{
			SetCVarAndPropertyFromValue(NewCVarString, Property, ConsoleString);
		}
		else if (FParse::Param(FCommandLine::Get(), *ConsoleVariableToCommandArgParam(LegacyCVarString)))
		{
			SetCVarAndPropertyFromValue(NewCVarString, Property, TEXT("true"));
		}
		else
		{
			continue;
		}

		UE_LOGFMT(LogPixelStreaming2Settings, Warning, "{0} is a legacy setting and has been converted to {1}", ConsoleVariableToCommandArgParam(LegacyCVarString), ConsoleVariableToCommandArgParam(NewCVarString));
	}

	ParseLegacyCommandLineOption(TEXT("PixelStreamingDebugDumpFrame"), CVarEncoderDebugDumpFrame);
	// End legacy PixelStreaming command line args
}

void UPixelStreaming2PluginSettings::PostInitProperties()
{
	Super::PostInitProperties();

	UE_LOGFMT(LogPixelStreaming2Settings, Log, "Initialising Pixel Streaming settings.");

	// Set all the CVars to reflect the state of the ini
	InitializeCVarsFromProperties();

	// Validate command line args to log if they're invalid
	ValidateCommandLineArgs();

	// Update CVars and properties based on command line args
	ParseCommandlineArgs();

	// Handle parsing of legacy command line args (such as -PixelStreamingUrl) after .ini and new commandline args.
	ParseLegacyCommandlineArgs();

	// These cvars don't have matching properties so need to be manually parsed
	ParseLegacyCommandLineOption(*ConsoleVariableToCommandArgParam(TEXT("PixelStreaming2.Encoder.DumpDebugFrames")), CVarEncoderDebugDumpFrame);
	ParseLegacyCommandLineOption(*ConsoleVariableToCommandArgParam(TEXT("PixelStreaming2.DumpDebugAudio")), CVarDebugDumpAudio);
}

UPixelStreaming2PluginSettings::FDelegates* UPixelStreaming2PluginSettings::Delegates()
{
	if (DelegateSingleton == nullptr && !IsEngineExitRequested())
	{
		DelegateSingleton = new UPixelStreaming2PluginSettings::FDelegates();
		return DelegateSingleton;
	}
	return DelegateSingleton;
}

TArray<FString> UPixelStreaming2PluginSettings::GetVideoCodecOptions() const
{
	FProperty*	   Property = GetClass()->FindPropertyByName(TEXT("EnableSimulcast"));
	FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);
	bool		   bSimulcastEnabled = BoolProperty->GetPropertyValue_InContainer(this);

	if (bSimulcastEnabled)
	{
		return {
			UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::H264),
			UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP8)
		};
	}

	return {
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::AV1),
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::H264),
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP8),
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP9)
	};
}

TArray<FString> UPixelStreaming2PluginSettings::GetScalabilityModeOptions() const
{
	FProperty*	  Property = GetClass()->FindPropertyByName(TEXT("Codec"));
	FStrProperty* StrProperty = CastField<FStrProperty>(Property);
	FString		  SelectedCodec = StrProperty->GetPropertyValue_InContainer(this);
	// H.264 and VP8 only support temporal scalability
	bool bRestrictModes = SelectedCodec == TEXT("H264") || SelectedCodec == TEXT("VP8");

	if (bRestrictModes)
	{
		return {
			UE::PixelStreaming2::GetCVarStringFromEnum(EScalabilityMode::L1T1),
			UE::PixelStreaming2::GetCVarStringFromEnum(EScalabilityMode::L1T2),
			UE::PixelStreaming2::GetCVarStringFromEnum(EScalabilityMode::L1T3),
		};
	}

	TArray<FString> ScalabilityModes;
	for (uint32 i = 0; i <= static_cast<uint32>(EScalabilityMode::None); i++)
	{
		ScalabilityModes.Add(UE::PixelStreaming2::GetCVarStringFromEnum(static_cast<EScalabilityMode>(i)));
	}

	return ScalabilityModes;
}

TArray<FString> UPixelStreaming2PluginSettings::GetWebRTCCodecPreferencesOptions() const
{
	TSet<FString> PossibleCodecs = {
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::AV1),
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::H264),
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP9),
		UE::PixelStreaming2::GetCVarStringFromEnum(EVideoCodec::VP8)
	};

	FProperty*		Property = GetClass()->FindPropertyByName(TEXT("WebRTCCodecPreferences"));
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
	TArray<FString> CurrentCodecArray = *ArrayProperty->ContainerPtrToValuePtr<TArray<FString>>(this);

	for (FString VideoCodec : CurrentCodecArray)
	{
		PossibleCodecs.Remove(VideoCodec);
	}

	return PossibleCodecs.Array();
}

TArray<FString> UPixelStreaming2PluginSettings::GetDefaultStreamerTypeOptions() const
{
	return IPixelStreaming2StreamerFactory::GetAvailableFactoryTypes();
}
