// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FPixelStreaming2SettingsModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FPixelStreaming2SettingsModule, PixelStreaming2Settings);
