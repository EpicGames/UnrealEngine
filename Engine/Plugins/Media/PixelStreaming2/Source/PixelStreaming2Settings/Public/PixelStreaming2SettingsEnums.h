// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelStreaming2SettingsEnums.generated.h"

/**
 * @brief The possible stream types Editor Streaming supports. 
 * 
 */
UENUM()
enum class EPixelStreaming2EditorStreamTypes : uint8
{
	/** `LevelEditorViewport` will stream just the level editor */
	LevelEditorViewport = 0,
	/** `Editor` will stream the full editor and any of its child windows */
	Editor = 1
};